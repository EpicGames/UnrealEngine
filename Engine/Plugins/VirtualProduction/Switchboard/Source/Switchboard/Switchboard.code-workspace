{
	"folders": [
		{
			"path": "."
		},
	],
	"launch":
	{
		"version": "0.2.0",
		"configurations": [
			{
				"presentation": {
					"group": "00_Switchboard",
					"order": 0
				},
				"name": "Switchboard",
				"type": "debugpy",
				"request": "launch",
				"module": "switchboard",
				"cwd": "${workspaceFolder}",
				"justMyCode": false,
				"env": {
					"PYTHONASYNCIODEBUG": "1",
					//"QT_LOGGING_RULES": "qt.pyside.libpyside.warning=true",
				}
			},
			{
				"presentation": {
					"group": "01_sb_setup",
					"order": 0
				},
				"name": "sb_setup.py verify",
				"type": "debugpy",
				"request": "launch",
				"program": "${workspaceFolder}\\sb_setup.py",
				"args": ["verify"],
				"cwd": "${workspaceFolder}",
				"justMyCode": false
			}
		]
	},
	"settings":
	{
		"[python]": {
			"editor.rulers": [
				120
			]
		},
		"python.defaultInterpreterPath": "${workspaceFolder}\\..\\..\\..\\..\\..\\Extras\\ThirdPartyNotUE\\SwitchboardThirdParty\\Python\\Scripts\\python.exe",
		"python.analysis.extraPaths": [
			"${workspaceFolder}"
		],
		"python.testing.cwd": "${workspaceFolder}",
		"python.testing.unittestEnabled": true,
		"python.testing.unittestArgs": [
			"-v",
			"-s",
			"./tests",
			"-p",
			"test_*.py"
		],
		"pylint.args": [
			"--max-line-length=120",
			"--extension-pkg-allow-list=PySide6",
			"--disable=logging-fstring-interpolation,logging-not-lazy",
			"--disable=broad-exception-caught",
		],
	}
}