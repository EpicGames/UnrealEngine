By opening `Switchboard.code-workspace` in Visual Studio Code, you should find a number of helpful settings preconfigured, including:

  * A debug launch configuration, enabling you to launch Switchboard via F5, set breakpoints, and catch and inspect unhandled exceptions

    * > 🛈 **Note**: The default Python interpreter path is set to the standard Switchboard virtual environment. This ensures any prerequisite packages are available, but assumes you've run `switchboard.bat`/`switchboard.sh` at least once to complete the standard first-time setup prior to debugging

  * `unittest` integration

  * 120 column rulers and corresponding `pylint` override.
