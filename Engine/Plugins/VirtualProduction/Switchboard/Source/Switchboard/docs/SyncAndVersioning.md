# Introduction
When building Unreal Engine from source code, the use of the UnrealGameSync utility typically plays an essential role in the correct versioning of Unreal projects, particularly when it comes to package saving and loading compatibility determination.

However, while UGS assumes a single revision across the entire workspace, Switchboard enables users to choose which revision to sync their project and engine to separately. This capability has proven invaluable in virtual production workflows, enabling users to ensure stability by freezing the bulk of the code at a known-good version, while still permitting them to iterate on content and disseminate project updates in a fast-paced stage environment.

With a handful of notable exceptions, Switchboard (and its sync script `sbl_helper.py`) endeavors to be consistent with how UnrealGameSync handles versioning. This document is intended to provide a quick overview of some of the salient details, caveats, and rationale for places where we've diverged in behavior.

> ⚠️ **Warning**: For reasons which will be elaborated upon throughout this document, UnrealGameSync should not be used in a workspace managed by Switchboard, and vice versa. UnrealGameSync relies on the assumption that its cached state remains valid, including but not limited to the last changelist synced, and sync filters being consistently applied.
>
> One possible failure mode is that syncing with Switchboard can cause files that would otherwise be excluded via UGS sync filters to be re-added to the workspace, but a subsequent UGS sync using filters would neither update nor remove them, leading to version skew.


# Background: UnrealGameSync
This section describes how some of the relevant systems work using standard UGS workflows.

### `Build.version`
Stored alongside the engine is a JSON file, `Engine/Build/Build.version`, which contains several fields describing aspects of the Unreal Engine version.

Here's an example of how the file is stored in version control in the UE5 Main stream/branch at the time of writing:
```json
{
	"MajorVersion": 5,
	"MinorVersion": 6,
	"PatchVersion": 0,
	"Changelist": 0,
	"CompatibleChangelist": 0,
	"IsLicenseeVersion": 0,
	"IsPromotedBuild": 0,
	"BranchName": "UE5"
}
```

Whereas this is how it appears in UE 5.5.3 installed via the Epic Games Launcher:
```json
{
	"MajorVersion": 5,
	"MinorVersion": 5,
	"PatchVersion": 3,
	"Changelist": 39772772,
	"CompatibleChangelist": 37670630,
	"IsLicenseeVersion": 0,
	"IsPromotedBuild": 1,
	"BranchName": "++UE5+Release-5.5"
}
```

This file is treated as an input to the build process, and many of the values become preprocessor definitions which are "baked into" the compiled binaries.

While some fields are manually updated by editing the file as it's stored in Perforce as one might expect, other fields are automatically rewritten by UnrealGameSync (or Switchboard) on the local machine as part of every sync operation performed. Here's a breakdown:

  * `MajorVersion`, `MinorVersion`, and `PatchVersion` are only updated manually by Epic at certain points during our development and release cycle.

  * `Changelist` is always rewritten following sync operations, and reflects whichever revision the user has selected to sync to. **Notably, this value is used internally by the Concert (Multi-User) code to attempt to ensure that everyone joining the session is doing so with the same base revision of the content, and is stored as part of the session.**

  * `CompatibleChangelist` is relevant when reading/writing uasset files, and used for determining whether a saved asset is compatible with the running engine code attempting to load it. The handling of this field is the most complex, and differs significantly depending on whether the file stored in Perforce has a value of 0:

    * If the value stored in Perforce is 0, which is typical of a mainline development stream/branch, then it will be updated on sync to reflect the most recent "code changelist"* less than or equal to the (potentially content-only) changelist the user has selected to sync to. Classification of a changelist as affecting code is based on the presence of certain modified file extensions (e.g. .h/.cpp).
      > **: Refer to the `p4_latest_code_change` function in `p4_utils.py` for details.*

    * If this value is NOT 0, UnrealGameSync will NOT update it automatically, and will instead use it as-is. This is indicative of a release stream/branch which is in hotfix/maintenance mode. The value will have been manually updated to match the changelist that corresponds to the initial (5.x.0) release, and reflects the manual scrutiny Epic applies internally when vetting changes for inclusion in hotfix releases in order to maintain serialization and ABI compatibility.

  * `IsLicenseeVersion` is a boolean flag that's updated on sync to reflect the presence or absence of specific files in the engine source tree that are only available to Epic employees. This in turn sets a single bit causing `CompatibleChangelist` to be interpreted differently. (TODO: Elaborate on this.)

  * `IsPromotedBuild` is a boolean flag that's only set as part of Epic's internal release build process when producing official binary installations for the Epic Games Launcher.

  * `BranchName` is always updated on sync to reflect the Perforce stream name from which the workspace was synced, with plus characters substituted for forward slashes (e.g. `++UE5+Main` or `++UE5+Release-5.5`).

### `UnrealEditor.version`
Produced by UnrealBuildTool as part of a successful build is an analogous output artifact, written alongside the compiled executable, e.g. `Engine/Binaries/Win64/UnrealEditor.version`. This file is largely a duplicate of the `Build.version` file read as input at build time, with the addition of a `BuildId` string identifier appended to the end.

```json
{
	"...": "...",
	"BranchName": "++UE5+Main",
	"BuildId": "f28a59da-6a9b-46e1-93e3-d5843d7c446d"
}
```

The same `BuildId` value is also written to the `.modules` files alongside all modular DLLs, and intended to ensure all dynamically loaded modules have been built from the same code, and are therefore ABI-compatible. The exact nature of the value depends on whether a source build or an official EGL binary build is being used:

  * For source builds, it's set to a new randomly-generated UUID on each build.

  * For launcher builds, this value is overridden* to the same `CompatibleChangelist` number discussed above, corresponding to the initial minor (5.x.0) version release, and with the same rationale: a great deal of manual scrutiny is applied to ensure ABI compatibility between hotfixes.
    > **: Refer to the `FinalizeInstalledBuild` UnrealAutomationTool script for details.*

If you've ever been notified that modules are out of date and need to be rebuilt, a mismatch in the `BuildId` field is probably the reason (for example, syncing and build a project that does not use plugin X, then subsequently attempting to load a different project which would otherwise attempt to use existing stale binaries for plugin X).

### `$(ProjectName)Editor.target`
A related build output, known as a target receipt, is produced for the active project's editor target upon each successful build. The target receipt contains, among (many) other things, another copy of the same fields as `UnrealEditor.version`.

Importantly, however, while `UnrealEditor.version` is updated when building *any* project, target receipts are specific to a single project's editor target.

### `UnrealGameSync.ini`
UnrealGameSync coalesces configuration from a variety of possible .ini file locations distributed throughout the engine and project directory hierarchies*, but the two most commonly used locations are the stream-wide `/Engine/Programs/UnrealGameSync/UnrealGameSync.ini`, and the per-project `$(ProjectDir)/Build/UnrealGameSync.ini`.
> **: Refer to the `get_depot_config_paths` function in `ugs_utils.py` for details.*

By modifying these files, it's possible to influence the behavior of UGS both stream-wide, as well in the context of only the active project.

One such configuration directive which we have added support for in Switchboard is `AdditionalPathsToSync`. This allows users to configure paths that are neither part of the engine nor part of the project, but are synced simultaneously. We have found this valuable in virtual production workflows to have a content plugin that serves as a shared asset library used across multiple projects. (See also: "Additional Plugin Directories" functionality in the Plugin Browser.)


# Switchboard: Engine vs. Project Changelists
As mentioned previously, Switchboard supports workflows where the project may be synced to a different revision than the engine. Specifically:

  * Switchboard considers the "Engine" to consist of any files beneath the `/Engine` directory, combined with any loose files at the root of the workspace (e.g. `/GenerateProjectFiles.bat`). These are the paths that are enumerated for changes in order to populate the "Engine" changelist selection dropdown.

  * Switchboard considers the "Project" to consist of any files located in the directory containing the uproject file, as well as any paths referenced via the UnrealGameSync `AdditionalPathsToSync` directive. We consider these additional paths to be versioned "with the project" because it is frequently useful to sync external plugins to a later version than the engine for the same reasons (whereas plugins stored within the engine are already shared between projects). All of these paths are enumerated for changes when populating the "Project" changelist dropdown.

### Differences With UGS
Because of this, some versioning concepts designed with UnrealGameSync in mind do not map entirely cleanly to Switchboard:

  * Switchboard sets the `Changelist` field to whichever is the more recent of the selected engine/project changelists.

  * Switchboard sets the `CompatibleChangelist` field to the most recent code change in either the engine or the .uproject directory, but does NOT consider code changes in `AdditionalPathsToSync`.

  * Switchboard ignores any `CompatibleChangelist` value stored in Perforce, as hotfix release compatibility is not likely to be relevant to stage workflows, nor is it easy to guarantee when dealing with arbitrary project code. (This behavior can be overridden by passing `--ugs-versioning` to `sbl_helper.py`.)


# Switchboard: Sync and Build Indicators
Switchboard has per-device indicators of sync and build status, driven by various aspects of the device's workspace.

### Sync
In older versions of Switchboard, the indication of what changelist(s) a device is currently synced to were based on invoking `p4 cstat`, passing engine or project directory, followed by a `...#have` wildcard + revision specifier. However, this approach left a number of edge cases unhandled, such as failing to reflect subsets of files synced to older revisions, changelists consisting entirely of file deletions, or changelists only affecting "external" paths referenced via `AdditionalPathsToSync`.

As of UE 5.7, `sbl_helper.py` now implements a `syncstatus` command which is used instead. During each `sbl_helper.py sync` command, we write a file (`.sblsync.json`) to both the engine and project directories, containing a record of the corresponding changelist the user had selected for sync, as well as a timestamp. Then, during subsequent `syncstatus` commands, we can simply "preview" a sync to that changelist, and rely on Perforce to tell us whether it would have taken any action to update the workspace (i.e. whether it's "dirty" and actually needs to be synced). If the workspace is "dirty", this will be indicated by an asterisk next to the changelist indicators in Switchboard.

The changelist returned by the command is compared against the one selected from the corresponding Switchboard combo box. If there's a mismatch, the corresponding changelist indicator and the sync button will both be highlighted, indicating the device needs to be synced to the desired changelist.

### Build
Similarly, Switchboard will indicate if a device has not yet built the changelist it's currently synced to. For this, Switchboard relies on a few relevant facts:

  * The `Build.version` file will be updated on every sync performed, with both the changelist selected for sync, as well as the corresponding code changelist (`Changelist` and `CompatibleChangelist`, respectively).

  * The `UnrealEditor.version` file, along with any shared engine modules, will be updated upon building *any* project.

  * The `$(ProjectName)Editor.target` file will be updated to match on each build of a *specific* project.

By comparing these fields across all three files, we can be relatively confident that the relevant build outputs are up-to-date with the last sync performed in the workspace. Any mismatch will cause the build button to be highlighted, and the tooltip will contain the relevant details.
