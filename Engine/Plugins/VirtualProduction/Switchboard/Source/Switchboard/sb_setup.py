# Copyright Epic Games, Inc. All Rights Reserved.

from __future__ import annotations

import argparse
import importlib
import io
import json
import logging
import os
import pathlib
import shutil
import stat
import subprocess
import sys
from typing import Any, Dict, Optional, Set, Sequence
import venv


SCRIPT_NAME = os.path.basename(__file__)
__version__ = '1.0.0'


def find_engine_ancestor_dir(start: pathlib.Path) -> pathlib.Path:
    ''' Searches parent directories recursively for SB prerequisites. '''
    search = start
    while search != search.parent:
        sb_thirdparty_subdir = (
            search / 'Extras/ThirdPartyNotUE/SwitchboardThirdParty')

        if sb_thirdparty_subdir.is_dir():
            return search
        else:
            search = search.parent

    raise RuntimeError("Couldn't find prerequisites in ancestor directories")


SWITCHBOARD_PATH = pathlib.Path(__file__).parent
ENGINE_PATH = find_engine_ancestor_dir(SWITCHBOARD_PATH)
THIRDPARTY_PATH = ENGINE_PATH / 'Extras/ThirdPartyNotUE'
SB_THIRDPARTY_PATH = THIRDPARTY_PATH / 'SwitchboardThirdParty'

CWRSYNC_SRC_DIR = THIRDPARTY_PATH / 'cwrsync'
CWRSYNC_DEST_DIR = SB_THIRDPARTY_PATH / 'cwrsync'
CWRSYNC_FSTAB_PATH = CWRSYNC_DEST_DIR / 'etc/fstab'
CWRSYNC_FSTAB_CONTENTS = '''\
# This is equivalent to the default, except for the addition of the "noacl"
# option, which skips any attempt by Cygwin to store POSIX permissions via the
# Windows host's NTFS access control lists.
none /cygdrive cygdrive binary,posix=0,user,noacl 0 0
'''


class SbEnvBuilder(venv.EnvBuilder):
    def __init__(self, *args, **kwargs):
        self.post_setup_succeeded = False
        super().__init__(*args, **kwargs)

    def post_setup(self, context):
        try:
            logging.info('post_setup() - calling install_scripts()')
            self.install_scripts(context, str(
                                 SWITCHBOARD_PATH / 'venv_install_scripts'))

            logging.info('post_setup() - invoking pip install')
            args = [context.env_exe, '-m', 'pip', 'install',
                    '-r', str(SB_THIRDPARTY_PATH / 'requirements.txt')]
            with subprocess.Popen(args, stdout=subprocess.PIPE,
                                  stderr=subprocess.STDOUT) as proc:
                for line in proc.stdout:
                    logging.info(f'pip> {line.decode().rstrip()}')

                exit_code = proc.wait()

                logging.info(f'pip install exited with code {exit_code}')

                if exit_code == 0:
                    self.post_setup_succeeded = True
        except Exception as exc:
            logging.error('SbEnvBuilder.post_setup() failed', exc_info=exc)


class SbSetup:
    @classmethod
    def build_parser(cls) -> argparse.ArgumentParser:
        parser = argparse.ArgumentParser()

        # Global options
        parser.add_argument(
            '--log-level', default='INFO',
            help='DEBUG, INFO (default), WARNING, ERROR, CRITICAL')

        subparsers = parser.add_subparsers(dest='action')
        subparsers.required = True

        # Install action
        install_parser = subparsers.add_parser('install')
        install_parser.add_argument(
            '--venv-dir', type=pathlib.Path, required=True,
            help='Path to virtual environment directory')

        # Verify action
        verify_parser = subparsers.add_parser('verify')
        verify_parser.add_argument(
            '--output-json', action='store_true',
            help='All output is returned in the form of a JSON object')

        return parser

    def __init__(self):
        self.parser = self.build_parser()

    def ensure_rsync_copied(self):
        if not sys.platform.startswith('win'):
            return

        if not (CWRSYNC_DEST_DIR / 'bin/rsync.exe').exists():
            logging.info('Copying cwrsync')
            shutil.copytree(CWRSYNC_SRC_DIR, CWRSYNC_DEST_DIR,
                            dirs_exist_ok=True)

        if not (CWRSYNC_FSTAB_PATH).exists():
            logging.info('Writing cwrsync fstab')
            with open(CWRSYNC_FSTAB_PATH, 'wt', encoding='utf-8') as fstab:
                fstab.write(CWRSYNC_FSTAB_CONTENTS)

    @staticmethod
    def is_venv(checkdir: pathlib.Path) -> bool:
        ''' Conservative estimate of whether `dir` contains a Python venv. '''
        if not checkdir.is_dir():
            return False

        # These lowercase names are canonical for non-Windows OSes, while
        # also serving as a case-insensitive normalization under Windows.
        expected_files: Set[str] = {'pyvenv.cfg', 'provision.log'}
        expected_dirs: Set[str] = {'include', 'lib'}
        if sys.platform.startswith('win'):
            expected_dirs.update(['scripts'])
        else:
            expected_dirs.update(['bin', 'lib64'])

        existing_files: Set[str] = set()
        existing_dirs: Set[str] = set()

        with os.scandir(checkdir) as scan:
            for entry in scan:
                name: str = entry.name
                if sys.platform.startswith('win'):
                    name = name.lower()

                if entry.is_dir():
                    existing_dirs.add(name)
                elif entry.is_file():
                    existing_files.add(name)

        unexpected_files = existing_files - expected_files
        unexpected_dirs = existing_dirs - expected_dirs

        if len(unexpected_files) or len(unexpected_dirs):
            unexpected = unexpected_dirs | unexpected_files
            logging.warning(
                f'is_venv(): Unexpected files/directories: {unexpected}')
            return False

        return True

    def run_install(self, options: argparse.Namespace) -> int:
        if sys.platform.startswith('win'):
            self.ensure_rsync_copied()

        venv_dir: pathlib.Path = options.venv_dir

        # abspath is workaround for https://bugs.python.org/issue38671
        venv_dir = pathlib.Path(os.path.abspath(venv_dir.resolve()))

        logging.info(f'VENV_DIR: {venv_dir}')

        dest_not_empty = False
        if venv_dir.exists():
            with os.scandir(venv_dir) as scan:
                if next(scan, None):
                    dest_not_empty = True

        if dest_not_empty:
            if self.is_venv(venv_dir):
                logging.info('Removing existing venv')

                # Read-only files copied into venv lead to "access denied"
                def rmtree_on_error(func, path, exc_info):
                    if sys.platform.startswith('win'):
                        stat_result = os.stat(path)
                        attrs = stat_result.st_file_attributes
                        if attrs & stat.FILE_ATTRIBUTE_READONLY:
                            mode = stat_result.st_mode
                            logging.debug(f'Removing read-only: {path}')
                            os.chmod(path, mode | stat.S_IWRITE)
                            os.unlink(path)
                            if not os.path.exists(path):
                                return  # recovered; proceed

                    raise exc_info[1]

                shutil.rmtree(venv_dir, onerror=rmtree_on_error)
            else:
                logging.error(
                    'Refusing to (re)install into VENV_DIR '
                    'which does not appear to be a venv')
                return 1

        builder = SbEnvBuilder(
            with_pip=True,
            symlinks=(os.name == 'posix'),
            prompt='switchboard_venv')

        logging.info('Creating virtual environment')
        builder.create(venv_dir)

        if builder.post_setup_succeeded:
            return 0
        else:
            return 1

    def run_verify(self, options: argparse.Namespace) -> Dict[str, Any]:
        result = {}
        result['imports'] = {
            'aioquic': False,
            'PySide6': False,
            'pythonosc.osc_server': False,
            'requests': False,
            'six': False,
        }

        result['success'] = True
        for package in result['imports'].keys():
            stmt = f'import {package}'
            try:
                importlib.import_module(package)
                logging.info(f'Third-party import succeeded: `{stmt}`')
                result['imports'][package] = True
            except Exception as exc:
                logging.error(f'Third-party import failed: `{stmt}`',
                              exc_info=exc)
                result['success'] = False

        return result

    def run(self, args: Optional[Sequence[str]] = None) -> int:
        options = self.parser.parse_args(args)

        root_logger = logging.getLogger()
        root_logger.setLevel(options.log_level)

        shortfmt = logging.Formatter('(%(levelname)s) %(message)s')
        longfmt = logging.Formatter('%(asctime)s %(levelname)-8s %(message)s')
        longfmt.default_msec_format = '%s.%03d'  # period instead of comma

        stderr_handler = logging.StreamHandler()
        stderr_handler.formatter = shortfmt

        log_buffer = io.StringIO()
        buffer_handler = logging.StreamHandler(log_buffer)
        buffer_handler.formatter = longfmt

        logging.basicConfig(handlers=[stderr_handler, buffer_handler])

        if options.action == 'install':
            install_result = self.run_install(options)
            with open(options.venv_dir / 'provision.log', 'wt+',
                      encoding='utf-8') as logfile:
                logfile.write(log_buffer.getvalue())
            return install_result
        elif options.action == 'verify':
            if options.output_json:
                # Suppress logging to stderr
                root_logger.removeHandler(stderr_handler)

            # Capture self-test result dict
            verify_result = self.run_verify(options)

            if options.output_json:
                # Append in-memory logging buffer contents
                verify_result['log'] = log_buffer.getvalue()

                # Output results to stdout in JSON format
                print(json.dumps(verify_result, indent=4))

            return 0
        else:
            # Shouldn't happen: argparse early outs on unregistered actions.
            assert False


def main() -> int:
    app = SbSetup()
    result = app.run()
    logging.info(f'Finished with return code {result}')
    return result


if __name__ == "__main__":
    sys.exit(main())
