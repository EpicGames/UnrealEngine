# Copyright Epic Games, Inc. All Rights Reserved.

from __future__ import annotations

import argparse
import copy
from dataclasses import dataclass
import datetime
from enum import StrEnum
import io
import json
import logging
import marshal
import os
import pathlib
from queue import Empty, Queue
import subprocess
import sys
import threading
import tempfile
import traceback
from typing import Any, Callable, Optional, Sequence, Union
import uuid

from switchboard import p4_utils, ugs_utils as UGS


MAX_PARALLEL_WORKERS = 8  # Consistent with UGS, ushell, etc.

SCRIPT_NAME = os.path.basename(__file__)
__version__ = '1.3.0'


class Subcommands(StrEnum):
    Sync = 'sync'
    SyncStatus = 'syncstatus'


class SyncFile:
    ''' Represents a single file pending sync. '''
    def __init__(
        self, depot_path: str, client_path: Optional[str], rev: int, size: int,
        action: str
    ):
        self.depot_path = depot_path
        self.client_path = client_path
        self.rev = rev
        self.size = size
        self.action = action

        # Used during noclobber resolution; controls whether to force sync in a
        # second pass.
        self.clobber = False

    @property
    def spec(self) -> str:
        '''
        Returns the full path + rev spec (compatible with `p4 sync -L`).
        '''
        return f'{self.depot_path}#{self.rev}'

    def __str__(self) -> str:
        return f'{self.spec} ({friendly_bytes(self.size)}) ({self.action})'


class SyncWorkload:
    '''
    Represents a sync operation, or e.g. a parallel worker's subset thereof.
    '''
    def __init__(self):
        self.sync_files: dict[str, SyncFile] = {}
        self.workload_size = 0
        self.client_path_to_depot_path: dict[str, str] = {}

    def add_or_replace(self, new_file: SyncFile):
        if new_file.depot_path in self.sync_files:
            self.remove(new_file.depot_path)
        self.sync_files[new_file.depot_path] = new_file
        self.workload_size += new_file.size
        if new_file.client_path:
            self.client_path_to_depot_path[
                new_file.client_path] = new_file.depot_path

    def add(self, new_file: SyncFile):
        assert new_file.depot_path not in self.sync_files
        self.add_or_replace(new_file)

    def remove(self, depot_path: str):
        remove_file = self.sync_files[depot_path]
        self.workload_size -= remove_file.size
        del self.sync_files[depot_path]
        if remove_file.client_path is not None:
            if remove_file.client_path in self.client_path_to_depot_path:
                del self.client_path_to_depot_path[remove_file.client_path]

    def get_by_client_path(self, client_path: str) -> Optional[SyncFile]:
        if client_path in self.client_path_to_depot_path:
            depot_path = self.client_path_to_depot_path[client_path]
            return self.sync_files.get(depot_path)


def p4_sync_worker(
    sync_workload: SyncWorkload,
    *,
    completion_queue: Optional[Queue[SyncFile]] = None,
    clobber_queue: Optional[Queue[SyncFile]] = None,
    dry_run: bool = False,
    force: bool = False,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
):
    '''
    Performs a sync (subset), and reports file sync completions via
    `completion_queue`.
    '''
    gopts = []
    opts = ['--parallel=0']

    # p4 2020.1 relnotes: "'p4 sync -L' now allows files to be specified with
    # revision specifiers of #0."
    # To support older servers, we don't use -L in the presence of deletes.
    workload_has_deletes = any(
        x.rev == 0 for x in sync_workload.sync_files.values())
    if not workload_has_deletes:
        opts.append('-L')

    if dry_run:
        opts.append('-n')

    if force:
        opts.append('-f')

    argfile = tempfile.NamedTemporaryFile(
        prefix=f'{SCRIPT_NAME}-', suffix='.txt', mode='wt', delete=False)
    gopts.extend(['-x', argfile.name])

    for item in sync_workload.sync_files.values():
        argfile.write(f'{item.spec}\n')
    argfile.close()  # Otherwise the p4 process can't access the file.

    p4kwargs: dict[str, Any] = {'port': port, 'user': user, 'client': client}

    proc = p4_utils.p4('sync', opts, gopts, **p4kwargs)

    remaining_work = copy.deepcopy(sync_workload)

    def complete_item(depot_path: str):
        completed_file = remaining_work.sync_files[depot_path]
        remaining_work.remove(depot_path)
        if completion_queue is not None:
            completion_queue.put(completed_file)

    CLOBBER_PREFIX = b"Can't clobber writable file "

    while True:
        try:
            record = marshal.load(proc.stdout)
        except EOFError:
            break

        if record[b'code'] == b'stat':
            depot_path = record[b'depotFile'].decode()
            complete_item(depot_path)
            continue
        elif record[b'code'] == b'error':
            data: bytes = record[b'data']

            # Detect file sync cancellation caused by client being noclobber
            # and the file being writable. Note that this only happens AFTER a
            # typical completion 'stat' record has also been emitted.
            if data.startswith(CLOBBER_PREFIX):
                # :-1 to also trim \n
                clobber_client_path = data[len(CLOBBER_PREFIX):-1].decode()
                clobber_file = sync_workload.get_by_client_path(
                    clobber_client_path)
                if clobber_file:
                    if clobber_queue:
                        clobber_queue.put(copy.deepcopy(clobber_file))
                    else:
                        # Caller isn't gathering the list of clobber files;
                        # just emit the error message.
                        logging.error(record[b'data'].decode())
                else:
                    # Shouldn't happen.
                    logging.error('Unable to map clobber file to depot path: '
                                  f'{clobber_client_path}')
                continue

        # NB: Not currently handling '... up-to-date' here; should have been
        # excluded in preview.
        logging.warning(f'p4_sync_worker(): Unhandled record: {record}')

    proc.stdout.close()
    os.unlink(argfile.name)  # We can clean up the temporary `-x` file now.

    # This shouldn't happen, but if something goes wrong it prevents a deadlock
    # UPDATE: FIXME: Happens if the connection to the Perforce server is lost.
    if len(remaining_work.sync_files):
        for depot_path, item in remaining_work.sync_files.copy().items():
            logging.warning('p4_sync_worker(): Flushing item not explicitly '
                            f'completed: {item}')
            complete_item(depot_path)


def p4_get_preview_sync_files(
    input_specs: list[str],
    *,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> SyncWorkload:
    ''' Runs a preview sync to generate a list of files and revisions. '''
    result_workload = SyncWorkload()

    p4kwargs: dict[str, Any] = {'port': port, 'user': user, 'client': client}
    records = p4_utils.p4_get_records(
        'sync', ['-n', *input_specs], include_info=True, include_error=True,
        **p4kwargs)

    for record in records:
        if record[b'code'] == b'stat':
            depot_path = record[b'depotFile'].decode()
            action = record[b'action'].decode()
            client_path = None
            if b'clientFile' in record:
                client_path = record[b'clientFile'].decode()

            if action == 'deleted':
                # Delete records report the 'rev' PRIOR to the delete, and have
                # no 'fileFize'.
                rev = 0
                size = 0
            else:
                rev = int(record[b'rev'])
                size = int(record[b'fileSize'])

            result_workload.add_or_replace(SyncFile(depot_path, client_path,
                                                    rev, size, action))
            continue
        elif record[b'code'] == b'info':
            msg = f'{record[b"data"].decode()}'
            if record[b'level'] > 0:
                logging.warning(msg)
            else:
                logging.info(msg)
            continue
        elif record[b'code'] == b'error':
            if record[b'data'].endswith(b'up-to-date.\n'):
                logging.info(record[b'data'].decode().rstrip())
                continue
            elif record[b'data'].endswith(b'no such file(s).\n'):
                continue  # Harmless, and common with revision ranges.

        logging.warning('p4_get_preview_sync_files(): Unhandled record: '
                        f'{record}')

    return result_workload


def friendly_bytes(size: float) -> str:
    suffixes = ['bytes', 'KiB', 'MiB', 'GiB', 'TiB']
    mag = 0
    while size >= 1000:
        size /= 1024
        mag += 1
    return f'{size:.{2 if mag>0 else 0}f} {suffixes[mag]}'


def p4_sync(
    total_work: SyncWorkload,
    num_workers: int = MAX_PARALLEL_WORKERS,
    *,
    clobber_filter_fn: Optional[Callable[[SyncWorkload], None]] = None,
    progress_fn: Optional[Callable[[int, int, int, int], None]] = None,
    dry_run: bool = False,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> Optional[int]:
    # First, gather complete list of files/revisions that need to be synced.
    logging.info('Determining files to be synced')

    p4kwargs: dict[str, Any] = {'port': port, 'user': user, 'client': client}

    if len(total_work.sync_files) < 1:
        logging.info('Nothing to do')
        return None

    # Separate deletes and run them on their own worker process. We no longer
    # refer to `total_work` after this.
    # TODO?: This is only necessary if the server is < 2020.1, which added
    #        support for revision specifiers of #0 using `p4 sync -L`.
    nondelete_work = SyncWorkload()
    delete_work = SyncWorkload()
    for item in total_work.sync_files.copy().values():
        if item.rev == 0:
            delete_work.add(item)
        else:
            nondelete_work.add(item)

    # First pass: Attempt to run all deletes (in the main thread).
    total_delete_files = len(delete_work.sync_files)
    if total_delete_files > 0:
        logging.info(f'Removing {total_delete_files} files from workspace...')

        clobber_queue: Optional[Queue[SyncFile]] = None
        if clobber_filter_fn:
            clobber_queue = Queue()

        p4_sync_worker(delete_work, clobber_queue=clobber_queue, dry_run=dry_run, **p4kwargs)

        # First pass clobber retry: Force sync any deletes where we want to
        # override noclobber.
        if clobber_queue and (clobber_queue.qsize() > 0):
            clobber_work = SyncWorkload()
            while clobber_queue.qsize() > 0:
                clobber_work.add(clobber_queue.get())

            clobber_filter_fn(clobber_work)

            for depot_path, file in clobber_work.sync_files.copy().items():
                if not file.clobber:
                    logging.error('NOT clobbering writable file: '
                                  f'{file.client_path or file.depot_path}')
                    clobber_work.remove(depot_path)
                else:
                    logging.warning('Forcing clobber of writable file: '
                                    f'{file.client_path}')

            num_files_to_clobber = len(clobber_work.sync_files)
            if num_files_to_clobber:
                logging.info(f'Clobbering {num_files_to_clobber} files')
                p4_sync_worker(clobber_work, force=True, dry_run=dry_run, **p4kwargs)

        logging.info('Done removing files from workspace')

    # Second pass: Dispatch all non-delete sync actions to worker threads.
    # FIXME: This block is almost the same as above, but with multiple workers.
    clobber_queue: Optional[Queue[SyncFile]] = None
    if clobber_filter_fn:
        clobber_queue = Queue()

    dispatch_sync_workers(
        nondelete_work,
        num_workers,
        dry_run=dry_run,
        clobber_queue=clobber_queue,
        progress_fn=progress_fn,
        **p4kwargs)

    # Second pass clobber retry: Force sync where we want to override
    # noclobber.
    if clobber_queue and (clobber_queue.qsize() > 0):
        clobber_work = SyncWorkload()
        while clobber_queue.qsize() > 0:
            clobber_work.add(clobber_queue.get())

        clobber_filter_fn(clobber_work)

        for depot_path, file in clobber_work.sync_files.copy().items():
            if not file.clobber:
                logging.error('NOT clobbering writable file: '
                              f'{file.client_path or file.depot_path}')
                clobber_work.remove(depot_path)
            else:
                logging.warning('Forcing clobber of writable file: '
                                f'{file.client_path}')

        num_files_to_clobber = len(clobber_work.sync_files)
        if num_files_to_clobber:
            logging.info(f'Clobbering {num_files_to_clobber} files')
            dispatch_sync_workers(
                clobber_work,
                num_workers,
                dry_run=dry_run,
                force=True,
                progress_fn=progress_fn,
                **p4kwargs)

    logging.info('Sync complete')
    return 0


def dispatch_sync_workers(
    work: SyncWorkload,
    num_workers: int,
    *,
    dry_run: bool = False,
    force: bool = False,
    clobber_queue: Optional[Queue[SyncFile]] = None,
    progress_fn: Optional[Callable[[int, int, int, int], None]] = None,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> int:
    ''' Divides `work` evenly among up to `num_workers` sync subprocesses. '''
    # Display summary of total sync workload.
    total_sync_bytes = work.workload_size
    total_sync_files = len(work.sync_files)
    logging.info(f'Syncing {friendly_bytes(total_sync_bytes)} '
                 f'in {total_sync_files:,} files')

    # Upper bound is least of 1) CPU cores minus one, 2) number of files, or
    # 3) hardcoded max constant.
    num_workers = min(filter(None, [num_workers, (os.cpu_count() or 1) - 1,
                                    total_sync_files, MAX_PARALLEL_WORKERS]))

    # Distribute balanced loads to workers.
    worker_loads = [SyncWorkload() for _ in range(num_workers)]
    work.sync_files = dict(sorted(work.sync_files.items(),
                                  key=lambda x: x[1].size, reverse=True))
    for sync_item in work.sync_files.values():
        lightest_load = min(worker_loads,
                            key=lambda x: (x.workload_size, len(x.sync_files)))
        lightest_load.add_or_replace(sync_item)

    logging.info(f'Using {num_workers} parallel workers')

    # Used by workers to notify main thread when files finish syncing.
    completion_queue: Queue[SyncFile] = Queue()

    # Summarize work distribution and start workers.
    workers: list[threading.Thread] = []
    for i in range(num_workers):
        worker_name = f'Worker {i}'
        load = worker_loads[i]
        if num_workers > 1:
            logging.info(
                f'    {worker_name} - {friendly_bytes(load.workload_size)} '
                f'in {len(load.sync_files):,} files')

        worker = threading.Thread(
            name=worker_name,
            target=p4_sync_worker,
            args=(load,),
            kwargs={'completion_queue': completion_queue,
                    'clobber_queue': clobber_queue, 'force': force,
                    'dry_run': dry_run, 'port': port, 'user': user, 'client': client}
        )
        workers.append(worker)
        worker.start()

    # Wait for workers to signal completed file syncs, and display progress.

    # Items are removed from this copy as they are completed.
    remaining_load = copy.deepcopy(work)
    remaining_num_files = len(remaining_load.sync_files)
    while remaining_num_files:
        try:
            completed_file = completion_queue.get(timeout=1.0)
        except Empty:
            if any(worker.is_alive() for worker in workers):
                continue  # Workers still running; wait patiently.
            else:
                logging.error(f'{remaining_num_files} unsynced files '
                              'remaining, but all workers have exited')
                return 1  # Shouldn't happen.

        remaining_load.remove(completed_file.depot_path)
        remaining_num_files = len(remaining_load.sync_files)

        if progress_fn is not None:
            completed_bytes = total_sync_bytes - remaining_load.workload_size
            completed_files = total_sync_files - remaining_num_files
            progress_fn(completed_bytes, total_sync_bytes, completed_files,
                        total_sync_files)

    for worker in workers:
        worker.join()

    return 0


def path_is_relative_to(
    root: Union[str, os.PathLike[str]],
    other: Union[str, os.PathLike[str]]
) -> bool:
    try:
        pathlib.PurePath(other).relative_to(root)
        return True
    except ValueError:
        return False


class SbListenerHelper:
    def __init__(self):
        self.parser = self.build_parser()
        self.log_string = io.StringIO()

        self.uproj_path: Optional[pathlib.Path] = None
        self.project_dir: Optional[pathlib.Path] = None
        self.engine_dir: Optional[pathlib.Path] = None
        self.additional_paths_to_sync: list[str] = []
        self.sync_engine_cl: Optional[int] = None
        self.sync_project_cl: Optional[int] = None
        self.clobber_engine: bool = False
        self.clobber_project: bool = False
        self.generate_after_sync: bool = False
        self.ugs_versioning: bool = False
        self.no_fast_sync: bool = False

        # UGS CLI options
        self.use_ugs: bool = False
        self.sync_pcbs: bool = False
        self.ugs_lib_dir: Optional[pathlib.Path] = None

        self.dry_run: bool = False
        self.p4port: Optional[str] = None
        self.p4user: Optional[str] = None
        self.p4client: Optional[str] = None
        self.p4client_project: Optional[str] = None
        self.p4kwargs: dict[str, Any] = {}
        self.p4clientspec: dict[str, str] = {}

        self.engine_state: Optional[SbListenerHelper.EngineSyncState] = None
        self.project_state: Optional[SbListenerHelper.SyncState] = None

        if UGS.SyncFilters.supported():
            self.ugs_filters = UGS.SyncFilters()
        else:
            self.ugs_filters = None

    @classmethod
    def build_parser(cls):
        parser = argparse.ArgumentParser()

        # Global options
        parser.add_argument('--log-level', default='INFO',
                            help='Sets the minimum severity of log messages before appearing on stderr.',
                            choices=list(filter(lambda n: n != 'NOTSET',
                                         [n for n, _ in logging.getLevelNamesMapping().items()] + ['NONE'])))

        subparsers = parser.add_subparsers(dest='action')
        subparsers.required = True

        # Parent parsers used for common options
        p4_parser = argparse.ArgumentParser(add_help=False)
        p4_parser.add_argument('-p', '--p4port', type=str,
                               help='Override Perforce P4UPORT')
        p4_parser.add_argument('-u', '--p4user', type=str,
                               help='Override Perforce P4USER')
        p4_parser.add_argument('-c', '--p4client', type=str,
                               help='Override Perforce P4CLIENT')
        p4_parser.add_argument('--p4client-project', type=str,
                               help='Override Perforce P4CLIENT just for project operations')
        p4_parser.add_argument('--project', type=pathlib.Path, metavar='UPROJECT',
                               help='Path to .uproject file')
        p4_parser.add_argument('--engine-dir', type=pathlib.Path,
                               help='Path to Engine directory. If omitted, the ancestors of the '
                                    'project directory will be searched')
        p4_parser.add_argument('--engine-cl', type=int,
                               help='Changelist to sync engine to (requires UPROJECT)')
        p4_parser.add_argument('--project-cl', type=int,
                               help='Changelist to sync project to (requires UPROJECT)')

        # UGS sync filter options
        p4_parser.add_argument('--include-categories', type=str,
                                 help='List of UGS SyncCategory UUIDs to include in sync')
        p4_parser.add_argument('--custom-view', type=str,
                                 help='Comma-separated Perforce wildcards to exclude from sync')
        p4_parser.add_argument('--remove-excluded-workspace-files', action='store_true',
                                 help='Scan workspace to remove all files excluded by sync filter')

        # Add subcommands
        cls.build_sync_parser(subparsers.add_parser(Subcommands.Sync, parents=[p4_parser]))
        cls.build_syncstatus_parser(subparsers.add_parser(Subcommands.SyncStatus, parents=[p4_parser]))

        return parser

    @classmethod
    def build_sync_parser(cls, sync_parser: argparse.ArgumentParser):
        ''' Configures subparser arguments specific to the `sync` subcommand. '''

        sync_parser.add_argument('--generate', action='store_true',
                                 help='Run GenerateProjectFiles after syncing')
        sync_parser.add_argument('--clobber-engine', action='store_true',
                                 help='Override noclobber for engine files')
        sync_parser.add_argument('--clobber-project', action='store_true',
                                 help='Override noclobber for project files')
        sync_parser.add_argument('--ugs-versioning', action='store_true',
                                 help='Maintain depot CompatibleChangelist (e.g. hotfix stream)')
        sync_parser.add_argument('--no-fast-sync', action='store_true',
                                 help='Disable "fast sync" (revision range) optimization')

        sync_parser.add_argument('-n', '--dry-run', action='store_true',
                                 help='Preview sync; do not update files')

        # UGS CLI integration options
        sync_parser.add_argument('--use-ugs', action='store_true',
                                 help='Specifies that UnrealGameSync should be used to sync the '
                                      'project and engine together.')
        sync_parser.add_argument('--use-pcbs', action='store_true',
                                 help='Specifies that precompiled binaries should be synced along '
                                      'with the project (requires syncing via UnrealGameSync).')
        sync_parser.add_argument('--ugs-lib-dir', type=pathlib.Path,
                                 help='Directory path specifying where to find the UnrealGameSync '
                                      'library (ugs.dll).')

        # Debugging options
        sync_parser.add_argument('--dump-sync', type=argparse.FileType('wt'), metavar='SYNC_WORKLOAD_TEXT_FILE',
                                 help='Write the complete list of depot paths and revisions that '
                                 'would be synced, but exit without syncing')

    @classmethod
    def build_syncstatus_parser(cls, syncstatus_parser: argparse.ArgumentParser):
        ''' Configures subparser arguments specific to the `syncstatus` subcommand. '''
        pass

    def run(self, args: Optional[Sequence[str]] = None) -> int:
        options = self.parser.parse_args(args)

        file_handler = logging.FileHandler('sbl_helper.log', mode='w', encoding='utf-8')
        handlers: list[logging.Handler] = [file_handler]
        string_handler: logging.StreamHandler | None = None
        stderr_handler: logging.StreamHandler | None = None

        # file_handler is always full verbosity; log_level only affects stderr/log_string
        if options.log_level.lower() != 'none':
            stderr_handler = logging.StreamHandler()
            stderr_handler.setLevel(options.log_level)
            stderr_handler.setFormatter(logging.Formatter('%(relativeCreated)06dms %(levelname)-8s %(message)s'))
            handlers.append(stderr_handler)

            string_handler = logging.StreamHandler(self.log_string)
            string_handler.setLevel(options.log_level)
            handlers.append(string_handler)

        logging.basicConfig(
            level=logging.NOTSET,
            format='%(asctime)s [%(levelname)-8s] <%(name)s>: %(message)s',
            handlers=handlers
        )

        logging.debug(f'Options: {options}')

        if options.project:
            try:
                self.uproj_path = options.project.resolve(strict=True)
            except FileNotFoundError:
                self.parser.error('argument --project: FileNotFoundError: '
                                  f'{options.project}')

            try:
                _ = json.load(self.uproj_path.open())
            except json.JSONDecodeError:
                self.parser.error('argument --project: unable to parse JSON'
                                  f'\n\n{traceback.format_exc()}')

        self.engine_dir = self.find_engine_dir(options)

        if not self.engine_dir and not self.uproj_path:
            self.parser.error('Neither engine nor project path were provided')

        self.project_dir = self.uproj_path.parent if self.uproj_path else None

        # Attempt to read the notionally synced changelists, as written during the last sync.
        self.engine_state = self.read_engine_sync_state()
        self.project_state = self.read_project_sync_state()

        if options.action == Subcommands.Sync:
            return self.run_sync(options)
        elif options.action == Subcommands.SyncStatus:
            return self.run_syncstatus(options)

        assert False

    def _get_sync_workload(
        self,
        options: argparse.Namespace,
        *,
        engine_dir: pathlib.Path | None = None,
        prev_engine_cl: Optional[int] = None,
        sync_engine_cl: Optional[int] = None,
        project_dir: pathlib.Path | None = None,
        prev_project_cl: Optional[int] = None,
        sync_project_cl: Optional[int] = None,
        port: Optional[str] = None,
        user: Optional[str] = None,
        client: Optional[str] = None,
    ) -> tuple[SyncWorkload, BuildVerInfo]:
        assert engine_dir or project_dir

        p4kwargs: dict[str, Any] = {'port': port, 'user': user, 'client': client}

        engine_paths: list[str] = []
        project_paths: list[str] = []
        pathspecs: list[str] = []

        ugs_config = UGS.parse_depot_ugs_configs(engine_dir=engine_dir,
                                                 project_dir=project_dir,
                                                 **p4kwargs)

        self.additional_paths_to_sync = ugs_config.try_get('Perforce', 'AdditionalPathsToSync') or []

        # Parse sync filter categories from INI and match to command line args
        if options.include_categories or options.custom_view:
            if not self.ugs_filters:
                self.parser.error('Sync filters were specified, but filtering is unavailable (missing P4Python?)')

            if options.include_categories:
                include_ids = set(
                    uuid.UUID(id)
                    for id in options.include_categories.split(','))
                self.ugs_filters.read_categories_from_ini_parser(ugs_config)
                for cat in self.ugs_filters.categories.values():
                    if cat.catid not in include_ids:
                        logging.debug(f'Excluding sync filter "{cat.name}"')
                        self.ugs_filters.exclude_category(cat.catid)
                    else:
                        include_ids.remove(cat.catid)

                # Any remaining IDs here were unmatched to categories
                if len(include_ids):
                    unknown = ', '.join(str(id) for id in include_ids)
                    self.parser.error(
                        f'Unknown --include-categories: {unknown}')

            if options.custom_view:
                custom_views: list[str] = options.custom_view.split(',')
                for view in custom_views:
                    self.ugs_filters.map.insert(view)

            logging.debug(f'Sync filter: {self.ugs_filters.map.as_array()}')

        if sync_engine_cl:
            assert engine_dir
            engine_paths.append(f'{engine_dir.parent / "*"}')
            engine_paths.append(f'{engine_dir / "..."}')

            revspec = f'@{sync_engine_cl}'
            if prev_engine_cl and prev_engine_cl <= sync_engine_cl:
                revspec = f'@{prev_engine_cl},{sync_engine_cl}'

            pathspecs.extend(
                [f'{path}{revspec}' for path in engine_paths])

        if sync_project_cl:
            assert project_dir
            project_paths.append(f'{project_dir / "..."}')

            for path in self.additional_paths_to_sync:
                project_paths.append(f'//{self.p4client}{path}')

            revspec = f'@{sync_project_cl}'
            if prev_project_cl and prev_project_cl <= sync_project_cl:
                revspec = f'@{prev_project_cl},{sync_project_cl}'

            pathspecs.extend(
                [f'{path}{revspec}' for path in project_paths])

        if len(pathspecs) < 1:
            self.parser.error('Nothing specified to sync')
        else:
            logging.info('Scheduled for sync:')
            for pathspec in pathspecs:
                logging.info(f' - {pathspec}')

        logging.info('Determining CompatibleChangelist')

        # [:1] slice because we exclude AdditionalPathsToSync here
        buildver_info = self.get_buildver_info(engine_paths, project_paths[:1])

        if sync_engine_cl:
            # Remove Build.version if present, so we can write our own.
            if buildver_info.local_path and buildver_info.have_rev:
                pathspecs.append(f"{buildver_info.local_path}#0")

        remove_excluded_workload = SyncWorkload()
        if options.remove_excluded_workspace_files and self.ugs_filters:
            logging.info('Finding files in workspace that need to be removed.')
            haves = p4_utils.p4_have([*engine_paths, *project_paths], **p4kwargs)
            if stream := self.p4clientspec.get('Stream'):
                stream_path = pathlib.PurePosixPath(stream)
                num_exclusions = 0
                for have in haves:
                    client_str = have['path']
                    depot_str = have['depotFile']
                    depot_path = pathlib.PurePosixPath(depot_str)
                    rel_path = f'/{depot_path.relative_to(stream_path)}'
                    if not self.ugs_filters.includes_path(rel_path):
                        num_exclusions += 1
                        remove_excluded_workload.add(
                            SyncFile(depot_str, client_str, 0, 0, 'deleted'))
                logging.info(f'Found {num_exclusions:,} excluded files')
            else:
                logging.error('Unable to determine stream from client spec')

        def sync_filter(workload: SyncWorkload):
            # Don't add Build.version to the workspace, but do allow removal.
            buildver_path = buildver_info.depot_path
            if buildver_path and (buildver_path in workload.sync_files):
                if workload.sync_files[buildver_path].rev != 0:
                    workload.remove(buildver_path)

            if self.ugs_filters and self.ugs_filters.excluded_categories:
                num_files = len(workload.sync_files)
                logging.info(f'Applying sync filters... ({num_files:,} files)')
                if stream := self.p4clientspec.get('Stream'):
                    stream_path = pathlib.PurePosixPath(stream)
                    excluded_files: set[str] = set()
                    for depot_str, syncfile in workload.sync_files.items():
                        depot_path = pathlib.PurePosixPath(depot_str)
                        depot_rel_path = f'/{depot_path.relative_to(stream_path)}'
                        if not self.ugs_filters.includes_path(depot_rel_path):
                            excluded_files.add(depot_str)

                    num_exclusions = len(excluded_files)
                    logging.info(f'Filter excluded {num_exclusions:,} files')
                    for exclusion in excluded_files:
                        workload.sync_files.pop(exclusion)
                else:
                    logging.error('Unable to determine stream from client spec')

            for removal in remove_excluded_workload.sync_files.values():
                workload.add_or_replace(removal)

        sync_workload = p4_get_preview_sync_files(pathspecs, **p4kwargs)
        sync_filter(sync_workload)

        return sync_workload, buildver_info

    def run_sync(self, options: argparse.Namespace) -> int:
        self.check_sync_options(options)

        sync_result = None
        if self.use_ugs:
            sync_result = UGS.sync(
                self.uproj_path,
                sync_cl=self.sync_project_cl,
                sync_pcbs=self.sync_pcbs,
                ugs_bin_dir=self.ugs_lib_dir,
                user=self.p4user,
                client=self.p4client
            )
        else:
            prev_engine_cl=self.engine_state.changelist if self.engine_state else None
            prev_project_cl=self.project_state.changelist if self.project_state else None

            # If the user has synced the workspace with UGS recently, flag it
            if self.engine_dir:
                ugs_state_path = self.engine_dir.parent / '.ugs' / 'state.json'
                if ugs_state_path.exists():
                    with open(ugs_state_path, 'rt', encoding='utf-8') as f:
                        try:
                            ugs_state = json.load(f)
                            ugs_last_sync_time = ugs_state['LastSyncTime']
                            ugs_last_sync_datetime = datetime.datetime.fromisoformat(ugs_last_sync_time)
                            if self.engine_state and self.engine_state.sync_time < ugs_last_sync_datetime:
                                logging.warning('Disabling fast sync because UGS last synced the workspace')
                                self.no_fast_sync = True
                        except Exception:
                            logging.warning('Error comparing .ugs/state.json LastSyncTime')

            if self.no_fast_sync:
                logging.info('Fast sync is disabled')
                prev_engine_cl = None
                prev_project_cl = None
            else:
                if (not self.engine_state
                    or not self.engine_state.last_synced_project
                    or self.engine_state.last_synced_project != self.uproj_path
                ):
                    logging.warning('Project fast sync disabled because last synced project differs')
                    if self.engine_state:
                        logging.warning(f'({self.engine_state.last_synced_project})')
                    prev_project_cl = None

            sync_workload, buildver_info = self._get_sync_workload(options,
                engine_dir=self.engine_dir,
                prev_engine_cl=prev_engine_cl,
                sync_engine_cl=self.sync_engine_cl,
                project_dir=self.project_dir,
                prev_project_cl=prev_project_cl,
                sync_project_cl=self.sync_project_cl,
                **self.p4kwargs)

            # User may have requested to write the pending sync files and revisions to
            # file and exit early.
            write_workload_file: io.TextIOWrapper | None = options.dump_sync
            if write_workload_file is not None:
                for item in sync_workload.sync_files.values():
                    write_workload_file.write(f'{item.spec}\n')
                return 0

            def clobber_filter(clobber: SyncWorkload):
                clobber.sync_files = dict(sorted(clobber.sync_files.items()))

                for depot_path, clobber_file in clobber.sync_files.items():
                    if clobber_file.client_path is not None:
                        if self.clobber_engine and self.engine_dir:
                            if path_is_relative_to(self.engine_dir, clobber_file.client_path):
                                clobber_file.clobber = True

                        if self.clobber_project and self.uproj_path:
                            if path_is_relative_to(self.uproj_path.parent, clobber_file.client_path):
                                clobber_file.clobber = True
                    else:
                        # Shouldn't happen.
                        logging.error(f'clobber_filter(): Missing client path for clobber file: {depot_path}')
                        continue

            if not self.dry_run:
                self.write_sync_state()

            sync_result = p4_sync(
                sync_workload,
                clobber_filter_fn=clobber_filter,
                progress_fn=self.on_sync_progress,
                dry_run=self.dry_run,
                port=self.p4port,
                user=self.p4user,
                client=self.p4client,
            )

            if sync_result is None or self.dry_run:
                return 0 # Nothing needed to be synced

            # Write updated Build.version.
            if buildver_info.local_path and buildver_info.updated_text:
                logging.info('Updating Build.version')
                try:
                    with open(buildver_info.local_path, 'wt', encoding='utf-8') as buildver_file:
                        buildver_file.write(buildver_info.updated_text)
                except Exception as exc:
                    logging.error('Exception writing to Build.version', exc_info=exc)

        # Generate project files.
        if self.generate_after_sync:
            logging.info('Generating project files...')
            gpf_result = self.generate_project_files()
            if gpf_result == 0:
                logging.info('Done generating project files')
            else:
                logging.error(f'GenerateProjectFiles failed with code {gpf_result}')
                return gpf_result

        return sync_result

    STATE_FILE = '.sblsync.json'

    @property
    def engine_sync_state_path(self) -> pathlib.Path | None:
        if self.engine_dir:
            return self.engine_dir.parent / self.STATE_FILE
        else:
            return None

    @property
    def project_sync_state_path(self) -> pathlib.Path | None:
        if self.uproj_path:
            return self.uproj_path.parent / self.STATE_FILE
        else:
            return None

    @dataclass
    class SyncState:
        changelist: int
        sync_time: datetime.datetime

        def save_to_file(self, path: pathlib.Path):
            data = self._get_fields()

            try:
                with open(path, 'wt', encoding='utf-8') as f:
                    json.dump(data, f, indent=4)
            except Exception as exc:
                logging.error(f'Unable to write sync state to {path}', exc_info=exc)
                return

            logging.debug(f'Sync state written to {path}: {data}')

        def _get_fields(self) -> dict[str, Any]:
            return {
                'changelist': self.changelist,
                'sync_time': self.sync_time.isoformat(),
            }

        @classmethod
        def load_from_file(cls, path: pathlib.Path) -> SbListenerHelper.SyncState | None:
            logging.debug(f'Reading SyncState from {path}')
            fields = cls._read_fields_from_file(path)
            logging.debug(f'SyncState: {fields}')

            if not fields:
                return None

            return SbListenerHelper.SyncState(
                changelist=int(fields['changelist']),
                sync_time=datetime.datetime.fromisoformat(fields['sync_time'])
            )

        @classmethod
        def _read_fields_from_file(cls, path: pathlib.Path) -> dict[str, Any] | None:
            if not path.exists():
                logging.debug(f'No sync state located at {path}')
                return None

            try:
                with open(path, 'r', encoding='utf-8') as f:
                    state_json: dict = json.load(f)
                    return state_json
            except Exception as exc:
                logging.error(f'Unable to read sync state from {path}', exc_info=exc)
                return None

    @dataclass
    class EngineSyncState(SyncState):
        last_synced_project: pathlib.Path | None

        @classmethod
        def load_from_file(cls, path: pathlib.Path) -> SbListenerHelper.EngineSyncState | None:
            logging.debug(f'Reading EngineSyncState from {path}')
            fields = cls._read_fields_from_file(path)
            logging.debug(f'EngineSyncState: {fields}')

            if not fields:
                return None

            return SbListenerHelper.EngineSyncState(
                changelist=int(fields['changelist']),
                sync_time=datetime.datetime.fromisoformat(fields['sync_time']),
                last_synced_project=(pathlib.Path(fields['last_synced_project'])
                                     if 'last_synced_project' in fields else None),
            )

        def _get_fields(self) -> dict[str, Any]:
            return {
                **super()._get_fields(),
                'last_synced_project': str(self.last_synced_project),
            }

    def read_engine_sync_state(self) -> EngineSyncState | None:
        if self.engine_sync_state_path:
            return SbListenerHelper.EngineSyncState.load_from_file(self.engine_sync_state_path)
        else:
            logging.warning(f'{self.STATE_FILE} not available for {self.engine_sync_state_path}')

    def read_project_sync_state(self) -> SyncState | None:
        if self.project_sync_state_path:
            return SbListenerHelper.SyncState.load_from_file(self.project_sync_state_path)
        else:
            logging.warning(f'{self.STATE_FILE} not available for {self.project_sync_state_path}')

    def write_sync_state(self):
        sync_time = datetime.datetime.now(datetime.timezone.utc)

        if self.sync_engine_cl and self.engine_sync_state_path:
            engine_state = SbListenerHelper.EngineSyncState(self.sync_engine_cl, sync_time, self.uproj_path)
            engine_state.save_to_file(self.engine_sync_state_path)

        if self.sync_project_cl and self.project_sync_state_path:
            project_state = SbListenerHelper.SyncState(self.sync_project_cl, sync_time)
            project_state.save_to_file(self.project_sync_state_path)

    def run_syncstatus(self, options: argparse.Namespace) -> int:
        self.check_p4_options(options)

        # syncstatus only considers engine_dir if it was passed in, NOT if we auto-detected it.
        specified_engine_dir = options.engine_dir is not None

        def guess_have_cl(path: pathlib.Path, client: str | None = None) -> int | None:
            logging.debug(f'Falling back to `p4 cstat #have` for {path}')

            if not path.is_dir():
                return None

            p4kwargs = {**self.p4kwargs}
            if client:
                p4kwargs['client'] = client

            try:
                orig_cwd = os.getcwd()  # Ensure user's P4CONFIG files are read appropriately
                os.chdir(path)
                have_changes = p4_utils.p4_get_decoded_records('cstat', [f'{path / "...#have"}'],
                                                               **p4kwargs)
            finally:
                os.chdir(orig_cwd)

            have_changes = sorted(have_changes, key=lambda x: int(x.get('change', -1)), reverse=True)

            if have_changes:
                if change := have_changes[0].get('change'):
                    return int(change)

            return None

        result = {}

        if specified_engine_dir:
            assert self.engine_dir
            if eng_detected_ws := p4_utils.p4_get_var('P4CLIENT', self.engine_dir):
                result['engine_detected_ws'] = eng_detected_ws

            if self.engine_state:
                self.sync_engine_cl = self.engine_state.changelist
                result['engine_cl'] = self.engine_state.changelist
                result['engine_sync_time'] = self.engine_state.sync_time.isoformat()
            else:
                guess_eng_cl = guess_have_cl(self.engine_dir)
                self.sync_engine_cl = guess_eng_cl
                result['engine_cl'] = guess_eng_cl
                result['engine_sync_time'] = '(Unknown)'

        if self.project_dir:
            if proj_detected_ws := p4_utils.p4_get_var('P4CLIENT', self.project_dir):
                result['project_detected_ws'] = proj_detected_ws

            if self.project_state:
                self.sync_project_cl = self.project_state.changelist
                result['project_cl'] = self.project_state.changelist
                result['project_sync_time'] = self.project_state.sync_time.isoformat()
            else:
                guess_proj_cl = guess_have_cl(self.project_dir, self.p4client_project)
                self.sync_project_cl = guess_proj_cl
                result['project_cl'] = guess_proj_cl
                result['project_sync_time'] = '(Unknown)'

        # Preview whether attempting to sync to the notional changelists would actually result in any changes.
        # If the workspace already reflects the notional changelists, the "workload" should be empty
        # (i.e. "//...@12345678 - file(s) up-to-date.").
        #
        # If syncing would have taken an action, the workspace is considered dirty.

        try:
            orig_cwd = os.getcwd()  # Ensure user's P4CONFIG files are read appropriately
            if self.p4client_project and self.p4client_project != self.p4client:
                # Potentially two queries if we specify a separate project workspace.
                engine_dirty = False
                project_dirty = False

                if specified_engine_dir:
                    assert self.engine_dir
                    os.chdir(self.engine_dir)
                    eng_workload, _ = self._get_sync_workload(options,
                        engine_dir=self.engine_dir,
                        sync_engine_cl=self.sync_engine_cl,
                        port=self.p4port, user=self.p4user, client=self.p4client)
                    if len(eng_workload.sync_files) > 0:
                        engine_dirty = True
                        logging.debug('Engine would sync:')
                        for idx, sync_file in enumerate(eng_workload.sync_files):
                            logging.debug(f'\t- {sync_file}')
                            if idx >= 100:
                                logging.debug(f'\t...and {len(eng_workload.sync_files) - idx} more')
                                break

                if self.project_dir:
                    os.chdir(self.project_dir)
                    proj_workload, _ = self._get_sync_workload(options,
                        project_dir=self.project_dir,
                        sync_project_cl=self.sync_project_cl,
                        port=self.p4port, user=self.p4user, client=self.p4client_project)
                    if len(proj_workload.sync_files) > 0:
                        project_dirty = True
                        logging.debug('Project would sync:')
                        for idx, sync_file in enumerate(proj_workload.sync_files):
                            logging.debug(f'\t- {sync_file}')
                            if idx >= 100:
                                logging.debug(f'\t...and {len(proj_workload.sync_files) - idx} more')
                                break

                result['dirty'] = engine_dirty or project_dirty
            else:
                sync_workload, _ = self._get_sync_workload(options,
                    engine_dir=self.engine_dir,
                    sync_engine_cl=self.sync_engine_cl,
                    project_dir=self.project_dir,
                    sync_project_cl=self.sync_project_cl,
                    **self.p4kwargs)

                if len(sync_workload.sync_files) > 0:
                    result['dirty'] = True
                    logging.debug('Would sync:')
                    for idx, sync_file in enumerate(sync_workload.sync_files):
                        logging.debug(f'\t- {sync_file}')
                        if idx >= 100:
                            logging.debug(f'\t...and {len(sync_workload.sync_files) - idx} more')
                            break
                else:
                    result['dirty'] = False

        finally:
            os.chdir(orig_cwd)

        result['log'] = self.log_string.getvalue()
        json.dump(result, sys.stdout, indent=4)

        return 0

    @dataclass
    class BuildVerInfo:
        local_path: Optional[str] = None
        have_rev: Optional[int] = None
        depot_path: Optional[str] = None
        depot_text: Optional[str] = None
        updated_text: Optional[str] = None

    def get_buildver_info(
        self,
        engine_paths: list[str],
        project_paths: list[str],
    ) -> BuildVerInfo:
        '''
        Read `Build.version` from depot and return its info, depot contents,
        and an updated version of its contents which reflects the local build.
        '''
        ret_info = SbListenerHelper.BuildVerInfo()

        if not self.engine_dir:
            return ret_info

        epicint_local_path = os.path.join(
            self.engine_dir, 'Restricted', 'NotForLicensees', 'Build',
            'EpicInternal.txt')
        buildver_local_path = os.path.join(self.engine_dir, 'Build',
                                           'Build.version')

        ret_info.local_path = buildver_local_path

        record = p4_utils.p4_have([buildver_local_path], **self.p4kwargs)[0]
        ret_info.have_rev = int(record['haveRev']) if (
            record['code'] == 'stat') else None

        if not self.sync_engine_cl:
            return ret_info

        prints = p4_utils.p4_print(
            [f'{path}@{self.sync_engine_cl}'
             for path in (epicint_local_path, buildver_local_path)],
            **self.p4kwargs)
        epicint_print_result = prints[0]
        buildver_print_result = prints[1]

        ret_info.depot_text = buildver_print_result.text  # May be None
        buildver_meta = buildver_print_result.stat

        if not buildver_print_result.is_valid:
            logging.error('Unable to p4_print() Build.version '
                          f'({buildver_meta})')

            return ret_info

        ret_info.depot_path = buildver_meta[b'depotFile'].decode()

        # Determine latest code change within each group (by CL) of paths.
        # First try to narrow using revcx-friendly check (1000 CLs, just *.cpp)
        lower_bound_cl: Optional[int] = None
        eng_upper_bound_cl = self.sync_engine_cl
        proj_upper_bound_cl = self.sync_project_cl

        def quick_narrow_lower_cl(paths: list[str], upper_cl: int,
                                  in_lower_cl: Optional[int]) -> Optional[int]:
            quick_lower = upper_cl - 1_000
            if in_lower_cl and in_lower_cl > quick_lower:
                quick_lower = in_lower_cl
            if upper_cl <= quick_lower:
                return upper_cl
            if quick_lower > 0:
                quick_range = f'{quick_lower},{upper_cl}'
                check_lower = p4_utils.p4_latest_code_change(
                    paths, in_range=quick_range, exts=['.cpp'], **self.p4kwargs)

                if check_lower:
                    if in_lower_cl:
                        return max(in_lower_cl, check_lower)
                    else:
                        return check_lower

            return in_lower_cl

        lower_bound_cl = quick_narrow_lower_cl(engine_paths,
                                               eng_upper_bound_cl,
                                               lower_bound_cl)

        if (lower_bound_cl or 0) < (proj_upper_bound_cl or 0):
            lower_bound_cl = quick_narrow_lower_cl(project_paths,
                                                   proj_upper_bound_cl,
                                                   lower_bound_cl)

        # Comprehensive check (all exts), but hopefully with narrowed ranges
        eng_range = f'<={eng_upper_bound_cl}'
        proj_range = f'<={proj_upper_bound_cl}'

        if lower_bound_cl:
            eng_range = f'{lower_bound_cl},{eng_upper_bound_cl}'
            proj_range = f'{lower_bound_cl},{proj_upper_bound_cl}'

        if lower_bound_cl and (lower_bound_cl >= eng_upper_bound_cl):
            eng_code_cl = eng_upper_bound_cl
        else:
            eng_code_cl = p4_utils.p4_latest_code_change(
                engine_paths, in_range=eng_range, **self.p4kwargs)

        if proj_upper_bound_cl:
            if lower_bound_cl and (lower_bound_cl >= proj_upper_bound_cl):
                proj_code_cl = proj_upper_bound_cl
            else:
                proj_code_cl = p4_utils.p4_latest_code_change(
                    project_paths, in_range=proj_range, **self.p4kwargs)
        else:
            proj_code_cl = None

        # These are the values we'll actually write to the JSON
        ver_cl = max(filter(None, [self.sync_engine_cl, self.sync_project_cl]))
        ver_compatible_cl = max(filter(None, [eng_code_cl, proj_code_cl]))

        # Generate and return the updated Build.version contents.
        try:
            build_ver: dict[str, Union[int, str]] = json.loads(
                ret_info.depot_text)
        except json.JSONDecodeError:
            logging.error('Unable to parse depot Build.version JSON',
                          exc_info=sys.exc_info())
            return ret_info

        branch_name = ret_info.depot_path.replace(
            '/Engine/Build/Build.version', '')
        updates = {
            'Changelist': ver_cl,
            'IsLicenseeVersion': 0 if epicint_print_result.is_valid else 1,
            'BranchName': branch_name.replace('/', '+'),
        }

        if self.ugs_versioning:
            # Don't overwrite the compatible changelist if we're in a hotfix
            # release stream.
            no_depot_compat = build_ver['CompatibleChangelist'] == 0
            licensee_changed = (build_ver['IsLicenseeVersion']
                                != updates['IsLicenseeVersion'])
        else:
            # Always overwrite compatible changelist.
            no_depot_compat = True

        if no_depot_compat or licensee_changed:
            updates['CompatibleChangelist'] = ver_compatible_cl

        build_ver.update(updates)

        # Read existing local file and skip update if unchanged
        existing_build_ver = None
        try:
            with open(buildver_local_path, 'r', encoding='utf-8') as f:
                existing_build_ver = json.load(f)
        except Exception as exc:
            logging.warning('Exception reading local Build.version',
                            exc_info=exc)

        if build_ver == existing_build_ver:
            logging.info("get_buildver_info(): contents haven't changed")
        else:
            logging.info(f"get_buildver_info(): {updates}")
            ret_info.updated_text = json.dumps(build_ver, indent=2)

        return ret_info

    def on_sync_progress(
        self,
        completed_bytes: int,
        total_bytes: int,
        completed_files: int,
        total_files: int
    ):
        if total_bytes != 0:
            progress_pct = completed_bytes / total_bytes
        else:
            progress_pct = completed_files / total_files
        bytes_str = (f'{friendly_bytes(completed_bytes)} / '
                     f'{friendly_bytes(total_bytes)}')
        files_str = f'{completed_files:,} / {total_files:,} files'
        progress_str = (f'Progress: {progress_pct:.2%} '
                        f'({bytes_str}; {files_str})')

        if sys.stdout.isatty():
            # Continuously erase and rewrite interactive progress status line.
            end = '' if progress_pct != 1.0 else None
            print(f'\r{" "*70}\r{progress_str}\r', end=end, flush=True)
        else:
            # Limit number of progress lines; otherwise one per file
            MAX_PROGRESS_LINES = 5000
            progress_interval = max(1, total_files // MAX_PROGRESS_LINES)
            if (
                (completed_files % progress_interval == 0) or
                (completed_files == total_files)
            ):
                logging.info(progress_str)

    def check_p4_options(self, options: argparse.Namespace):
        self.p4port = options.p4port
        self.p4user = options.p4user
        self.p4client = options.p4client
        self.p4client_project = options.p4client_project

        if self.p4port:
            self.p4kwargs['port'] = self.p4port

        if self.p4user:
            self.p4kwargs['user'] = self.p4user

        if self.p4client:
            self.p4kwargs['client'] = self.p4client
            self.p4clientspec = p4_utils.p4_get_client(self.p4client, **self.p4kwargs) or {}

        if p4login := p4_utils.p4_get_login(**self.p4kwargs):
            logging.info(f'Perforce: Logged in as {p4login}')
        else:
            self.parser.error('No valid Perforce session found. Make sure you are logged into Perforce.')

    def check_sync_options(self, options: argparse.Namespace):
        '''
        Parse `sync` options, caching their values, and ensuring consistency
        between related options.
        '''
        self.check_p4_options(options)

        self.dry_run = options.dry_run
        self.clobber_engine = options.clobber_engine
        self.clobber_project = options.clobber_project
        self.ugs_versioning = options.ugs_versioning
        self.no_fast_sync = options.no_fast_sync

        # UGS CLI options
        self.use_ugs = options.use_ugs
        self.sync_pcbs = options.use_pcbs
        self.ugs_lib_dir = options.ugs_lib_dir

        if options.project is None:
            if options.project_cl is not None:
                self.parser.error('--project-cl requires --project')

        if options.engine_cl is not None and options.engine_cl <= 0:
            self.parser.error(f'Invalid --engine-cl {options.engine_cl}')
        elif options.project_cl is not None and options.project_cl <= 0:
            self.parser.error(f'Invalid --project-cl {options.project_cl}')

        self.sync_engine_cl = options.engine_cl
        self.sync_project_cl = options.project_cl

        # UGS will generate the project files by default (don't duplicate work)
        self.generate_after_sync = options.generate and not self.use_ugs

        if self.sync_pcbs and not self.use_ugs:
            self.parser.error('`--use-pcbs` requires `--use-ugs`.')

        if self.use_ugs and self.sync_engine_cl:
            self.parser.error('`--use-ugs` was specified along with '
                              '`--engine-cl`; these arguments are mutally '
                              'exclusive and should not be used together.')

        engine_dir_required = ((self.sync_engine_cl is not None)
                               or (self.generate_after_sync))

        if engine_dir_required:
            if self.engine_dir is None:
                if self.sync_engine_cl is not None:
                    self.parser.error('--engine-cl was specified, but unable '
                                      'to determine engine directory')
                elif self.generate_after_sync:
                    self.parser.error('--generate was specified, but unable '
                                      'to determine engine directory')

    def find_engine_dir(
        self,
        options: argparse.Namespace
    ) -> Optional[pathlib.Path]:
        if options.engine_dir:
            return options.engine_dir

        # Look up the tree from the project for an Engine/ dir
        if self.uproj_path:
            iter_dir = self.uproj_path.parent.parent
            while not iter_dir.samefile(iter_dir.anchor):
                candidate = iter_dir / 'Engine'
                if candidate.is_dir():
                    logging.debug(f'Engine directory: {candidate}')
                    return candidate
                iter_dir = iter_dir.parent

        # TODO: Consider looking for Engine dir as ancestor of this script?
        logging.debug('Engine directory not found')
        return None

    def generate_project_files(self) -> int:
        '''
        Invokes GenerateProjectFiles script, streams the output to INFO
        logging, and returns the exit code.
        '''
        assert self.engine_dir

        if sys.platform.startswith('win'):
            gpf_script = os.path.join(self.engine_dir, 'Build', 'BatchFiles',
                                      'GenerateProjectFiles.bat')
        else:
            gpf_script = os.path.join(self.engine_dir, 'Build', 'BatchFiles',
                                      'Linux', 'GenerateProjectFiles.sh')

        args = [gpf_script]
        if self.uproj_path:
            args.append(str(self.uproj_path))

        logging.debug(f'generate_project_files(): invoking subprocess: args={args}')

        try:
            with subprocess.Popen(args, stdout=subprocess.PIPE) as proc:
                for line in proc.stdout:
                    logging.info(f'gpf> {line.decode().rstrip()}')

                return proc.wait()
        except Exception as exc:
            logging.error('generate_project_files(): exception during Popen',
                          exc_info=exc)
            return -1


def main() -> int:
    p4_utils.meta_zprog = SCRIPT_NAME
    p4_utils.meta_zversion = __version__

    app = SbListenerHelper()
    result = app.run()
    logging.info(f'Finished with exit code {result}')
    return result


if __name__ == "__main__":
    sys.exit(main())
