@echo off

setlocal ENABLEDELAYEDEXPANSION

REM This script optionally takes a single argument, representing the path to the desired Python
REM virtual environment directory. If omitted, it defaults to the value of %_defaultVenvDir%.
REM
REM Additionally, in the case where no argument was provided, pause on errors, in case the user
REM double-clicked the batch file and we don't want the console window to vanish inexplicably.
REM Otherwise, we exit immediately.

set _Args=%*

set _switchboardDir=%~dp0
REM pushd and %CD% are used to normalize the relative path to a shorter absolute path.
pushd "%~dp0..\..\..\..\.."
set _engineDir=%CD%
set _enginePythonPlatformDir=%_engineDir%\Binaries\ThirdParty\Python3\Win64
set _defaultVenvDir=%_engineDir%\Extras\ThirdPartyNotUE\SwitchboardThirdParty\Python
popd

if "%~1"=="" goto:defaultenv
if "%~1"=="--defaultenv" goto:defaultenv

    set _bUsingDefaultVenv=
    set _venvDir=%~1
    echo Using provided path for Python virtual environment (!_venvDir!^)

    goto:fi_env

:defaultenv

    set _bUsingDefaultVenv=1
    set _venvDir=%_defaultVenvDir%
    echo Using DEFAULT path for Python virtual environment (%_defaultVenvDir%^)

:fi_env

call:main

endlocal
goto:eof

::------------------------------------------------------------------------------
:main

if not exist "%_venvDir%\Scripts\pythonw.exe" (
    echo Performing Switchboard first-time setup
    call "%_enginePythonPlatformDir%\python.exe" "%~dp0\sb_setup.py" install --venv-dir="%_venvDir%"
    if !ERRORLEVEL! NEQ 0 (
        echo Installation failed with non-zero exit code^^^!
        if defined _bUsingDefaultVenv ( pause )
        exit /B !ERRORLEVEL!
    )
)

call:start_sb

goto:eof

::------------------------------------------------------------------------------
:start_sb

set PYTHONPATH=%_switchboardDir%;%PYTHONPATH%

start "Switchboard" /D "%_switchboardDir%" "%_venvDir%\Scripts\pythonw.exe" -m switchboard %_Args%
if %ERRORLEVEL% NEQ 0 (
    echo Failed to launch Switchboard^!
    if defined _bUsingDefaultVenv ( pause )
    exit /B %ERRORLEVEL%
)
