#!/bin/sh

# This script optionally takes a single argument, representing the path to the desired Python
# virtual environment directory. If omitted, it defaults to the value of $_defaultVenvDir.

_Args=$@
_switchboardDir=$(dirname "$0")
_engineDir=$(cd "$_switchboardDir/../../../../.."; pwd)
_enginePythonDir="$_engineDir/Binaries/ThirdParty/Python3/Linux"
_defaultVenvDir="$_engineDir/Extras/ThirdPartyNotUE/SwitchboardThirdParty/Python"

if [ $# -lt 1 ] || [ $1 == "--defaultenv" ]; then
    _venvDir=$_defaultVenvDir
else
    _venvDir="$1"
fi

if [ ! -x "$_venvDir/bin/python3" ]; then
    "$_enginePythonDir/bin/python3" "$_switchboardDir/sb_setup.py" install --venv-dir="$_venvDir"
    _installResult=$?
    if [ $_installResult -ne 0 ]; then
        echo "Installation failed with non-zero exit code!"
        exit $_installResult
    fi
fi

(cd "$_switchboardDir" && PYTHONPATH="$_switchboardDir:$PYTHONPATH" "$_venvDir/bin/python3" -m switchboard $_Args)
