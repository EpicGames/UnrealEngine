# Copyright Epic Games, Inc. All Rights Reserved.

import argparse
import signal
import sys

from PySide6 import QtCore, QtWidgets

from switchboard.config import SETTINGS, CONFIG
from switchboard.switchboard_scripting import ScriptManager
from .switchboard_dialog import SwitchboardDialog
from .switchboard_logging import LOGGER, FILE_HANDLER


# Build resources
# "Engine\Extras\ThirdPartyNotUE\SwitchboardThirdParty\Python\Scripts\pyside6-rcc" -o "Engine\Plugins\VirtualProduction\Switchboard\Source\Switchboard\switchboard\resources.py" "Engine\Plugins\VirtualProduction\Switchboard\Source\Switchboard\switchboard\ui\resources.qrc"


def parse_arguments():
    ''' Parses command line arguments and returns the populated namespace 
    '''
    parser = argparse.ArgumentParser()
    parser.add_argument('--script'    , default='', help='Path to script that contains a SwichboardScriptBase subclass')
    parser.add_argument('--scriptargs', default='', help='String to pass to SwichboardScriptBase subclass as arguments')
    args, unknown = parser.parse_known_args()
    return args


def launch():
    """
    Main for running standalone or in another application.
    """
    if sys.platform == 'win32':
        # works around some windows quirks so we can show the window icon
        import ctypes
        app_id = u'epicgames.virtualproduction.switchboard.0.1'
        ctypes.windll.shell32.SetCurrentProcessExplicitAppUserModelID(app_id)

    app = QtWidgets.QApplication(sys.argv)

    # Global variables can be inited after QT has been started (they may show dialogues if there are errors)
    SETTINGS.init()
    CONFIG.init_with_file_path(SETTINGS.CONFIG)

    # parse arguments
    args = parse_arguments()

    # script manager
    script_manager = ScriptManager()

    # add script passed from command line
    if args.script:
        try:
            script_manager.add_script_from_path(args.script, args.scriptargs)
        except Exception as e:
            LOGGER.warning(f"Could not initialize '{args.script}': {e}")

    # script pre-init
    script_manager.on_preinit()

    # create main window
    main_window = SwitchboardDialog(script_manager)

    if not main_window.window:
        return

    # closure so we can access main_window and app
    def sigint_handler(*args):
        LOGGER.info("Received SIGINT, exiting...")
        main_window.on_exit()
        app.quit()

    # install handler for SIGINT so it's possible to exit the app when pressing ctrl+c in the terminal.
    signal.signal(signal.SIGINT, sigint_handler)

    main_window.window.show()

    # Logging start.
    LOGGER.info('----==== Switchboard ====----')
    LOGGER.info(f'Log file: {FILE_HANDLER.baseFilename}')

    # Log unhandled exceptions.
    def except_hook(exc_type: type[BaseException], exc_value: BaseException, exc_traceback):
        exc_info=(exc_type, exc_value, exc_traceback)
        LOGGER.critical('Unhandled exception', exc_info=exc_info)

    sys.excepthook = except_hook

    # this will pump the event loop every 200ms so we can react faster on a SIGINT.
    # otherwise it will take several seconds before sigint_handler is called.
    timer = QtCore.QTimer()
    timer.start(200)
    timer.timeout.connect(lambda: None)

    # execute the app
    appresult = app.exec()

    # script exit
    script_manager.on_exit()

    sys.exit(appresult)

if __name__ == "__main__":
    launch()