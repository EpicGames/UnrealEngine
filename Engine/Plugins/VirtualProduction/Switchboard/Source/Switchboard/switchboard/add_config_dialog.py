# Copyright Epic Games, Inc. All Rights Reserved.

import os
import pathlib
import shlex
import socket
import subprocess
import sys
import threading

from pathlib import Path
from PySide6 import QtCore
from PySide6 import QtWidgets

from switchboard import config
from switchboard import p4_utils as P4
from switchboard import switchboard_utils as sb_utils
from switchboard.switchboard_logging import LOGGER


class AddConfigDialog(QtWidgets.QDialog):

    class AddConfigDialogState(object):
        ''' Proxy that holds the state of the AddConfigDialog,
        used to avoid accessing AddConfigDialog UI fields directly from worker threads.
        '''

        def __init__(self, addConfigDialog):

            # load the data from the real dialog
            self.load(addConfigDialog)

        def load(self, addConfigDialog):
            ''' Reads the relevant fields from AddConfigDialog
            Args:
                addConfigDialog(AddConfigDialog): The dialog to load the data from
            '''
            self.config_path_str    = addConfigDialog.config_path_line_edit.text()
            self.engineDir          = addConfigDialog.engine_dir_line_edit.text()
            self.project            = addConfigDialog.uproject_line_edit.text()
            self.p4Engine           = addConfigDialog.p4_engine_path_line_edit.text()
            self.p4Project          = addConfigDialog.p4_project_path_line_edit.text()
            self.p4Workspace        = addConfigDialog.p4_workspace_line_edit.text()
            self.p4ProjectWorkspace = addConfigDialog.p4_project_workspace_line_edit.text()
            self.p4Enabled          = addConfigDialog.p4_group.isChecked()

        def dump(self, addConfigDialog):
            ''' Writes the relevant fields from AddConfigDialog
            Args:
                addConfigDialog(AddConfigDialog): The dialog to load the data from
            '''
            lineEdits = [
                (self.config_path_str   , addConfigDialog.config_path_line_edit         ),
                (self.engineDir         , addConfigDialog.engine_dir_line_edit          ),
                (self.project           , addConfigDialog.uproject_line_edit            ),
                (self.p4Engine          , addConfigDialog.p4_engine_path_line_edit      ),
                (self.p4Project         , addConfigDialog.p4_project_path_line_edit     ),
                (self.p4Workspace       , addConfigDialog.p4_workspace_line_edit        ),
                (self.p4ProjectWorkspace, addConfigDialog.p4_project_workspace_line_edit),
            ]

            # Update line edits
            for prop, lineEdit in lineEdits:
                if lineEdit.text() != prop:
                    lineEdit.setText(prop)

            # Update checkboxes
            if addConfigDialog.p4_group.isChecked() != self.p4Enabled:
                addConfigDialog.p4_group.setChecked(self.p4Enabled)


    def __init__(self, uproject_search_path, previous_engine_dir, parent):
        super().__init__(parent=parent, f=QtCore.Qt.WindowCloseButtonHint)

        self.config_path = None
        self.uproject = ''
        self.engine_dir = None

        self.setWindowTitle("Add new Switchboard Configuration")

        self.form_layout = QtWidgets.QFormLayout()

        # Path to this configuration file

        self.config_path_line_edit = QtWidgets.QLineEdit()
        self.config_path_line_edit.setValidator(config.ConfigPathValidator())
        self.config_path_line_edit.textChanged.connect(self.update_button_box)
        self.config_path_browse_button = QtWidgets.QPushButton("Browse")
        self.config_path_browse_button.clicked.connect(
            self.on_browse_config_path)

        config_path_layout = QtWidgets.QHBoxLayout()
        config_path_layout.addWidget(self.config_path_line_edit)
        config_path_layout.addWidget(self.config_path_browse_button)
        self.form_layout.addRow("Config Path", config_path_layout)

        # Path to .uproject

        self.uproject_line_edit = QtWidgets.QLineEdit()
        self.uproject_line_edit.textChanged.connect(self.on_uproject_changed)
        self.uproject_browse_button = QtWidgets.QPushButton("Browse")
        self.uproject_browse_button.clicked.connect(lambda: self.on_browse_uproject_path(uproject_search_path))

        uproject_layout = QtWidgets.QHBoxLayout()
        uproject_layout.addWidget(self.uproject_line_edit)
        uproject_layout.addWidget(self.uproject_browse_button)
        self.form_layout.addRow("uProject", uproject_layout)

        # Path to the base Engine directory (e.g. D:\p4\Engine)

        self.engine_dir_line_edit = QtWidgets.QLineEdit()
        if os.path.exists(previous_engine_dir): # re-use previous engine dir
            normalized_previous_engine_dir = os.path.normpath(str(Path(previous_engine_dir)))
            self.engine_dir_line_edit.setText(normalized_previous_engine_dir)
            self.engine_dir = normalized_previous_engine_dir
        self.engine_dir_line_edit.textChanged.connect(self.on_engine_dir_changed)
        self.engine_dir_browse_button = QtWidgets.QPushButton("Browse")
        self.engine_dir_browse_button.clicked.connect(self.on_browse_engine_dir)

        engine_dir_layout = QtWidgets.QHBoxLayout()
        engine_dir_layout.addWidget(self.engine_dir_line_edit)
        engine_dir_layout.addWidget(self.engine_dir_browse_button)
        self.form_layout.addRow("Engine Dir", engine_dir_layout)

        layout = QtWidgets.QVBoxLayout()
        layout.insertLayout(0, self.form_layout)

        #
        # Perfoce settings
        #

        # Checkable group box
        self.p4_group = QtWidgets.QGroupBox("Perforce")
        self.p4_group.setCheckable(True)
        self.p4_group.setChecked(bool(config.CONFIG.P4_ENABLED.get_value()))
        self.p4_group.toggled.connect(self.on_p4_toggled)

        # Project p4 path
        self.p4_project_path_line_edit = QtWidgets.QLineEdit(
            config.CONFIG.P4_PROJECT_PATH.get_value())

        # Engine p4 path
        self.p4_engine_path_line_edit = QtWidgets.QLineEdit(
            config.CONFIG.P4_ENGINE_PATH.get_value())

        # Name of the workspace
        self.p4_workspace_line_edit = QtWidgets.QLineEdit(
            config.CONFIG.SOURCE_CONTROL_WORKSPACE.get_value())

        self.p4_project_workspace_line_edit = QtWidgets.QLineEdit(
            config.CONFIG.PROJECT_WORKSPACE.get_value())

        p4_layout = QtWidgets.QFormLayout()
        p4_layout.addRow("P4 Project Path", self.p4_project_path_line_edit)
        p4_layout.addRow("P4 Engine Path", self.p4_engine_path_line_edit)
        p4_layout.addRow("Workspace", self.p4_workspace_line_edit)
        p4_layout.addRow("Project Workspace Override", self.p4_project_workspace_line_edit)
        self.p4_group.setLayout(p4_layout)
        layout.addWidget(self.p4_group)

        self.button_box = QtWidgets.QDialogButtonBox(QtWidgets.QDialogButtonBox.Ok | QtWidgets.QDialogButtonBox.Cancel)
        self.button_box.button(QtWidgets.QDialogButtonBox.Ok).setEnabled(False)
        self.button_box.accepted.connect(self.accept)
        self.button_box.rejected.connect(self.reject)

        self.btnDetect = QtWidgets.QPushButton("Detect")
        self.btnDetect.clicked.connect(self.on_btnDetect_clicked)
        self.button_box.addButton(self.btnDetect, QtWidgets.QDialogButtonBox.ActionRole)

        layout.addWidget(self.button_box)

        self.setLayout(layout)
        self.setMinimumWidth(450)

        # connect the p4 populate signal with slot. This is to update UI elements from the proper thread.
        self.signal_populate_best_p4_guess_with_progressbar.connect(self.on_populate_best_p4_guess_with_progressbar_done)

        # auto-populate what you can, but respecting existing values
        self.populate_best_project_guess(overrideIfFound=False)
        self.populate_best_config_path(overrideIfFound=False)
        if self.p4_group.isChecked():
            self.populate_best_p4_guess_with_progressbar(overrideIfFound=False)

    def accept(self):
        '''
        Override to ensure that the config_path property is a valid, usable
        file path when the dialog is accepted.

        The validator for the config path line edit should ensure that the file
        path is valid by this point, but just in case it's not somehow, some
        error messages are offered about how to fix it, in which cases the
        dialog is *not* dismissed.
        '''
        uproject_path_str = self.uproject_line_edit.text().strip()
        config_path_str = self.config_path_line_edit.text().strip()

        if not uproject_path_str:
            warning_msg = QtWidgets.QMessageBox(self)
            msg_result = warning_msg.warning(self, 'Continue without uproject?',
                'Are you sure you want to continue without specifying a .uproject?',
                QtWidgets.QMessageBox.Yes | QtWidgets.QMessageBox.No, QtWidgets.QMessageBox.No)

            if msg_result != QtWidgets.QMessageBox.Yes:
                return

        try:
            config_path = config.CONFIG_MGR.get_absolute_config_path(config_path_str)
        except Exception as e:
            error_msg = QtWidgets.QErrorMessage(self)
            error_msg.showMessage(str(e))
            return

        self.config_path = config_path

        super().accept()

    def on_btnDetect_clicked(self):
        ''' The user has decided to override the current selections with auto-detected settings.
        '''
        self.populate_best_project_guess(overrideIfFound=True)
        self.populate_best_config_path(overrideIfFound=False) # seems wrong to override the path

        if self.p4_group.isChecked():
            self.populate_best_p4_guess_with_progressbar(overrideIfFound=True)


    signal_populate_best_p4_guess_with_progressbar = QtCore.Signal(AddConfigDialogState, QtWidgets.QProgressDialog)

    @QtCore.Slot(AddConfigDialogState, QtWidgets.QProgressDialog)
    def on_populate_best_p4_guess_with_progressbar_done(self, diagState, progressDiag):
        ''' Updates the dialog fields based on the results of _populate_best_p4_guess
        Args:
            diagState(AddConfigDialogState): The new state for the dialog
            progressDiag(QtWidgets.QProgressDialog): Progress bar to close
        '''
        # update the dialog
        diagState.dump(self)

        # close progress bar window
        progressDiag.close()


    def populate_best_p4_guess_with_progressbar(self, overrideIfFound):
        ''' Wrapper to populate_best_p4_guess that includes a progress bar

        Args:
            overrideIfFound(bool): See populate_best_p4_guess.
        '''

        # show a progress bar if it is taking more a trivial amount of time
        progressDiag = QtWidgets.QProgressDialog('Querying Perforce...','Cancel', 0, 0, parent=self)
        progressDiag.setModal(True)
        progressDiag.setMinimumDuration(1000) # time before it shows up
        progressDiag.setCancelButton(None)
        progressDiag.setWindowFlag(QtCore.Qt.FramelessWindowHint) # Looks much better without the window frame
        progressDiag.setRange(0,0) # Makes it show a self-cycling progress bar.
        progressDiag.setValue(0) # Required for setMinimumDuration to work

        diagState = self.AddConfigDialogState(self)

        def populate_p4_threadfn():
            try:
                self._populate_best_p4_guess(diagState, overrideIfFound=overrideIfFound)
            finally:
                self.signal_populate_best_p4_guess_with_progressbar.emit(diagState, progressDiag)

        # launch the p4 process, which may take some time depending on connection speeds to the p4 server.
        threading.Thread(target=populate_p4_threadfn, args=()).start()


    def _populate_best_p4_guess(self, diagState, overrideIfFound=False):
        ''' Auto-fill the p4 fields by modifiying the given diagStatus with the new values.
        This function can be called from a worker thread.

        Args:
            diagState(AddConfigDialogState): Status of the dialog.
            overrideIfFound(bool): If true, will overwrite the exising p4 paths
        '''

        # Try to infer the values if empty (don't replace manually entered data unless overrideIfFound is set)
        bUpdateEngine = diagState.engineDir and ((not diagState.p4Engine)  or overrideIfFound)
        bUpdateProject =  diagState.project and ((not diagState.p4Project) or overrideIfFound)

        if (not bUpdateEngine) and (not bUpdateProject):
            return

        # gather workspaces tied to this host
        #
        # Each workspace returned by p4 -G clients comes with:
        # * Host   : Hostname
        # * Root   : Local path
        # * Stream : p4 path
        # * client : Name of the workspace

        try:
            hostname = socket.gethostname().lower()
            workspaces = P4.run(cmd='clients', args=['--me'])

            # valid workspaces are those with the same host as the PC's, or are empty (that is, shared workspaces)
            workspaces = [ws for ws in workspaces
                if P4.valueForMarshalledKey(ws,'Host').lower() == hostname
                or not P4.valueForMarshalledKey(ws,'Host')
            ]

        except Exception as e:
            LOGGER.warning(f'Could not query p4: {repr(e)}')
            return

        client = diagState.p4Workspace

        # Infer Engine p4 path
        if  bUpdateEngine:
            try:
                enginedir = diagState.engineDir
                client,p4path = P4.p4_from_localpath(localpath=enginedir, workspaces=workspaces, preferredClient=client)

                diagState.p4Workspace = client
                diagState.p4Engine = p4path
            except Exception as e:
                LOGGER.warning(f"Could not auto-fill Engine p4 settings: {repr(e)}")

        # Infer project p4 path
        if bUpdateProject:
            try:
                projectpath = diagState.project
                projectdir = str(pathlib.PurePath(projectpath).parent)
                client,p4path = P4.p4_from_localpath(localpath=projectdir, workspaces=workspaces, preferredClient=client)

                diagState.p4Workspace = client
                diagState.p4Project = p4path
            except Exception as e:
                LOGGER.warning(f"Could not auto-fill project p4 settings: {repr(e)}")


    def on_p4_toggled(self, checked):
        ''' Called when the Perforce group is checked or unchecked
        If checked is true, it will try to auto-fill the p4 fields.
        '''

        # no need to do anything when unchecking it.
        if not checked:
            return

        self.populate_best_p4_guess_with_progressbar(overrideIfFound=False)

    def find_upstream_path_with_name(self, path, name, includeSiblings=False):
        ''' Goes up the folder chain until it finds the desired name, optionally considering sibling folders

        Args:
            path(str or pathlib.Path): The upstream path
            name(str): The directory name we're looking upstream for
            includeSiblings(bool):  Whether or not to look at sibling folders for the name.

        Returns:
            str: The full path to the found upstream folder of the given name.
        '''
        # ensure we have a pathlib.PurePath object
        path = pathlib.PurePath(path)

        # if the folder name coincides, we're done
        if path.name == name:
            return str(path)

        # If including siblings, check those as well.
        if includeSiblings:
            if name in next(os.walk(path.parent))[1]:
                return str(path.parent/name)

        # detect if we already reached the root folder
        if path == path.parent:
            raise FileNotFoundError(f'Could not find {name} in path')

        # go one up, recursively
        return self.find_upstream_path_with_name(path.parent, name, includeSiblings)

    def populate_best_config_path(self, overrideIfFound=False):
        ''' Populates the config path with a best guess based on the project name

        Args:
            overrideIfFound(bool): If true, will overwrite existing field if found.
        '''
        projectpath = self.uproject_line_edit.text()
        config_path_str = self.config_path_line_edit.text().strip()

        if projectpath and (overrideIfFound or not config_path_str):
            self.config_path_line_edit.setText(
                pathlib.PurePath(projectpath).stem)

    def populate_best_project_guess(self, overrideIfFound=False):
        ''' Populates the editor and project with a best guess based on the running processes

        Args:
            overrideIfFound(bool): If true, will overwrite existing fields if found.
        '''

        existingEngineDir = self.engine_dir_line_edit.text()
        existingProject = self.uproject_line_edit.text()

        # only update engine if empty or overriding
        bUpdateEngine = overrideIfFound or not existingEngineDir

        # only update project if empty or overriding
        bUpdateProject = overrideIfFound or not existingProject

        # If nothing to update, return early
        if not (bUpdateEngine or bUpdateProject):
            return

        # perform the detection of candidates
        editors,projects = self.detect_running_projects()

        editorfolder = ''
        projectpath = ''

        sbfolder = ''
        try:
            sbpath = os.path.abspath(__file__)
            sbfolder = self.find_upstream_path_with_name(sbpath, "Engine")
        except FileNotFoundError:
            pass

        # pick the best candidate out of the detected ones
        for idx, editor in enumerate(editors):
            try:
                editorfolder = self.find_upstream_path_with_name(editor, "Engine")
            except FileNotFoundError:
                continue

            if sbfolder == '' or editorfolder == sbfolder:
                projectpath = projects[idx]

            # prefer when we have both editor and project paths
            if editorfolder and projectpath:
                break

        # If process-based detection didn't work, try to find the Engine associated with the running Switchboard
        if not editorfolder and sbfolder:
            editorfolder = sbfolder

        # Populate the fields that we're updating and were found

        if bUpdateEngine and editorfolder:
            self.engine_dir_line_edit.setText(str(Path(editorfolder)))

        if bUpdateProject and projectpath:
            self.uproject_line_edit.setText(str(Path(projectpath)))

    def detect_running_projects(self):
        ''' Detects a running UnrealEngine editor and its project

        Returns:
            list,list: Detected editors,projects local paths.
        '''

        editors = []
        projects = []

        # Not detecting Debug runs since it is not common and would increase detection time
        # At some point this might need UE6Editor added.
        UEnames = ['UE4Editor', 'UnrealEditor']

        if sys.platform.startswith('win'):
            for UEname in UEnames:
                # Windows UE Editors will have .exe extension
                UEname += '.exe'

                cmd = ['powershell', '-Command', f'Get-WmiObject Win32_Process -Filter "Name=\'{UEname}\'" | Select-Object -ExpandProperty CommandLine']

                try:
                    commandlines = subprocess.check_output(
                        cmd, startupinfo=sb_utils.get_hidden_sp_startupinfo()
                        ).decode().splitlines()
                except Exception as exc:
                    LOGGER.error('Exception polling Unreal processes',
                                 exc_info=exc)
                    continue

                for line in commandlines:
                    if UEname.lower() not in line.lower():
                        continue

                    # split the cmdline as a list of the original arguments
                    try:
                        # Replacing \ with / will alter command line arguments in general, but is expected to be ok
                        # because we only look at the first two arguments and only interpret them as potential paths.
                        # This was done to work around shlex mis-parsing back-slashes on paths without double quotes.
                        argv = shlex.split(line.replace('\\','/'))
                    except ValueError:
                        continue

                    # There should be at least 2 arguments, the executable and the project.
                    if len(argv) < 2:
                        continue

                    editorpath = str(Path(argv[0]))
                    projectpath = str(Path(argv[1]))

                    if editorpath.lower().endswith(UEname.lower()) and os.path.exists(editorpath):
                        editors.append(editorpath)
                    else:
                        editors.append('')

                    if projectpath.lower().endswith('.uproject') and os.path.exists(projectpath):
                        projects.append(projectpath)
                    else:
                        # Try to find project by name using *.uprojectdirs files
                        found_project = self._find_project_by_name(editorpath, projectpath)
                        projects.append(found_project if found_project else '')

        return editors,projects

    def _find_project_by_name(self, editorpath, project_name):
        ''' Finds the .uproject path by name using *.uprojectdirs files in the given editor's root folder.
        '''
        try:
            # Look for .uprojectdirs files in tne Engine/'s parent folder.
            current_path = editorpath
            while current_path and current_path != os.path.dirname(current_path):
                current_path = os.path.dirname(current_path)
                if os.path.basename(current_path).lower() == 'engine':
                    engine_parent = os.path.dirname(current_path)
                    break
            else:
                return None
            
            import glob
            uprojectdirs_files = glob.glob(os.path.join(engine_parent, '*.uprojectdirs'))
            
            # Read all search paths from all .uprojectdirs files

            search_paths = []
            for uprojectdirs_file in uprojectdirs_files:
                try:
                    with open(uprojectdirs_file, 'r') as f:
                        for line in f:
                            line = line.strip()
                            if line and not line.startswith(';') and not line.startswith('#'):
                                search_paths.append(line)
                except:
                    continue
            
            # Search for the project in each path

            for search_path in search_paths:
                full_search_path = os.path.join(engine_parent, search_path)
                if os.path.isdir(full_search_path):

                    project_file = os.path.join(full_search_path, f"{project_name}.uproject")
                    if os.path.exists(project_file):
                        return os.path.normpath(project_file)
                    
                    # Also search in subdirectories named after the project
                    subdir_project = os.path.join(full_search_path, project_name, f"{project_name}.uproject")
                    if os.path.exists(subdir_project):
                        return os.path.normpath(subdir_project)
            
            return None
        except:
            return None

    def p4_settings(self):
        settings = {}
        settings['p4_enabled'] = self.p4_group.isChecked()
        settings['source_control_workspace'] = self.p4_workspace_line_edit.text() if self.p4_group.isChecked() else None
        settings['project_workspace'] = self.p4_project_workspace_line_edit.text() if self.p4_group.isChecked() else None
        settings['p4_sync_path'] = self.p4_project_path_line_edit.text() if self.p4_group.isChecked() else None
        settings['p4_engine_path'] = self.p4_engine_path_line_edit.text() if self.p4_group.isChecked() else None
        return settings

    def on_uproject_changed(self, text):
        self.uproject = os.path.normpath(text)

        # Update the config path with a suggestion based on the project name, if empty.
        if not self.config_path_line_edit.text().strip() and self.uproject.lower().endswith('.uproject'):
            self.config_path_line_edit.setText(pathlib.PurePath(self.uproject).stem)

        # Update the Engine directory with a best guess, if it is empty.
        if not self.engine_dir_line_edit.text():
            try:
                # Try going up the path until you find Engine in a sibling folder. This will work for custom build p4 setups.
                self.engine_dir_line_edit.setText(str(Path(self.find_upstream_path_with_name(self.uproject, 'Engine', includeSiblings=True))))
            except FileNotFoundError:
                pass

        self.update_button_box()

    def on_browse_config_path(self):
        config_path_str, _ = QtWidgets.QFileDialog.getSaveFileName(self,
            'Select config file', str(config.CONFIG_MGR.config_dir),
            f'Config files (*{config.CONFIG_SUFFIX})')

        if not config_path_str:
            return

        # First attempt to get a relative path then allow for an absolute
        #
        try:
            config_path_str = str(
                config.CONFIG_MGR.get_relative_config_path(config_path_str))
        except Exception as e:
            config_path_str = str(config.CONFIG_MGR.get_absolute_config_path(config_path_str))

        self.config_path_line_edit.setText(config_path_str)

    def on_browse_uproject_path(self, uproject_search_path):

        new_file, _ = QtWidgets.QFileDialog.getOpenFileName(self, "Select uProject file", self.engine_dir, "uProject (*.uproject)")

        if not new_file:
            return

        self.uproject = new_file
        self.uproject = os.path.normpath(self.uproject)
        self.uproject_line_edit.setText(self.uproject)

    def on_engine_dir_changed(self, text):
        self.engine_dir = os.path.normpath(text)
        self.update_button_box()

    def on_browse_engine_dir(self):
        new_dir = QtWidgets.QFileDialog.getExistingDirectory(self, "Select UE 'Engine' directory")

        if not new_dir:
            return

        self.engine_dir = os.path.normpath(str(Path(new_dir)))
        self.engine_dir_line_edit.setText(self.engine_dir)

    def update_button_box(self):
        self.button_box.button(QtWidgets.QDialogButtonBox.Ok).setEnabled(False)
        if (self.config_path_line_edit.hasAcceptableInput() and
                ((not self.uproject) or os.path.exists(self.uproject)) and
                self.engine_dir and os.path.exists(self.engine_dir)):
            self.button_box.button(QtWidgets.QDialogButtonBox.Ok).setEnabled(True)
