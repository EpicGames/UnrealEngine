# Copyright Epic Games, Inc. All Rights Reserved.

import collections
import datetime
import fnmatch
import io
import json
import os
import pathlib
import shutil
import socket
import sys
import threading
import time
import weakref

from enum import Enum
from pathlib import Path, PurePosixPath
from typing import Any, Callable, Optional, Tuple, Type, Union, List

from PySide6 import QtCore
from PySide6 import QtGui
from PySide6 import QtWidgets

from switchboard import switchboard_widgets as sb_widgets
from switchboard.switchboard_logging import LOGGER
from switchboard.switchboard_widgets import DropDownMenuComboBox, NonScrollableComboBox
from switchboard import ugs_utils
from switchboard.sbcache import SBCache, Map
from switchboard.ue_plugin_utils import UnrealPlugin, UnrealPluginManager


CONFIG_SUFFIX = '.json'

DEFAULT_MAP_TEXT = '-- Default Map --'

ENABLE_UGS_SUPPORT = False


class ConfigManager:
    DEFAULT_CONFIG_DIR = pathlib.Path(__file__).parent.with_name('configs')
    CONFIG_ENV_VAR = 'SWITCHBOARD_CONFIG_DIR'

    USER_SETTINGS_FILE_NAME = 'user_settings.json'
    USER_SETTINGS_BACKUP_FILE_NAME = 'corrupted_user_settings_backup.json'

    def __init__(
        self,
        config_dir: pathlib.Path | None = None
    ):
        LOGGER.debug('Initializing ConfigManager')

        self.config_dir = config_dir or self._determine_config_dir()
        LOGGER.debug(f'Config dir set to {self.config_dir}')

    @classmethod
    def _fmt_env_var(cls, var) -> str:
        return f'%{var}%' if sys.platform.startswith('win') else f'${var}'

    def _config_dir_valid(self, check_dir: pathlib.Path) -> bool:
        check_dir.mkdir(parents=True, exist_ok=True)
        return check_dir.is_dir()

    def _determine_config_dir(self) -> pathlib.Path:
        sb_buildver = ugs_utils.get_sb_buildver()
        if sb_branch := sb_buildver.get('BranchName'): # "++UE5+Main"
            sb_branch = str(sb_branch).strip('+').replace('+', '_').upper() # "UE5_MAIN"
            stream_env_var = f'{self.CONFIG_ENV_VAR}_{sb_branch}' # "SB_CONFIG_DIR_UE5_MAIN"

            LOGGER.debug(f'Looking for config dir override: {self._fmt_env_var(stream_env_var)}')
            if stream_dir := os.environ.get(stream_env_var):
                stream_dir = pathlib.Path(stream_dir)
                if self._config_dir_valid(stream_dir):
                    return stream_dir

        LOGGER.debug(f'Looking for config dir override: {self._fmt_env_var(self.CONFIG_ENV_VAR)}')
        if override_dir := os.environ.get(self.CONFIG_ENV_VAR):
            override_dir = pathlib.Path(override_dir)
            if self._config_dir_valid(override_dir):
                return override_dir

        return self.DEFAULT_CONFIG_DIR

    @property
    def user_settings_file_path(self) -> pathlib.Path:
        '''Path to the "global" settings, including the last loaded config.'''
        # Note: We use the default dir here to keep this local even if configs are shared.
        return self.DEFAULT_CONFIG_DIR.joinpath(self.USER_SETTINGS_FILE_NAME)

    @property
    def user_settings_backup_file_path(self) -> pathlib.Path:
        '''Path at which a backup is automatically created if we're unable to load.'''
        # Note: We use the default dir here to keep this local even if configs are shared.
        return self.DEFAULT_CONFIG_DIR.joinpath(self.USER_SETTINGS_BACKUP_FILE_NAME)

    def list_config_paths(self) -> list[pathlib.Path]:
        '''
        Returns a list of absolute paths to all config files in the configs dir.
        '''
        self.config_dir.mkdir(parents=True, exist_ok=True)

        # Find all JSON files in the config dir recursively, but exclude the user
        # settings file.
        config_paths = [
            path for path in self.config_dir.rglob(f'*{CONFIG_SUFFIX}')
            if (path != self.user_settings_file_path and
                path != self.user_settings_backup_file_path)
        ]

        return config_paths

    def get_absolute_config_path(
        self,
        config_path: Union[str, pathlib.Path]
    ) -> pathlib.Path:
        '''
        Returns the given string or path object as an absolute config path.

        The string/path is validated to ensure that:
        - It is not empty, or all whitespace
        - It ends with the config path suffix
        - It is not the same path as the user settings file path
        '''
        if isinstance(config_path, str):
            config_path = config_path.strip()
            if not config_path:
                raise ConfigPathEmptyError('Config path cannot be empty')

            config_path = pathlib.Path(config_path)

        # Manually add the suffix instead of using pathlib.Path.with_suffix().
        # For strings like "foo.bar", with_suffix() will first remove ".bar"
        # before adding the suffix, which we don't want it to do.
        if not config_path.name.endswith(CONFIG_SUFFIX):
            config_path = config_path.with_name(
                f'{config_path.name}{CONFIG_SUFFIX}')

        if not config_path.is_absolute():
            # Relative paths can simply be made absolute.
            config_path = self.config_dir.joinpath(config_path)

        if config_path.resolve() == self.user_settings_file_path:
            raise ConfigPathIsUserSettingsError(
                'Config path cannot be the same as the user settings file '
                f'path "{self.user_settings_file_path}"')

        return config_path

    def get_relative_config_path(
        self,
        config_path: Union[str, pathlib.Path]
    ) -> pathlib.Path:
        '''
        Returns the given string or path object as a config path relative to the
        root configs path.

        An absolute path is generated first to perform all of the same validation
        as get_absolute_config_path() before the relative path is computed and
        returned.
        '''
        config_path = self.get_absolute_config_path(config_path)
        return config_path.relative_to(self.config_dir)


CONFIG_MGR = ConfigManager()


def map_name_is_valid(map_name: str) -> bool:
    """
    Check if map is specified
    """
    return (map_name != ''
            and map_name is not None
            and map_name != DEFAULT_MAP_TEXT
            and map_name != 'Default level'
            and map_name != 'None')


def get_game_launch_level_path() -> str:
    level = CONFIG.CURRENT_LEVEL

    # check if map name is valid. If not - get map name from DefaultEngine.ini
    if not map_name_is_valid(level):
        level = ''

        # check if DefaultEngine.ini exists
        engine_ini = os.path.join(pathlib.Path(CONFIG.UPROJECT_PATH.get_value()).parent, 'Config', 'DefaultEngine.ini')
        if not os.path.exists(engine_ini):
            LOGGER.warning("DefaultEngine.ini not found")
            return ''

        parser = ugs_utils.IniParser()

        try:
            with open(engine_ini, encoding='utf-8') as engine_ini_file:
                parser.read_file(engine_ini_file)

                levels = parser.try_get('/Script/EngineSettings.GameMapsSettings',
                                        'GameDefaultMap')

            if not levels:
                raise Exception('Default map is not set in DefaultEngine.ini')

            level = levels[0]
        except Exception:
            LOGGER.warning("Map is not set. Please select 'Level' to run dedicated server.")

    return str(level)


def migrate_comma_separated_string_to_list(value) -> list[str]:
    if isinstance(value, str):
        return value.split(",")
    # Technically we should check whether every element is a string but we skip it here
    if isinstance(value, list):
        return value
    raise NotImplementedError("Migration not handled")


def override(method):
    '''Decorator to indicate that a method is overriding a base class method.'''
    return method


def get_parent_dialog_or_window(widget: QtWidgets.QWidget) -> QtWidgets.QWidget:
    '''
    Gets the parent dialog or main window of the given widget.

    Args:
        widget : The widget for which to find the parent dialog or main window.
    '''
    if widget is None:
        return None

    parent = widget.parentWidget()

    while parent and not isinstance(parent, QtWidgets.QDialog) and not isinstance(parent, QtWidgets.QMainWindow):
        parent = parent.parentWidget()

    return parent


class BatchItemToCreateUI:
    ''' create_ui batch item with all the Setting to be generated and added to the destination form layout '''

    def __init__(
            self,
            setting: 'Setting',
            layout: QtWidgets.QFormLayout,
            device_name: Optional[str] = None):  

        self.setting = setting  # the setting we're creating the top widget for
        self.layout = layout  # the layout where it should be added
        self.device_name = device_name  # The device override name this setting applies to


class SettingsBatchManagerToCreateUI(QtCore.QObject):
    '''
    Manages the batch processing of settings dialog widgets, adding them to the UI in manageable
    chunks to improve responsiveness.
    '''

    # Our singleton instance
    _instance = None

    @classmethod
    def get_instance(cls):
        if cls._instance is None:
            cls._instance = cls()
        return cls._instance

    '''
    Signal emitted when a batch of settings is completed.

    Args:
        int: Number of settings added during the batch. Receiver can use this to react to progress.
        bool: True if there is more pending work, False otherwise.
    '''
    signal_batch_completed = QtCore.Signal(int, bool)

    def __init__(self, max_milliseconds_per_batch=1000//60):

        # Singleton pattern.

        if self.__class__._instance is not None:
            raise Exception("This class is a singleton!")

        self.__class__._instance = self

        super().__init__()

        self.pending_settings: List[BatchItemToCreateUI] = []
        self.max_milliseconds_per_batch = max_milliseconds_per_batch
        self.batch_timer = QtCore.QTimer()
        self.batch_timer.timeout.connect(self.process_next_batch)

    def add_setting(
            self,
            setting: 'Setting',
            form_layout: QtWidgets.QFormLayout,
            override_device_name: Optional[str] = None):
        '''
        Adds a setting to the batch manager, to be processed later in a batch.

        Args:
            setting : The setting instance to add.
            form_layout : The layout to add the setting widget to.
            override_device_name : The device name to override. Can be None.
        '''

        self.pending_settings.append(BatchItemToCreateUI(setting, form_layout, override_device_name))

        # Start processing batches if the timer is not already active
        if not self.batch_timer.isActive():
            self.batch_timer.start(0)  # Start the timer to process on next event loop

    def process_next_batch(self):
        '''
        Processes the next batch of settings, replacing placeholders with the actual widgets.
        '''
        start_time = QtCore.QElapsedTimer()
        start_time.start()

        settings_done_count = 0

        while self.pending_settings:
            batchitem = self.pending_settings.pop(0)
            settings_done_count += 1

            # Skip the work if the parent dialog or window has been destroyed or is no longer visible
            owner = get_parent_dialog_or_window(batchitem.layout)
            if not owner or not owner.isVisible():
                continue

            batchitem.setting.add_setting_to_layout(batchitem)

            # Check if the maximum allowed time per batch has been exceeded
            if start_time.elapsed() > self.max_milliseconds_per_batch:
                break

        if not self.pending_settings:
            self.batch_timer.stop()  # Stop the timer when there are no more pending settings

        # Notify our observers of the progress
        self.signal_batch_completed.emit(settings_done_count, self.has_pending_work())

    def has_pending_work(self):
        return len(self.pending_settings) > 0


class Setting(QtCore.QObject):
    '''
    A type-agnostic value container for a configuration setting.

    This base class can be used directly for Settings that will
    never appear in the UI. Otherwise, Settings that can be modified
    in the UI must be represented by a derived class that creates
    the appropriate widget(s) for modifying the Setting's value.
    '''

    '''
    signal_setting_changed is emitted when the base value changes

    Args:
        old_value : old value
        value : new value
    '''
    signal_setting_changed = QtCore.Signal(object, object)

    '''
    signal_setting_overridden is emitted when the base value is overridden

    Args:
        device_name : Name of the device
        value : Base value
        override : Override value
    '''
    signal_setting_overridden = QtCore.Signal(str, object, object)

    def _filter_value(self, value):
        '''
        Filter function to modify the incoming value before updating or
        overriding the setting.

        The base class implementation does not apply any filtering to values.
        '''
        return value

    def __init__(
        self,
        attr_name: str,
        nice_name: str,
        value,
        tool_tip: Optional[str] = None,
        show_ui: bool = True,
        allow_reset: bool = True,
        migrate_data: Optional[Callable[[Any], None]] = None,
        category: str = 'Misc',
        allow_override_style: bool = True,
    ):
        '''
        Create a new Setting object.

        Args:
            attr_name   : Internal name.
            nice_name   : Display name.
            value       : The initial value of this Setting.
            tool_tip    : Tooltip to show in the UI for this Setting.
            show_ui     : Whether to show this Setting in the Settings UI.
            allow_reset : Allows showing a reset button when the value differs from the default.
            migrate_data: Optional function to migrate data already stored when value structure changes.
            category    : Used for UI grouping with other properties.
        '''
        super().__init__()

        self.attr_name = attr_name
        self.nice_name = nice_name
        self.category = category
        self.allow_override_style = allow_override_style

        ''' updating_widgets is a flag to avoid update recursion. 
        The Setting value or override cannot be set when True, except the first
        time in the recursion.
        '''
        self.updating_widgets = False

        value = self._filter_value(value)
        self._original_value = self._value = value

        # todo-dara: overrides are identified by device name right now. This
        # should be changed to the hash instead. That way we could avoid
        # having to patch the overrides and settings in CONFIG when a device
        # is renamed.
        self._overrides = {}

        # These members store the UI widgets for the "base" Setting as well as
        # any overrides of the setting, similar to the way we store the base
        # value and overrides of the value. They identify the widget in the UI
        # that should be highlighted when the Setting is overridden. Derived
        # classes should call set_widget() with an override device name if
        # appropriate in their implementations of _create_widgets() if they
        # want override highlighting.
        self._base_widget = None
        self._override_widgets = {}
        self._on_setting_changed_lambdas = {}

        # Appears when override value is different from _value
        self._allow_reset = allow_reset
        self._base_reset_widget = None
        self._reset_override_widgets = {}

        self._migrate_data = migrate_data

        self._get_json_override_fn: Optional[Callable[[Any], Any]] = None
        self._config_set_override_fn: Optional[Callable[[Any], Any]] = None

        self.tool_tip = tool_tip
        self.show_ui = show_ui

    def with_get_json_override_fn(self, fn: Callable[[Any], Any]):
        self._get_json_override_fn = fn
        return self

    def with_config_set_override_fn(self, fn: Callable[[Any], Any]):
        self._config_set_override_fn = fn
        return self

    def is_overridden(self, device_name: str) -> bool:
        return device_name in self._overrides

    def remove_override(self, device_name: str):
        self._overrides.pop(device_name, None)

    def update_value(self, new_value):
        ''' Sets the base value of this setting.
        Use override_value to override the value for a given device_name.
        '''

        # When updating widgets, we're inside a recursion, so we stop it here.
        if self.updating_widgets:
            return

        # prevent future recursion
        self.updating_widgets = True

        try:
            new_value = self._filter_value(new_value)

            if self._value == new_value:
                return

            old_value = self._value
            self._value = new_value

            self.signal_setting_changed.emit(old_value, self._value)
            self._refresh_reset_base_widget()
        finally:
            # Re-allow value updates
            self.updating_widgets = False

    def override_value(self, device_name: str, override):
        ''' Overrides the base value of this setting for the given device_name

        Args:
            device_name : The device name associated with the new override value.
            override    : The value associated with the device_name.
        '''
        # When updating widgets, we're inside a recursion, so we stop it here.
        if self.updating_widgets:
            return

        # prevent future recursion
        self.updating_widgets = True

        try:
            override = self._filter_value(override)

            # Don't do anything if the override has the same value
            if (device_name in self._overrides and
                    self._overrides[device_name] == override):
                return

            self._overrides[device_name] = override
            self.signal_setting_overridden.emit(device_name, self._value, override)

            self._refresh_reset_override_widget(device_name)
        finally:
            # Re-allow value updates
            self.updating_widgets = False

    def get_value(self, device_name: Optional[str] = None):
        ''' Reads the value of this setting.

        Args:
            device_name : Use to read the value overridden for the given device_name
        '''
        def get_value(self, device_name: Optional[str] = None):
            try:
                return self._overrides[device_name]
            except KeyError:
                return self._value

        value = get_value(self, device_name)
        return self._migrate_data(value) if self._migrate_data is not None else value

    def get_value_json(self, device_name: Optional[str] = None):
        # Optionally override to change serialization behavior.
        value = self.get_value(device_name)

        if self._get_json_override_fn:
            value = self._get_json_override_fn(value)

        return value

    def update_value_from_config(self, new_value):
        # Optionally override to change deserialization behavior.
        if self._config_set_override_fn:
            new_value = self._config_set_override_fn(new_value)

        self.update_value(new_value)

    def override_value_from_config(self, device_name: str, override):
        # Optionally override to change deserialization behavior.
        if self._config_set_override_fn:
            override = self._config_set_override_fn(override)

        self.override_value(device_name, override)

    def on_device_name_changed(self, old_name: str, new_name: str):
        if old_name in self._overrides.keys():
            self._overrides[new_name] = self._overrides.pop(old_name)

        if old_name in self._override_widgets.keys():
            self._override_widgets[new_name] = (self._override_widgets.pop(old_name))

    def reset(self):
        self._value = self._original_value
        self._overrides = {}
        self._override_widgets = {}

    def _create_widgets(
            self, override_device_name: Optional[str] = None) \
            -> Union[QtWidgets.QWidget, QtWidgets.QLayout]:
        '''
        Create the widgets necessary to manipulate this Setting in the UI.

        Settings that can appear in the UI must provide their own
        implementation of this function. If override highlighting is desired,
        the implementation should also set the override widget member variable.

        This function should return the "top-level" widget or layout. In
        some cases such as the BoolSetting this will just be a QCheckBox,
        whereas in others like the FilePathSetting, this will be a QHBoxLayout
        that contains line edit and button widgets.
        '''
        raise NotImplementedError(
            f'No UI for Setting "{self.nice_name}". '
            'Settings that are intended to display in the UI must '
            'derive from the Setting class and override _create_widgets().')

    def _on_widget_value_changed(
            self, new_value,
            override_device_name: Optional[str] = None):
        '''
        Update this Setting in response to a change in value caused by UI
        manipulation.

        The value is applied as an override if appropriate, in which case if
        an override widget has been identified, it will be highlighted.

        It should not be necessary to override this function in derived
        classes.
        '''
        if override_device_name is None:
            self.update_value(new_value)
            self._refresh_reset_override_widgets()
            return

        old_value = self.get_value(override_device_name)
        if new_value != old_value:
            self.override_value(override_device_name, new_value)

        if self.allow_override_style:
            if (widget := self.get_widget(override_device_name)):
                sb_widgets.set_qt_property(widget, "override", self.is_overridden(override_device_name))

    def _on_setting_changed(
            self, new_value,
            override_device_name: Optional[str] = None):
        '''
        Callback invoked when the value of this Setting changes.

        This can be implemented in derived classes to update the appropriate
        UI elements in response to value changes *not* initiated by the
        Setting's UI.

        The default implementation does nothing.
        '''
        pass

    def _on_setting_overridden(
            self,
            new_value,
            override_device_name: str):
        '''
        Callback invoked when the override value of this Setting changes.

        This can be implemented in derived classes to update the appropriate
        UI elements.

        The default implementation does nothing.
        '''
        pass

    def create_ui(
            self,
            override_device_name: Optional[str] = None,
            form_layout: Optional[QtWidgets.QFormLayout] = None) \
            -> None:
        '''
        Create the UI for this Setting. The widget will be added incrementally by the batch manager.
        '''

        # Honor request to not be shown in the UI
        if not self.show_ui:
            return None

        if form_layout:
            # Schedule the widget to be added incrementally
            self.get_create_ui_batch_manager().add_setting(self, form_layout, override_device_name)

        return None

    def add_setting_to_layout(self, item: BatchItemToCreateUI):
        '''
        Adds a setting to the provided layout.

        Args:
            item (BatchItemToCreateUI):
                The batch item containing the setting information, layout, and an optional 
                device name.
        '''

        # Create the widget for this setting.
        top_level_widget = self._create_widgets(item.device_name)

        widget = self.get_widget(item.device_name)
        if widget and self.is_overridden(item.device_name):
            sb_widgets.set_qt_property(widget, "override", True)

        # Register the Setting's change listeners
        self._register_on_setting_changed(top_level_widget, item.device_name)

        if top_level_widget and item.layout:

            # Create the row label.
            setting_label = QtWidgets.QLabel()
            setting_label.setText(self.nice_name)
            if self.tool_tip:
                setting_label.setToolTip(self.tool_tip)

            # Create the field widget
            field_widget = self._decorate_with_reset_widget(item.device_name, top_level_widget)

            # Add the row with the label and widget to the layout.
            item.layout.addRow(setting_label, field_widget)

    @staticmethod
    def get_create_ui_batch_manager():
        return SettingsBatchManagerToCreateUI.get_instance()

    def _pre_on_setting_changed(self, old_value, new_value, override_device_name):
        ''' Called when the base value of the setting changes. '''

        # Since this is a notification of base value changed, don't call _on_setting_changed if the value is being
        # overridden for the given device name. Otherwise _on_setting_changed may incorrectly update the override
        # widget with the base value.
        if not self.is_overridden(override_device_name):
            self._on_setting_changed(new_value, override_device_name=override_device_name)

    def _register_on_setting_changed(self, top_level_widget: QtWidgets.QWidget, override_device_name: str):
        ''' Registers the callbacks when the value of the setting changes. '''

        # Register the "pre" change which will filter the call to _on_setting_changed

        self_weak_ref = weakref.ref(self)  # Use weak to avoid holding a strong reference to the object

        def pre_on_setting_changed_weak(old_value, new_value):
            ''' Captures override_device_name but not self to avoid memory leaks due to strong reference '''
            self_instance = self_weak_ref()
            if self_instance is not None:
                self_instance._pre_on_setting_changed(old_value, new_value, override_device_name)

        self.signal_setting_changed.connect(pre_on_setting_changed_weak)

        # Clear the widget when it is destroyed

        def on_widget_destroyed_weak(destroyed_object=None):
            ''' Captures override_device_name but not self to avoid memory leaks due to strong reference '''
            self_instance = self_weak_ref()
            if self_instance is not None:

                # Disconnect signal_setting_changed to prevent accumulation
                try:
                    self_instance.signal_setting_changed.disconnect(pre_on_setting_changed_weak)
                except (RuntimeError, TypeError):
                    pass

                self_instance._on_widget_destroyed(override_device_name=override_device_name)

        top_level_widget.destroyed.connect(on_widget_destroyed_weak)

    def _on_widget_destroyed(self, destroyed_object=None, override_device_name: str | None = None):
        ''' Called when the widget associated with the given override_device_name is destroyed. '''
        self.set_widget(widget=None, override_device_name=override_device_name)
        if override_device_name is None:
            self._base_reset_widget = None
        else:
            self._reset_override_widgets.pop(override_device_name, None)

    def _decorate_with_reset_widget(self, override_device_name: str, setting_editor_widget: QtWidgets.QWidget):
        # Reset will still be shown on overrides
        if not self._allow_reset and override_device_name is None:
            return setting_editor_widget

        horizontal_box = QtWidgets.QWidget()
        horizontal_layout = QtWidgets.QHBoxLayout(horizontal_box)
        horizontal_layout.setContentsMargins(0, 0, 0, 0)
        horizontal_layout.setSpacing(3)

        if isinstance(setting_editor_widget, QtWidgets.QWidget):
            horizontal_layout.addWidget(setting_editor_widget)
        elif isinstance(setting_editor_widget, QtWidgets.QLayout):
            horizontal_layout.addLayout(setting_editor_widget)

        button = QtWidgets.QPushButton()
        pixmap = QtGui.QPixmap(":icons/images/reset_to_default.png")
        button.setIcon(QtGui.QIcon(pixmap))
        button.setFlat(True)
        button.setSizePolicy(QtWidgets.QSizePolicy.Maximum, QtWidgets.QSizePolicy.Maximum)
        button.setMaximumWidth(12)
        button.setMaximumHeight(12)
        button.setToolTip("Reset to default")
        button.setFocusPolicy(QtCore.Qt.FocusPolicy.NoFocus)
        button.pressed.connect(
            lambda override_device_name=override_device_name:
                self._on_press_reset_override(override_device_name)
        )

        horizontal_layout.addWidget(button)
        is_base_reset_widget = override_device_name is None
        if is_base_reset_widget:
            self._base_reset_widget = button
            self._refresh_reset_base_widget()
        else:
            self._reset_override_widgets[override_device_name] = button
            self._refresh_reset_override_widget(override_device_name)
        return horizontal_box

    def _on_press_reset_override(self, override_device_name: str):
        if override_device_name is None:
            self.remove_override(override_device_name)
            self.update_value(self._original_value)
            self._base_reset_widget.setVisible(False)
            self._refresh_reset_override_widgets()
            return

        if self.is_overridden(override_device_name):
            self.remove_override(override_device_name)

            # Update UI
            self._on_widget_value_changed(self._value, override_device_name)
            self._on_setting_changed(self._value, override_device_name=override_device_name)
            self._reset_override_widgets[override_device_name].setVisible(False)

    def _refresh_reset_base_widget(self):
        if self._base_reset_widget:
            self._base_reset_widget.setVisible(self._value != self._original_value)

    def _refresh_reset_override_widgets(self):
        for device_name in self._overrides.keys():
            self._refresh_reset_override_widget(device_name)

    def _refresh_reset_override_widget(self, device_name: str):
        if device_name in self._reset_override_widgets:
            self._reset_override_widgets[device_name].setVisible(self.is_overridden(device_name))

    def set_widget(
            self, widget: Optional[QtWidgets.QWidget] = None,
            override_device_name: Optional[str] = None):
        '''
        Set the widget to be used to manipulate this Setting, or
        this particular device's override of the Setting.

        A value of None can be provided for the widget to clear any
        stored widgets.
        '''
        if widget is None:
            # Clear the widget for this setting.
            if override_device_name is None:
                self._base_widget = None
            else:
                self._override_widgets.pop(override_device_name, None)
        else:
            if override_device_name is None:
                self._base_widget = widget
            else:
                self._override_widgets[override_device_name] = widget

    def get_widget(
            self, override_device_name: Optional[str] = None) \
            -> Optional[QtWidgets.QWidget]:
        '''
        Get the widget to be used to manipulate this Setting, or
        this particular device's override of the Setting.

        If no such widget was ever specified, None is returned.
        '''
        return self._override_widgets.get(
            override_device_name, self._base_widget)


class BoolSetting(Setting):
    '''
    A UI-displayable Setting for storing and modifying a boolean value.
    '''

    def _create_widgets(
            self, override_device_name: Optional[str] = None) \
            -> QtWidgets.QCheckBox:
        check_box = QtWidgets.QCheckBox()
        check_box.setChecked(self.get_value(override_device_name))
        if self.tool_tip:
            check_box.setToolTip(self.tool_tip)

        self.set_widget(
            widget=check_box, override_device_name=override_device_name)

        check_box.stateChanged.connect(
            lambda state, override_device_name=override_device_name:
                self._on_widget_value_changed(
                    bool(state), override_device_name=override_device_name))

        return check_box

    def _on_setting_changed(
            self, new_value: bool,
            override_device_name: Optional[str] = None):

        widget = self.get_widget(override_device_name=override_device_name)
        if widget:
            widget.setChecked(new_value)


class IntSetting(Setting):
    '''
    A UI-displayable Setting for storing and modifying an integer value.
    '''

    def __init__(
        self,
        *args,
        is_read_only: bool = False,
        **kwargs
    ):
        '''
        Create a new IntSetting object.

        Args:
            is_read_only    : Whether to make entry field editable or not.
        '''
        super().__init__(*args, **kwargs)

        self.is_read_only = is_read_only

    def _create_widgets(
            self, override_device_name: Optional[str] = None) \
            -> QtWidgets.QLineEdit:
        line_edit = QtWidgets.QLineEdit()
        if self.tool_tip:
            line_edit.setToolTip(self.tool_tip)
        line_edit.setValidator(QtGui.QIntValidator())

        value = str(self.get_value(override_device_name))
        line_edit.setText(value)
        line_edit.setCursorPosition(0)
        line_edit.setReadOnly(self.is_read_only)

        self.set_widget(
            widget=line_edit, override_device_name=override_device_name)

        line_edit.editingFinished.connect(
            lambda line_edit=line_edit,
            override_device_name=override_device_name:
                self._on_widget_value_changed(
                    int(line_edit.text()),
                    override_device_name=override_device_name))

        return line_edit

    def _on_setting_changed(
            self, new_value: int,
            override_device_name: Optional[str] = None):

        widget = self.get_widget(override_device_name=override_device_name)
        if not widget:
            return

        old_value = int(widget.text())
        if new_value != old_value:
            widget.setText(str(new_value))
            widget.setCursorPosition(0)


class FloatSetting(Setting):
    '''
    A UI-displayable Setting for storing and modifying a float value.
    '''

    def __init__(
        self,
        *args,
        is_read_only: bool = False,
        **kwargs
    ):
        '''
        Create a new FloatSetting object.

        Args:
            is_read_only    : Whether to make entry field editable or not.
        '''
        super().__init__(*args, **kwargs)

        self.is_read_only = is_read_only

    def _create_widgets(
            self, override_device_name: Optional[str] = None) \
            -> QtWidgets.QLineEdit:
        line_edit = QtWidgets.QLineEdit()
        if self.tool_tip:
            line_edit.setToolTip(self.tool_tip)
        line_edit.setValidator(QtGui.QDoubleValidator())

        value = str(self.get_value(override_device_name))
        line_edit.setText(value)
        line_edit.setCursorPosition(0)
        line_edit.setReadOnly(self.is_read_only)

        self.set_widget(
            widget=line_edit, override_device_name=override_device_name)

        line_edit.editingFinished.connect(
            lambda line_edit=line_edit,
            override_device_name=override_device_name:
                self._on_widget_value_changed(
                    float(line_edit.text()),
                    override_device_name=override_device_name))

        return line_edit

    def _on_setting_changed(
            self, new_value: float,
            override_device_name: Optional[str] = None):

        widget = self.get_widget(override_device_name=override_device_name)
        if not widget:
            return

        old_value = float(widget.text())
        if new_value != old_value:
            widget.setText(str(new_value))
            widget.setCursorPosition(0)


class StringSetting(Setting):
    '''
    A UI-displayable Setting for storing and modifying a string value.
    '''

    def __init__(
        self,
        *args,
        placeholder_text: str = '',
        is_read_only: bool = False,
        **kwargs
    ):
        '''
        Create a new StringSetting object.

        Args:
            placeholder_text: Placeholder for this Setting's value in the UI.
            is_read_only    : Whether to make entry field editable or not.
        '''
        super().__init__(*args, **kwargs)

        self.placeholder_text = placeholder_text
        self.is_read_only = is_read_only

    def _create_widgets(
            self, override_device_name: Optional[str] = None) \
            -> QtWidgets.QLineEdit:
        line_edit = QtWidgets.QLineEdit()
        if self.tool_tip:
            line_edit.setToolTip(self.tool_tip)

        value = str(self.get_value(override_device_name))
        line_edit.setText(value)
        line_edit.setPlaceholderText(self.placeholder_text)
        line_edit.setCursorPosition(0)
        line_edit.setReadOnly(self.is_read_only)

        self.set_widget(
            widget=line_edit, override_device_name=override_device_name)

        def on_editing_finished():
            self._on_widget_value_changed(
                line_edit.text().strip(),
                override_device_name=override_device_name)

            # In case the value the user entered is filtered to be the same as
            # the prior value, control should reflect that it was "rejected."
            line_edit.setText(self.get_value(override_device_name))

        line_edit.editingFinished.connect(on_editing_finished)

        return line_edit

    def _on_setting_changed(
            self, new_value: str,
            override_device_name: Optional[str] = None):

        widget = self.get_widget(override_device_name=override_device_name)
        if not widget:
            return

        old_value = widget.text().strip()
        if new_value != old_value:
            widget.setText(new_value)
            widget.setCursorPosition(0)


class FileSystemPathSetting(StringSetting):
    '''
    An abstract UI Setting for storing and modifying a string value that
    represents a file system path.

    This class provides the foundation for the DirectoryPathSetting and
    FilePathSetting classes. It should not be used directly.
    '''

    def _getFileSystemPath(
            self, parent: Optional[QtWidgets.QWidget] = None,
            start_path: str = '') -> str:
        raise NotImplementedError(
            f'Setting "{self.nice_name}" uses the FileSystemPathSetting '
            'class directly. A derived class (e.g. DirectoryPathSetting or '
            'FilePathSetting) must be used instead.')

    def _getStartPath(self) -> str:
        ''' Returns a reasonable start path for the file dialog'''

        start_path = str(pathlib.Path.home())

        if (SETTINGS.LAST_BROWSED_PATH and os.path.exists(SETTINGS.LAST_BROWSED_PATH)):
            start_path = SETTINGS.LAST_BROWSED_PATH

        return start_path

    def _create_widgets(
            self, override_device_name: Optional[str] = None) \
            -> QtWidgets.QHBoxLayout:
        line_edit = super()._create_widgets(
            override_device_name=override_device_name)

        edit_layout = QtWidgets.QHBoxLayout()
        edit_layout.addWidget(line_edit)

        browse_btn = QtWidgets.QPushButton('Browse')
        browse_btn.setFocusPolicy(QtCore.Qt.FocusPolicy.NoFocus)
        edit_layout.addWidget(browse_btn)

        def on_browse_clicked():
            start_path = self._getStartPath()

            fs_path = self._getFileSystemPath(
                parent=browse_btn, start_path=start_path)
            if len(fs_path) > 0 and os.path.exists(fs_path):
                fs_path = os.path.normpath(fs_path)

                self._on_widget_value_changed(
                    fs_path,
                    override_device_name=override_device_name)

                SETTINGS.LAST_BROWSED_PATH = os.path.dirname(fs_path)
                SETTINGS.save()

        browse_btn.clicked.connect(on_browse_clicked)

        return edit_layout


class DirectoryPathSetting(FileSystemPathSetting):
    '''
    A UI-displayable Setting for storing and modifying a string value that
    represents the path to a directory on the file system.
    '''

    def _getFileSystemPath(
            self, parent: Optional[QtWidgets.QWidget] = None,
            start_path: str = '') -> str:
        return QtWidgets.QFileDialog.getExistingDirectory(
            parent=parent, dir=start_path)


class FilePathSetting(FileSystemPathSetting):
    '''
    A UI-displayable Setting for storing and modifying a string value that
    represents the path to a file on the file system.
    '''

    def __init__(
        self,
        *args,
        file_path_filter: str = '',
        **kwargs
    ):
        '''
        Create a new FilePathSetting object.

        Args:
            file_path_filter: Filter to use in the file browser.
        '''
        super().__init__(*args, **kwargs)

        self.file_path_filter = file_path_filter

    def _getStartPath(self) -> str:
        ''' Override from base class'''

        current_path = self.get_value()

        if os.path.exists(current_path):
            return current_path

        return super()._getStartPath()

    def _getFileSystemPath(
            self, parent: Optional[QtWidgets.QWidget] = None,
            start_path: str = ''
            ) -> str:
        ''' Override from base class'''

        file_path, _ = QtWidgets.QFileDialog.getOpenFileName(
            parent=parent, dir=start_path, filter=self.file_path_filter)

        return file_path


class PerforcePathSetting(StringSetting):
    '''
    A UI-displayable Setting for storing and modifying a string value that
    represents a Perforce depot path.
    '''

    def __init__(
        self,
        *args,
        placeholder_text: str = '',
        is_read_only: bool = False,
        **kwargs
    ):
        # Trim matching file paths to the parent directory (e.g. ['.uproject'])
        self.truncate_files_with_extensions: list[str] = []

        super().__init__(*args, placeholder_text=placeholder_text, is_read_only=is_read_only, **kwargs)

    def _filter_value(self, value: Optional[str]) -> str:
        '''
        Clean the p4 path value by removing whitespace and trailing '/'.
        '''
        if not value:
            return ''

        value = value.strip()

        # Fix up common case where user incorrectly includes a file name.
        value_lower = value.lower()
        for ext in self.truncate_files_with_extensions:
            if value_lower.endswith(ext.lower()):
                value = value[:value.rfind('/')]
                break

        return value.rstrip('/')


class OptionSetting(Setting):
    '''
    A UI-displayable Setting for storing and modifying a value that is
    one of a fixed set of options.
    '''

    def __init__(
        self,
        *args,
        possible_values: Optional[list] = None,
        **kwargs
    ):
        '''
        Create a new OptionSetting object.

        Args:
            possible_values: Possible values for this Setting.
        '''
        super().__init__(*args, **kwargs)

        self.possible_values = possible_values or []

    def _create_widgets(
        self, override_device_name: Optional[str] = None, *,
        widget_class: Type[NonScrollableComboBox] = NonScrollableComboBox
    ) -> NonScrollableComboBox:
        combo = widget_class()
        if self.tool_tip:
            combo.setToolTip(self.tool_tip)

        for value in self.possible_values:
            combo.addItem(str(value), value)

        combo.setCurrentIndex(
            combo.findData(self.get_value(override_device_name)))

        self.set_widget(widget=combo,
                        override_device_name=override_device_name)

        combo.currentIndexChanged.connect(
            lambda index, override_device_name=override_device_name, combo=combo:
                self._on_widget_value_changed(
                    combo.itemData(index), override_device_name=override_device_name))

        return combo

    def _on_setting_changed(
            self, new_value,
            override_device_name: Optional[str] = None):

        widget = self.get_widget(override_device_name=override_device_name)
        if not widget:
            return

        old_value = widget.currentText()
        new_str_value = str(new_value)
        if new_str_value != old_value:
            index = widget.findText(new_str_value)
            if index != -1:
                widget.setCurrentIndex(index)


class MultiOptionSetting(OptionSetting):
    '''
    A UI-displayable Setting for storing and modifying a set of values,
    which may optionally be a subset of a fixed set of options.
    '''

    def _create_widgets(
            self, override_device_name: Optional[str] = None
    ) -> sb_widgets.MultiSelectionComboBox:
        combo = sb_widgets.MultiSelectionComboBox()
        if self.tool_tip:
            combo.setToolTip(self.tool_tip)

        selected_values = self.get_value(override_device_name)
        possible_values = (
            self.possible_values
            if len(self.possible_values) > 0 else selected_values)
        combo.add_items(selected_values, possible_values)

        self.set_widget(
            widget=combo, override_device_name=override_device_name)

        combo.signal_selection_changed.connect(
            lambda entries, override_device_name=override_device_name:
                self._on_widget_value_changed(
                    entries, override_device_name=override_device_name))

        return combo

    def _on_setting_changed(
            self, new_value,
            override_device_name: Optional[str] = None):

        widget = self.get_widget(override_device_name=override_device_name)
        if not widget:
            return

        widget_model = widget.model()
        widget_items = [widget_model.item(i, 0) for i in range(widget.count())]
        widget_items = [item for item in widget_items if item.isEnabled()]

        old_value = [
            item.data(QtCore.Qt.UserRole)
            for item in widget_items
            if item.checkState() == QtCore.Qt.Checked]
        if new_value != old_value:
            selected_item_texts = []
            for item in widget_items:
                if item.data(QtCore.Qt.UserRole) in new_value:
                    item.setCheckState(QtCore.Qt.Checked)
                    selected_item_texts.append(item.text())
                else:
                    item.setCheckState(QtCore.Qt.Unchecked)

            widget.setEditText(widget.separator.join(selected_item_texts))


class ListRow(QtWidgets.QWidget):
    INSERT_TEXT = "Insert"
    DUPLICATE_TEXT = "Duplicate"
    DELETE_TEXT = "Delete"

    def __init__(
        self,
        array_index: int,
        editor_widget: QtWidgets.QWidget,
        insert_item_callback: Optional[Callable[[], None]] = None,
        duplicate_item_callback: Optional[Callable[[], None]] = None,
        delete_item_callback: Optional[Callable[[], None]] = None,
        parent=None
    ):
        super().__init__(parent=parent)
        self._editor_widget = editor_widget
        self._insert_item_callback = insert_item_callback
        self._duplicate_item_callback = duplicate_item_callback
        self._delete_item_callback = delete_item_callback

        layout = QtWidgets.QHBoxLayout(self)
        layout.setSpacing(1)
        layout.setContentsMargins(0, 0, 0, 0)

        # Shift the elements to the right
        layout.addItem(
            QtWidgets.QSpacerItem(10, 0, QtWidgets.QSizePolicy.Maximum, QtWidgets.QSizePolicy.Maximum)
        )

        self._index_label = QtWidgets.QLabel()
        self._index_label.setFixedWidth(20)
        self.update_index(array_index)
        layout.addWidget(self._index_label)

        layout.addWidget(editor_widget)

        element_actions = DropDownMenuComboBox()
        element_actions.on_select_option.connect(self._on_view_option_selected)
        element_actions.addItem(self.INSERT_TEXT)
        element_actions.addItem(self.DUPLICATE_TEXT)
        element_actions.addItem(self.DELETE_TEXT)
        layout.addWidget(element_actions)

    def _on_view_option_selected(self, selected_item):
        if selected_item == self.INSERT_TEXT:
            self._insert_item_callback()
        elif selected_item == self.DUPLICATE_TEXT:
            self._duplicate_item_callback()
        elif selected_item == self.DELETE_TEXT:
            self._delete_item_callback()

    @property
    def editor_widget(self):
        return self._editor_widget

    def update_index(self, index: int):
        self._index_label.setText(str(index))


class ListSetting(Setting):
    '''
    A setting which has add and clear buttons. Each item will be displayed in a new line.
    Operates similarly to array properties in Unreal Editor.

    Subclasses are responsible for generating widgets for the array contents, see e.g. ArrayStringSetting.
    '''

    def __init__(
        self,
        *args,
        **kwargs
    ):
        '''
        Create a new ListSetting object.

        Args:
            value : list
        '''
        super().__init__(*args, **kwargs)

        self.array_count_labels = {}
        self.element_layouts = {}

    def create_element(self, override_device_name: str, index: int) -> Tuple[QtWidgets.QWidget, object]:
        '''
        Called when a new array element is supposed to be created.
        @returns The widget to use for editing and the default value for the new array element
        '''
        raise NotImplementedError("Subclasses must override this")

    def update_element_value(self, editor_widget: QtWidgets.QWidget, list_value: list, index: int):
        '''
        Called to update the editor_widget with the the value from list_value[index].
        '''
        raise NotImplementedError("Subclasses must override this")

    def _create_widgets(self, override_device_name: Optional[str] = None):
        root = QtWidgets.QWidget()
        root_layout = QtWidgets.QVBoxLayout(root)
        root_layout.setSpacing(1)
        root_layout.setContentsMargins(1, 1, 1, 1)

        root_layout.addWidget(
            self._create_header(override_device_name)
        )

        elements_root = QtWidgets.QWidget()
        elements_layout = QtWidgets.QVBoxLayout(elements_root)
        self.element_layouts[override_device_name] = elements_layout
        elements_layout.setSpacing(1)
        elements_layout.setContentsMargins(8, 5, 2, 2)
        root_layout.addWidget(
            elements_root
        )

        self.set_widget(
            widget=root, override_device_name=override_device_name)
        self._on_setting_changed(self.get_value(override_device_name), override_device_name)
        return root

    def _create_header(self, override_device_name) -> QtWidgets.QWidget:
        header = QtWidgets.QWidget()
        header_layout = QtWidgets.QHBoxLayout(header)
        header_layout.setSpacing(5)
        header_layout.setContentsMargins(0, 0, 0, 0)

        array_count_label = QtWidgets.QLabel()
        self.array_count_labels[override_device_name] = array_count_label
        array_count_label.setFixedWidth(100)
        self._update_array_count_label(override_device_name)

        add_button = QtWidgets.QPushButton()
        pixmap = QtGui.QPixmap(":icons/images/PlusSymbol_12x.png")
        add_button.setIcon(QtGui.QIcon(pixmap))
        add_button.setFlat(True)
        add_button.setSizePolicy(QtWidgets.QSizePolicy.Maximum, QtWidgets.QSizePolicy.Maximum)
        add_button.setMaximumWidth(12)
        add_button.setMaximumHeight(12)
        add_button.setFocusPolicy(QtCore.Qt.FocusPolicy.NoFocus)
        add_button.pressed.connect(
            lambda override_device_name=override_device_name:
                self._on_press_add(override_device_name)
        )

        clear_button = QtWidgets.QPushButton()
        pixmap = QtGui.QPixmap(":icons/images/empty_set_12x.png")
        clear_button.setIcon(QtGui.QIcon(pixmap))
        clear_button.setFlat(True)
        clear_button.setSizePolicy(QtWidgets.QSizePolicy.Maximum, QtWidgets.QSizePolicy.Maximum)
        clear_button.setMaximumWidth(12)
        clear_button.setMaximumHeight(12)
        clear_button.setFocusPolicy(QtCore.Qt.FocusPolicy.NoFocus)
        clear_button.pressed.connect(
            lambda override_device_name=override_device_name:
            self._on_press_clear(override_device_name)
        )

        header_layout.addWidget(array_count_label)
        header_layout.addWidget(add_button)
        header_layout.addWidget(clear_button)
        header_layout.addItem(QtWidgets.QSpacerItem(0, 0, QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Minimum))

        return header

    def _update_array_count_label(self, override_device_name: str):
        current_value = self.get_value(override_device_name)
        self.array_count_labels[override_device_name].setText(f"{len(current_value)} Elements")

    def _on_press_add(self, override_device_name: str):
        current_value = self.get_value(override_device_name)
        row_widget, default_value = self._create_element_row(override_device_name, len(current_value))
        self.element_layouts[override_device_name].addWidget(
            row_widget
        )

        self._on_widget_value_changed(current_value + [default_value], override_device_name)
        # Force update in case _on_widget_value_changed implicitly changed array settings without telling us
        self._on_setting_changed(self.get_value(override_device_name), override_device_name)

    def _on_press_clear(self, override_device_name: str):
        self._on_widget_value_changed([], override_device_name)

        container_layout = self.element_layouts[override_device_name]
        for i in reversed(range(container_layout.count())):
            container_layout.itemAt(i).widget().setParent(None)

        # Force update in case _on_widget_value_changed implicitly changed array settings without telling us
        self._on_setting_changed(self.get_value(override_device_name), override_device_name)

    def _insert_at(self, override_device_name: str, index: int):
        current_value = self.get_value(override_device_name)
        row_widget, default_value = self._create_element_row(override_device_name, len(current_value))
        self.element_layouts[override_device_name].addWidget(
            row_widget
        )

        copied_value = current_value.copy()
        copied_value.insert(index, default_value)

        self._on_widget_value_changed(copied_value, override_device_name)
        # Force update in case _on_widget_value_changed implicitly changed array settings without telling us
        self._on_setting_changed(self.get_value(override_device_name), override_device_name)

    def _duplicate_at(self, override_device_name: str, index: int):
        current_value = self.get_value(override_device_name)
        row_widget, _ = self._create_element_row(override_device_name, len(current_value))
        self.element_layouts[override_device_name].addWidget(
            row_widget
        )

        copied_value = current_value.copy()
        copied_value.insert(index, current_value[index])

        self._on_widget_value_changed(copied_value, override_device_name)
        # Force update in case _on_widget_value_changed implicitly changed array settings without telling us
        self._on_setting_changed(self.get_value(override_device_name), override_device_name)

    def _remove_at(self, override_device_name: str, index: int):
        current_value = self.get_value(override_device_name)
        copied_value = current_value.copy()
        del copied_value[index]

        self._on_widget_value_changed(copied_value, override_device_name)
        # Force update in case _on_widget_value_changed implicitly changed array settings without telling us
        self._on_setting_changed(self.get_value(override_device_name), override_device_name)

    def _create_element_row(self, override_device_name: str, index: int) -> Tuple[ListRow, object]:
        editing_widget, default_value = self.create_element(override_device_name, index)
        row = ListRow(
            index,
            editing_widget, 
            lambda override_device_name=override_device_name, index=index:
                self._insert_at(override_device_name, index),
            lambda override_device_name=override_device_name, index=index:
                self._duplicate_at(override_device_name, index),
            lambda override_device_name=override_device_name, index=index:
                self._remove_at(override_device_name, index)
        )
        return row, default_value

    @override
    def _on_setting_changed(
        self,
        new_value: list,
        override_device_name: Optional[str] = None
    ) -> None:

        self.update_list_ui(new_value=new_value, override_device_name=override_device_name)

    @override
    def _on_setting_overridden(
        self,
        new_value,
        override_device_name: str
    ) -> None:

        self.update_list_ui(new_value=new_value, override_device_name=override_device_name)

    def update_list_ui(
        self,
        new_value,
        override_device_name: str
    ) -> None:
        '''
        Update the UI to reflect the given list value.

        Args:
            new_value (list):
                The new list of values that should be represented in the UI.

            override_device_name (Optional[str]):
                The name of the device to which the override applies.
                If provided, updates the UI for the specific device.
        '''

        container_layout = self.element_layouts[override_device_name]

        new_len = len(new_value)
        new_list_is_bigger = container_layout.count() < new_len
        new_list_is_smaller = container_layout.count() > new_len
        if new_list_is_bigger:
            for missing_index in range(container_layout.count(), len(new_value), 1):
                new_array_row, _ = self._create_element_row(override_device_name, missing_index)
                container_layout.addWidget(new_array_row)

        if new_list_is_smaller:
            for added_index in reversed(range(new_len, container_layout.count(), 1)):
                container_layout.itemAt(added_index).widget().deleteLater()

        for index in range(new_len):
            array_row: ListRow = container_layout.itemAt(index).widget()
            self.update_element_value(array_row.editor_widget, new_value, index)

        self._update_array_count_label(override_device_name)

    def _on_widget_destroyed(self, destroyed_object=None, override_device_name: str | None = None):
        super()._on_widget_destroyed(destroyed_object, override_device_name)
        
        # Clean up the layout references to prevent exceptions related to accessing deleted Qt C++ objects.
        self.element_layouts.pop(override_device_name, None)
        self.array_count_labels.pop(override_device_name, None)


class StringListSetting(ListSetting):
    '''
    An array setting where the elements are strings
    '''

    @override
    def create_element(self, override_device_name: str, index: int) -> Tuple[QtWidgets.QWidget, object]:
        line_edit = QtWidgets.QLineEdit()
        line_edit.editingFinished.connect(
            lambda line_edit=line_edit, override_device_name=override_device_name, index=index:
                self._on_editor_widget_changed(line_edit, override_device_name, index)
        )
        return line_edit, ""

    def _on_editor_widget_changed(self, line_edit: QtWidgets.QLineEdit, override_device_name: str, index: int):
        '''
        Updates the value in the list when the editor widget's value changes.

        Args:
            line_edit (QtWidgets.QLineEdit): The line edit widget containing the new value.
            override_device_name (str): The device name for which the value is being updated.
            index (int): The index of the element being updated.
        '''
        current_list = self.get_value(override_device_name)

        # List needs to a different object otherwise _on_widget_value_changed will think the value has not changed
        copied_value = current_list.copy()
        copied_value[index] = line_edit.text()
        self._on_widget_value_changed(copied_value, override_device_name)

    @override
    def update_element_value(self, editor_widget: QtWidgets.QLineEdit, list_value: list, index: int):
        '''
        Updates the editor widget with the value from the list.

        Args:
            editor_widget (QtWidgets.QLineEdit): The widget to be updated.
            list_value (list): The list of values.
            index (int): The index of the value to be set in the widget.
        '''
        editor_widget.setText(list_value[index])

    @override
    def _create_header(self, override_device_name: str) -> QtWidgets.QWidget:
        '''
        Creates the header widget for the list setting, including the bulk edit button.

        Args:
            override_device_name (str): The device name for which the header is being created.

        Returns:
            QtWidgets.QWidget: The header widget.
        '''
        header = super()._create_header(override_device_name)
        header_layout = header.layout()

        # Use the edit icon to represent Bulk Edit. Contrast with the plus button to add new elements.
        pixmap = QtGui.QPixmap(":icons/images/icon_edit.png")

        bulk_edit_button = QtWidgets.QPushButton()
        bulk_edit_button.setIcon(QtGui.QIcon(pixmap))
        bulk_edit_button.setToolTip("Bulk Edit")
        bulk_edit_button.setFlat(True)
        bulk_edit_button.setSizePolicy(QtWidgets.QSizePolicy.Maximum, QtWidgets.QSizePolicy.Maximum)
        bulk_edit_button.setMaximumWidth(12)
        bulk_edit_button.setMaximumHeight(12)
        bulk_edit_button.setFocusPolicy(QtCore.Qt.FocusPolicy.NoFocus)
        bulk_edit_button.pressed.connect(
            lambda override_device_name=override_device_name:
                self._on_press_bulk_edit(override_device_name)
        )

        header_layout.insertWidget(2, bulk_edit_button)  # Insert 'E' button after '+' button

        # Add a separator item to avoid pressing the clear button by mistake when wanting to bulk edit.
        separator = QtWidgets.QSpacerItem(12, 0, QtWidgets.QSizePolicy.Fixed, QtWidgets.QSizePolicy.Minimum)
        header_layout.insertItem(3, separator)

        return header

    def _on_press_bulk_edit(self, override_device_name: str):
        '''
        Opens a dialog to allow bulk editing of the list of strings.

        Args:
            override_device_name (str): The device name for which the bulk edit is being performed.
        '''

        current_list = self.get_value(override_device_name)

        # Remove empty elements from the list
        filtered_list = [item for item in current_list if item.strip()]

        dialog = QtWidgets.QDialog()
        dialog.setWindowTitle(f"Bulk Edit {self.nice_name}")
        layout = QtWidgets.QVBoxLayout(dialog)

        text_edit = QtWidgets.QPlainTextEdit()
        text_edit.setPlainText("\n".join(filtered_list))
        layout.addWidget(text_edit)

        button_box = QtWidgets.QDialogButtonBox(QtWidgets.QDialogButtonBox.Ok | QtWidgets.QDialogButtonBox.Cancel)
        button_box.accepted.connect(dialog.accept)
        button_box.rejected.connect(dialog.reject)
        layout.addWidget(button_box)

        result = dialog.exec()
        if result == QtWidgets.QDialog.Accepted:
            new_text = text_edit.toPlainText()
            new_list = [line for line in new_text.splitlines() if line.strip()]
            self._on_widget_value_changed(new_list, override_device_name)


class FilePathListSetting(ListSetting):
    '''
    An array setting where each element is a file path. Each row provides a
    line edit and a Browse button that opens a file dialog filtered by
    file_path_filter. This is the list-style counterpart to FilePathSetting.
    '''

    def __init__(
        self,
        *args,
        file_path_filter: str = '',
        **kwargs
    ):
        super().__init__(*args, **kwargs)
        self.file_path_filter = file_path_filter

    @override
    def create_element(self, override_device_name: str, index: int) -> Tuple[QtWidgets.QWidget, object]:
        row = QtWidgets.QWidget()
        row_layout = QtWidgets.QHBoxLayout(row)
        row_layout.setContentsMargins(0, 0, 0, 0)
        row_layout.setSpacing(2)

        line_edit = QtWidgets.QLineEdit()
        row_layout.addWidget(line_edit)

        browse_btn = QtWidgets.QPushButton('Browse')
        browse_btn.setFocusPolicy(QtCore.Qt.FocusPolicy.NoFocus)
        row_layout.addWidget(browse_btn)

        # Stash references so update_element_value can reach the line edit.
        row.line_edit = line_edit
        row.browse_btn = browse_btn

        line_edit.editingFinished.connect(
            lambda line_edit=line_edit, override_device_name=override_device_name, index=index:
                self._on_editor_widget_changed(line_edit, override_device_name, index)
        )

        browse_btn.clicked.connect(
            lambda _checked=False, line_edit=line_edit, override_device_name=override_device_name, index=index:
                self._on_browse_clicked(line_edit, override_device_name, index)
        )

        return row, ""

    @override
    def update_element_value(self, editor_widget: QtWidgets.QWidget, list_value: list, index: int):
        editor_widget.line_edit.setText(list_value[index])

    def _on_editor_widget_changed(self, line_edit: QtWidgets.QLineEdit, override_device_name: str, index: int):
        current_list = self.get_value(override_device_name)
        if index >= len(current_list):
            return
        copied_value = current_list.copy()
        copied_value[index] = line_edit.text()
        self._on_widget_value_changed(copied_value, override_device_name)

    def _get_start_path(self, current_value: str) -> str:
        if current_value and os.path.exists(current_value):
            return current_value
        if SETTINGS.LAST_BROWSED_PATH and os.path.exists(SETTINGS.LAST_BROWSED_PATH):
            return SETTINGS.LAST_BROWSED_PATH
        return str(pathlib.Path.home())

    def _on_browse_clicked(self, line_edit: QtWidgets.QLineEdit, override_device_name: str, index: int):
        start_path = self._get_start_path(line_edit.text())
        fs_path, _ = QtWidgets.QFileDialog.getOpenFileName(
            parent=line_edit, dir=start_path, filter=self.file_path_filter)

        if not fs_path:
            return

        fs_path = os.path.normpath(fs_path)
        line_edit.setText(fs_path)

        SETTINGS.LAST_BROWSED_PATH = os.path.dirname(fs_path)
        SETTINGS.save()

        self._on_editor_widget_changed(line_edit, override_device_name, index)


class LoggingModel(QtGui.QStandardItemModel):
    '''
    A data model for storing a logging configuration that maps logging
    category names to the desired log verbosity for messages of that
    category.

    A value of None can be mapped to a category indicating that messages
    for that category should be emitted with the default verbosity level.

    The model allows the list of categories to be extended beyond the list
    given at initialization. Categories added in this way can be dynamically
    added or removed by the user, while categories in the initial list are
    fixed and cannot be removed.
    '''

    CATEGORY_COLUMN = 0
    VERBOSITY_COLUMN = 1
    NUM_COLUMNS = 2

    def __init__(
            self,
            categories: list[str],
            verbosity_levels: list[str],
            parent: Optional[QtCore.QObject] = None):
        '''
        Create a new LoggingModel object.

        Args:
            categories      : List of logging category names.
            verbosity_levels: List of possible verbosity level settings for
                              a category.
            parent          : The QObject parent of this object.
        '''
        super().__init__(parent=parent)

        self._categories = categories or []
        self._user_categories = []
        self._verbosity_levels = verbosity_levels or []

        self.setColumnCount(LoggingModel.NUM_COLUMNS)
        self.setHorizontalHeaderLabels(['Category', 'Verbosity Level'])
        self.horizontalHeaderItem(
            LoggingModel.CATEGORY_COLUMN).setToolTip(
                'The name of the category of logging messages')
        self.horizontalHeaderItem(
            LoggingModel.VERBOSITY_COLUMN).setToolTip(
                'The level of verbosity at which to emit messages for the '
                'category')

    @property
    def categories(self) -> list[str]:
        return self._categories

    @property
    def user_categories(self) -> list[str]:
        return self._user_categories

    @property
    def verbosity_levels(self) -> list[str]:
        return self._verbosity_levels

    def is_user_category(self, category: str) -> bool:
        '''
        Returns True if the given category was added after the
        model was initialized.
        '''
        return (
            category in self.user_categories and
            category not in self.categories)

    def add_user_category(self, category: str) -> bool:
        '''
        Adds a row for the category to the model.

        Returns True if the category was added, or False otherwise.
        '''
        if category in self.categories or category in self.user_categories:
            return False

        self._user_categories.append(category)

        root_item = self.invisibleRootItem()

        root_item.appendRow(
            [QtGui.QStandardItem(category), QtGui.QStandardItem(None)])

        return True

    def remove_user_category(self, category: str) -> bool:
        '''
        Removes the category from the model.

        Returns True if the category was removed, or False otherwise.
        '''
        if not self.is_user_category(category):
            return False

        self._user_categories.remove(category)

        category_items = self.findItems(
            category, column=LoggingModel.CATEGORY_COLUMN)
        if not category_items:
            return False

        root_item = self.invisibleRootItem()

        row = category_items[0].index().row()

        root_item.removeRow(row)

        return True

    def category_at(self, index: QtCore.QModelIndex):
        return self.invisibleRootItem().child(index.row(), LoggingModel.CATEGORY_COLUMN).text()

    @property
    def category_verbosities(self) -> collections.OrderedDict:
        '''
        Returns the data currently stored in the model as a dictionary
        mapping each category name to a verbosity level (or None).
        '''
        value = collections.OrderedDict()

        root_item = self.invisibleRootItem()

        for row in range(root_item.rowCount()):
            category_item = root_item.child(
                row, LoggingModel.CATEGORY_COLUMN)
            verbosity_level_item = root_item.child(
                row, LoggingModel.VERBOSITY_COLUMN)

            value[category_item.text()] = verbosity_level_item.text() or None

        return value

    @category_verbosities.setter
    def category_verbosities(
            self,
            value: Optional[dict[str, Optional[str]]]):
        '''
        Sets the data in the model using the given dictionary of category
        names to verbosity levels (or None).
        '''
        value = value or collections.OrderedDict()

        # sort them by name to facilitate finding them
        value = collections.OrderedDict(sorted(value.items(), key=lambda item: str(item[0]).lower()))

        self.beginResetModel()
        count_before = self.rowCount()

        root_item = self.invisibleRootItem()
        for category, verbosity_level in value.items():
            if (category not in self._categories and
                    category not in self._user_categories):
                self._user_categories.append(category)

            root_item.appendRow(
                [QtGui.QStandardItem(category),
                    QtGui.QStandardItem(verbosity_level)])

        # This triggers the rowsRemoved event.
        # Remove rows after so external observers get the right result from category_verbosities.
        self.removeRows(0, count_before)
        self.endResetModel()

    def flags(self, index: QtCore.QModelIndex) -> QtCore.Qt.ItemFlags:
        '''
        Returns the item flags for the item at the given index.
        '''
        if not index.isValid():
            return QtCore.Qt.ItemIsEnabled

        item_flags = (QtCore.Qt.ItemIsEnabled | QtCore.Qt.ItemIsSelectable)

        if index.column() == LoggingModel.VERBOSITY_COLUMN:
            item_flags |= QtCore.Qt.ItemIsEditable

        return item_flags


class LoggingVerbosityItemDelegate(QtWidgets.QStyledItemDelegate):
    '''
    A delegate for items in the verbosity column of the logging view.

    This delegate manages creating a combo box with the available
    verbosity levels.
    '''

    def __init__(
            self,
            verbosity_levels: list[str],
            parent: QtWidgets.QTreeView):
        super().__init__(parent=parent)
        self._verbosity_levels = verbosity_levels or []
        self._parent = parent
        self._selected_categories = []

    def createEditor(
            self, parent: QtWidgets.QWidget,
            option: QtWidgets.QStyleOptionViewItem,
            index: QtCore.QModelIndex) -> sb_widgets.NonScrollableComboBox:
        editor = sb_widgets.NonScrollableComboBox(parent)
        editor.addItems(self._verbosity_levels)

        # Pre-select the level currently specified in the model, if any.
        current_value = index.model().data(index)
        if current_value in self._verbosity_levels:
            current_index = self._verbosity_levels.index(current_value)
            editor.setCurrentIndex(current_index)

        edited_category = self._parent.model().category_at(index)
        editor.onHoverScrollBox.connect(
            lambda edited_category=edited_category:
                self.on_hover_combo_box(edited_category)
        )
        # For multi-selection we want to also be modified even when index has not changed
        editor.activated.connect(
            lambda combo_index, editor=editor:
                self.on_current_index_changed(combo_index, editor)
        )

        return editor

    def setEditorData(
            self, editor: QtWidgets.QWidget, index: QtCore.QModelIndex):
        editor.blockSignals(True)
        editor.setCurrentIndex(editor.currentIndex())
        editor.blockSignals(False)

    def setModelData(
            self, editor: QtWidgets.QWidget, model: QtCore.QAbstractItemModel,
            index: QtCore.QModelIndex):
        model.setData(index, editor.currentText() or None)

    def on_hover_combo_box(self, edited_category: str):
        selection = self._parent.selected_categories()
        # Discard the selection if the user clicked a combo box outside of the selection
        self._selected_categories = selection if edited_category in selection else [edited_category]

    def on_current_index_changed(self, combo_index, editor):
        self._parent.update_category_verbosities(self._selected_categories, editor.itemText(combo_index))


class LoggingSettingVerbosityView(QtWidgets.QTreeView):
    '''
    A tree view that presents the logging configuration represented by the
    given LoggingModel.
    '''

    def __init__(
            self,
            logging_model: LoggingModel,
            parent: Optional[QtWidgets.QWidget] = None):
        super().__init__(parent=parent)

        self.setModel(logging_model)
        self.header().setSectionResizeMode(
            LoggingModel.CATEGORY_COLUMN,
            QtWidgets.QHeaderView.Stretch)
        self.resizeColumnToContents(LoggingModel.VERBOSITY_COLUMN)
        self.header().setSectionResizeMode(
            LoggingModel.VERBOSITY_COLUMN,
            QtWidgets.QHeaderView.Fixed)
        self.header().setStretchLastSection(False)

        self.setItemDelegateForColumn(
            LoggingModel.VERBOSITY_COLUMN,
            LoggingVerbosityItemDelegate(logging_model.verbosity_levels, self))

        self._open_persistent_editors()
        logging_model.modelAboutToBeReset.connect(self._pre_change_model)
        logging_model.modelReset.connect(self._post_change_model)

        self.setSelectionBehavior(QtWidgets.QTreeView.SelectRows)
        self.setSelectionMode(QtWidgets.QTreeView.ExtendedSelection)

    def _pre_change_model(self):
        self.scroll_height = self.verticalScrollBar().value()

    def _post_change_model(self):
        self.verticalScrollBar().setSliderPosition(self.scroll_height)
        self._open_persistent_editors()

    def _open_persistent_editors(self):
        for row in range(self.model().rowCount()):
            verbosity_level_index = self.model().index(row, LoggingModel.VERBOSITY_COLUMN)
            self.openPersistentEditor(verbosity_level_index)

    def selected_categories(self) -> list[str]:
        model = self.model()
        selected_categories = []
        for selection_range in self.selectionModel().selection():
            rows = range(selection_range.top(), selection_range.bottom() + 1)
            for row in rows:
                category_index = model.index(row, 0)
                category = model.data(category_index, QtCore.Qt.DisplayRole)
                selected_categories.append(category)

        return selected_categories

    def update_category_verbosities(self, categories: list[str], new_verbosity: str):
        model = self.model()
        category_verbosities = model.category_verbosities
        for category in categories:
            category_verbosities[category] = new_verbosity

        model.category_verbosities = category_verbosities


class LoggingSetting(Setting):
    '''
    A UI-displayable Setting for storing and modifying a set of logging
    categories and the verbosity level of each category.

    An initial set of categories can be provided when creating the Setting,
    but adding additional user-defined categories is supported as well.
    '''

    # Extracted from ParseLogVerbosityFromString() in
    # Engine\Source\Runtime\Core\Private\Logging\LogVerbosity.cpp
    # None indicates the "default" verbosity level should be used with
    # no override applied.
    DEFAULT_VERBOSITY_LEVELS = [
        None, 'VeryVerbose', 'Verbose', 'Log', 'Display',
        'Warning', 'Error', 'Fatal', 'NoLogging']

    def _filter_value(
            self,
            value: Optional[dict[str, Optional[str]]]
    ) -> collections.OrderedDict:
        '''
        Filter function to modify the incoming value before updating or
        overriding the setting.

        This ensures that LoggingSetting values are always provided using
        a dictionary (regular Python dict or OrderedDict) or None. An exception
        is raised otherwise.

        The resulting dictionary will include a key/value pair for each
        category in the LoggingSetting. Category names not present in the
        input dictionary will have a value of None in the output dictionary.
        '''
        if value is None:
            value = collections.OrderedDict()
        else:
            try:
                value = collections.OrderedDict(value)
            except Exception as e:
                raise ValueError(
                    'Invalid LoggingSetting value. Values must be '
                    f'either dictionary-typed or None: {e}')

        for category in self._categories:
            if category not in value:
                value[category] = None

        return value

    def __init__(
        self,
        *args,
        categories: Optional[list[str]] = None,
        verbosity_levels: Optional[list[str]] = None,
        **kwargs
    ):
        '''
        Create a new LoggingSetting object.

        Args:
            value           : dict[str, Optional[str]]
            categories      : The initial list of logging categories.
            verbosity_levels: The possible settings for verbosity level of
                              each category.
        '''

        # Set the categories before calling the base class init since they
        # will be used when filtering the value.
        self._categories = categories or []

        super().__init__(*args, **kwargs)

        self._verbosity_levels = (
            verbosity_levels or self.DEFAULT_VERBOSITY_LEVELS)

    def get_command_line_arg(
            self, override_device_name: Optional[str] = None
    ) -> str:
        '''
        Generate the command line argument for specifying the logging
        configuration based on the value currently stored in the Setting.

        Only categories that have a verbosity level specified are included in
        the result. If no categories have a verbosity level specified, an
        empty string is returned.
        '''
        value = self.get_value(override_device_name)

        logging_strings = [
            f'{category} {level}' for category, level in value.items()
            if level]
        if not logging_strings:
            return ''

        return f'-LogCmds=\"{", ".join(logging_strings)}\"'

    def _create_widgets(
            self, override_device_name: Optional[str] = None
    ) -> QtWidgets.QVBoxLayout:
        model = LoggingModel(
            categories=self._categories,
            verbosity_levels=self._verbosity_levels)
        model.category_verbosities = self._value
        view = LoggingSettingVerbosityView(logging_model=model)
        view.setMinimumHeight(150)

        self.set_widget(
            widget=view, override_device_name=override_device_name)

        edit_layout = QtWidgets.QVBoxLayout()
        edit_layout.addWidget(view)

        add_category_button = QtWidgets.QPushButton('Add Category')
        add_category_button.setFocusPolicy(QtCore.Qt.FocusPolicy.NoFocus)
        edit_layout.addWidget(add_category_button)

        def on_add_category_button_clicked():
            category, ok = QtWidgets.QInputDialog().getText(
                add_category_button, "Add Category",
                "Category name:", QtWidgets.QLineEdit.Normal)
            if not ok:
                return

            category = category.strip()
            if not category:
                return

            if model.add_user_category(category):
                verbosity_level_index = model.index(
                    model.rowCount() - 1, LoggingModel.VERBOSITY_COLUMN)
                view.openPersistentEditor(verbosity_level_index)

        add_category_button.clicked.connect(on_add_category_button_clicked)

        remove_category_button = QtWidgets.QPushButton('Remove Category')
        remove_category_button.setFocusPolicy(QtCore.Qt.FocusPolicy.NoFocus)
        edit_layout.addWidget(remove_category_button)

        def on_remove_category_button_clicked():
            category_indices = view.selectionModel().selectedRows(
                LoggingModel.CATEGORY_COLUMN)
            if not category_indices:
                return
            
            category_list_str = ''
            category_list = []
            for category_index in category_indices:
                category = model.itemFromIndex(category_index).text()
                category_list.append(category)
                category_list_str += f'{category}\n'
            reply = QtWidgets.QMessageBox.question(
                remove_category_button, 'Confirm Remove Category',
                ('Are you sure you would like to remove the following categories: '
                 f'{category_list_str}'),
                QtWidgets.QMessageBox.Yes, QtWidgets.QMessageBox.No)

            if reply == QtWidgets.QMessageBox.Yes:
                view.selectionModel().clear()
                for category in category_list:
                    model.remove_user_category(category)

        remove_category_button.clicked.connect(
            on_remove_category_button_clicked)

        # The remove button is disabled initially until a user category is
        # selected.
        REMOVE_BUTTON_DISABLED_TOOLTIP = (
            'Default categories for the device cannot be removed')
        remove_category_button.setEnabled(False)
        remove_category_button.setToolTip(REMOVE_BUTTON_DISABLED_TOOLTIP)

        # Enable the remove button when a user category is selected, and
        # disable it otherwise.
        def on_view_selectionChanged(selected, deselected):
            remove_category_button.setEnabled(False)
            remove_category_button.setToolTip(REMOVE_BUTTON_DISABLED_TOOLTIP)

            category_indices = view.selectionModel().selectedRows(
                LoggingModel.CATEGORY_COLUMN)
            if not category_indices:
                return

            all_user_category = True
            for category_index in category_indices:
                category = model.itemFromIndex(category_index).text()
                all_user_category &= model.is_user_category(category)
            if all_user_category:
                remove_category_button.setEnabled(True)
                remove_category_button.setToolTip('')

        view.selectionModel().selectionChanged.connect(
            on_view_selectionChanged)

        def on_logging_model_modified(override_device_name=None):
            category_verbosities = model.category_verbosities
            self._on_widget_value_changed(
                category_verbosities,
                override_device_name=override_device_name)

        model.dataChanged.connect(
            lambda top_left_index, bottom_right_index, roles,
            override_device_name=override_device_name:
                on_logging_model_modified(
                    override_device_name=override_device_name))

        model.rowsInserted.connect(
            lambda parent, first, last,
            override_device_name=override_device_name:
                on_logging_model_modified(
                    override_device_name=override_device_name))

        model.rowsRemoved.connect(
            lambda parent, first, last,
            override_device_name=override_device_name:
                on_logging_model_modified(
                    override_device_name=override_device_name))

        return edit_layout

    def _on_setting_changed(
            self, new_value,
            override_device_name: Optional[str] = None):

        widget = self.get_widget(override_device_name=override_device_name)
        if not widget:
            return

        old_value = widget.model().category_verbosities
        if new_value != old_value:
            widget.model().category_verbosities = new_value


class AddressSetting(OptionSetting):

    def __init__(self, *args, **kwargs):

        super().__init__(
            *args,
            possible_values=list(self.generate_possible_addresses()),
            **kwargs
        )

    def _create_widgets(
        self, override_device_name: Optional[str] = None
    ) -> NonScrollableComboBox:
        combo: NonScrollableComboBox = super()._create_widgets(
            override_device_name, widget_class=sb_widgets.AddressComboBox)

        combo.setInsertPolicy(QtWidgets.QComboBox.InsertPolicy.NoInsert)

        cur_value = self.get_value(override_device_name)
        if combo.findText(cur_value, QtCore.Qt.MatchFlag.MatchExactly) == -1:
            combo.addItem(str(cur_value), cur_value)
            combo.setCurrentIndex(combo.findText(cur_value))

        combo.lineEdit().editingFinished.connect(
            lambda: self._validate_and_commit_address(combo,
                                                      override_device_name)
        )

        return combo

    def _validate_and_commit_address(
            self, combo: NonScrollableComboBox, override_device_name: str):
        address_str = combo.lineEdit().text()
        self._on_widget_value_changed(address_str, override_device_name=override_device_name)

    def generate_possible_addresses(self):
        return set[str]()


def is_valid_unicast_address(address):
    # 169.254.x.x addresses can be generated by the OS when a failure to negotiate with DHCP.
    if address.startswith("169.254"):
        LOGGER.warning(f'Detected link assigned address {address}.  This address is not reachable outside the local link adapter. Please check your settings.')

    if address == '0.0.0.0':
        LOGGER.error('The IF_ANY address, 0.0.0.0, is not a valid switchboard unicast address.')
        return False

    split_addr = address.split('.')
    if len(split_addr) != 4:
        LOGGER.error(f'Unicast address ({address}) does not contain four parts.')
        return False

    for part in reversed(split_addr):
        if not part.isdigit() or part == '255':
            LOGGER.error(f'Broadcast-like addresses are not allowed.')
            return False

    classA_part = int(split_addr[0])
    if classA_part >= 224 and classA_part <= 239:
        LOGGER.error(f'Unicast address {address} is a multicast address and not valid.')
        return False

    return True


def is_local_classC_ip(address):
    if address.startswith("192."):
        return True
    return False


def is_local_classA_ip(address):
    if address.startswith("10."):
        return True
    return False


class LocalAddressSetting(AddressSetting):
    def check_and_fix_bad_ip(self):

        orig_value = self.get_value()

        if is_valid_unicast_address(orig_value):
            return

        addresses = self.generate_possible_addresses()

        def iterate_over_addresses():
            classC = filter(lambda x: is_local_classC_ip(x), addresses)
            classA = filter(lambda x: is_local_classA_ip(x), addresses)
            other  = filter(lambda x: not is_local_classC_ip(x) and not is_local_classA_ip(x), addresses)
            for ip in classC:
                yield ip
            for ip in classA:
                yield ip
            for ip in other:
                yield ip

        # Pick class C first, then class A followed by other detected IP.  Finally fallback to localhost
        new_value = "127.0.0.1"
        for addr in iterate_over_addresses():
            if is_valid_unicast_address(addr):
                new_value = addr
                break

        LOGGER.error(f'{orig_value} is not a valid IP address changing default IP to {new_value}')
        self.update_value(new_value)

    def generate_possible_addresses(self):
        addresses = set[str]()
        for address in socket.getaddrinfo(socket.gethostname(), None, socket.AF_INET):
            addresses.add(str(address[4][0]))
        addresses.add('127.0.0.1')
        addresses.add('localhost')
        addresses.add(socket.gethostname())
        return addresses


class ConfigPathError(Exception):
    '''
    Base exception type for config path related errors.
    '''
    pass


class ConfigPathEmptyError(ConfigPathError):
    '''
    Exception type raised when an empty or all whitespace string is used as a
    config path.
    '''
    pass


class ConfigPathIsUserSettingsError(ConfigPathError):
    '''
    Exception type raised when the user settings file path is used as a config
    path.
    '''
    pass


class ConfigPathValidator(QtGui.QValidator):
    '''
    Validator to determine whether the input is an acceptable config file
    path.

    If the input is not acceptable, the state is returned as Intermediate
    rather than Invalid so as not to interfere with the user typing in the
    text field.
    '''

    def validate(self, input, pos):
        try:
            CONFIG_MGR.get_absolute_config_path(input)
        except Exception:
            return QtGui.QValidator.Intermediate

        return QtGui.QValidator.Acceptable


class EngineSyncMethod(Enum):
    Use_Existing = "Use Existing (do not sync/build)"
    Build_Engine = "Build Engine"

    if ENABLE_UGS_SUPPORT:
        Sync_PCBs = "Sync Precompiled Binaries (requires UnrealGameSync)"
        Sync_From_UGS = "Sync engine and project together using UnrealGameSync"


class Config(object):

    DEFAULT_CONFIG_PATH = CONFIG_MGR.get_relative_config_path(f'Default')

    saving_allowed = True
    saving_allowed_fifo = []

    def push_saving_allowed(self, value):
        ''' Sets a new state of saving allowed, but pushes current to the stack
        '''
        self.saving_allowed_fifo.append(self.saving_allowed)
        self.saving_allowed = value

    def pop_saving_allowed(self):
        ''' Restores saving_allowed flag from the stack
        '''
        self.saving_allowed = self.saving_allowed_fifo.pop()

    def __init__(self):
        self.file_path: pathlib.Path | None = None

    def init_with_file_path(self, file_path: Union[str, pathlib.Path]):
        if file_path:
            try:
                self.file_path = CONFIG_MGR.get_absolute_config_path(file_path)

                # Read the json config file
                with open(self.file_path, encoding='utf-8') as f:
                    LOGGER.debug(f'Loading Config {self.file_path}')
                    data = json.load(f)

            except (ConfigPathError, FileNotFoundError) as e:
                LOGGER.error(f'Config: {e}')
                self.file_path = None
                data = {}
            except ValueError:
                # The original file will be overwritten
                self._backup_corrupted_config(str(self.file_path))
                data = {}
        else:
            self.file_path = None
            data = {}

        self.init_switchboard_settings(data)
        self.init_sblhelper_settings(data)
        self.init_project_settings(data)
        self.init_plugin_tracking()
        self.init_unreal_insights(data)
        self.init_muserver(data)

        # Automatically save whenever a project setting is changed or
        # overridden by a device.
        # TODO: switchboard_settings
        all_settings = [setting for _, setting in self.basic_project_settings.items()] \
            + [setting for _, setting in self.osc_settings.items()] \
            + [setting for _, setting in self.source_control_settings.items()] \
            + [setting for _, setting in self.unreal_insight_settings.items()] \
            + [setting for _, setting in self.mu_settings.items()]

        for setting in all_settings:
            setting.signal_setting_changed.connect(lambda: self.save())
            setting.signal_setting_overridden.connect(
                self.on_device_override_changed)

        # Directory Paths
        self.SWITCHBOARD_DIR = os.path.abspath(
            os.path.join(os.path.dirname(os.path.abspath(__file__)), '../'))

        self.CURRENT_LEVEL = data.get('current_level', DEFAULT_MAP_TEXT)

        self.init_devices(data)

    def _backup_corrupted_config(self, original_file_path: str):
        directory_name = os.path.dirname(original_file_path)
        original_file_name = os.path.basename(original_file_path)
        new_file_name = original_file_name.replace(".", "_corrupted_backup.")

        LOGGER.error(f'{original_file_name} has invalid JSON format. Creating default...')
        answer = QtWidgets.QMessageBox.question(
            None,
            'Invalid project settings',
            f'Config file { original_file_name } is invalid JSON and will be replaced by a new default JSON config.'
            f'\n\nDo you want to save a backup named { new_file_name }?',
            QtWidgets.QMessageBox.Yes | QtWidgets.QMessageBox.No
        )
        if answer == QtWidgets.QMessageBox.Yes:
            new_file_path = os.path.join(directory_name, new_file_name)
            if os.path.exists(new_file_path):
                os.remove(new_file_path)
            shutil.copy(original_file_path, new_file_path)

    def init_devices(self, data={}):
        ''' '''

        self._device_data_from_config = {}
        self._plugin_data_from_config = {}
        self._device_settings = {}
        self._plugin_settings = {}

        # Convert devices data from dict to list so they can be directly fed
        # into the kwargs.
        for device_type, devices in data.get('devices', {}).items():
            for device_name, data in devices.items():
                if device_name == 'settings':
                    self._plugin_data_from_config[device_type] = data
                else:
                    # Migrate ip_address -> address
                    if 'ip_address' in data:
                        address = data['ip_address']
                        del data['ip_address']
                    else:
                        address = data['address']

                    device_data = {
                        'name': device_name,
                        'address': address
                    }
                    device_data['kwargs'] = {
                        k: v for (k, v) in data.items() if k != 'address'}
                    self._device_data_from_config.setdefault(
                        device_type, []).append(device_data)

    def init_plugin_tracking(self):
        ''' Initializes the plugin manager '''

        engine_dir = self.ENGINE_DIR.get_value()
        uproj_path = self.UPROJECT_PATH.get_value()

        self.ue_plugin_mgr = UnrealPluginManager()
        self.ue_plugin_mgr.set_engine_dir(Path(engine_dir) if engine_dir else None)
        self.ue_plugin_mgr.set_uproject_path(Path(uproj_path) if uproj_path else None)

        self.ENGINE_DIR.signal_setting_changed.connect(
            lambda _, new: self.ue_plugin_mgr.set_engine_dir(Path(new)))

        self.UPROJECT_PATH.signal_setting_changed.connect(
            lambda _, new: self.ue_plugin_mgr.set_uproject_path(Path(new)))

    def init_new_config(self, file_path: Union[str, pathlib.Path], uproject, engine_dir, p4_settings):
        ''' Initialize new configuration '''

        # Assign self.file_path before the basic_project_settings line because it requires it to be updated.
        self.file_path = CONFIG_MGR.get_absolute_config_path(file_path)

        basic_project_settings = {
                "project_name": self.file_path.stem,
                "uproject": uproject,
                "engine_dir": engine_dir,
            }

        self.init_switchboard_settings()
        self.init_sblhelper_settings()
        self.init_project_settings(basic_project_settings | p4_settings)
        self.init_plugin_tracking()
        self.init_unreal_insights()
        self.init_muserver()

        self.CURRENT_LEVEL = DEFAULT_MAP_TEXT

        self.init_plugin_tracking()
        self.init_devices()

        LOGGER.info(f"Creating new config saved in {self.file_path}")
        self.save()

        SETTINGS.CONFIG = self.file_path
        SETTINGS.save()

    def init_switchboard_settings(self, data={}):
        self.switchboard_settings = {
            "listener_exe": StringSetting(
                attr_name="listener_exe",
                nice_name="Listener Executable Name",
                value=data.get('listener_exe', 'SwitchboardListener')
            )
        }

        self.LISTENER_EXE = self.switchboard_settings["listener_exe"]

    def init_sblhelper_settings(self, data={}):
        self.sblhelper_settings = {
            "sblhelper_exe": StringSetting(
                attr_name="sblhelper_exe",
                nice_name="Gpu Clocker Executable Name",
                value=data.get('sblhelper_exe', 'SwitchboardListenerHelper'),
                tool_tip="Name of the executable that SwitchboardListener can communicate with to lock Gpu clocks.",
                show_ui=True if sys.platform in ('win32', 'linux') else False  # Gpu Clocker is available in select platforms
            )
        }

        self.SBLHELPER_EXE = self.sblhelper_settings["sblhelper_exe"]

    def init_project_settings(self, data={}):
        self.basic_project_settings = {
            "project_name": StringSetting(
                "project_name",
                "Project Name",
                data.get('project_name', 'Default'),
                category="General Settings",
            ),
            "uproject": FilePathSetting(
                "uproject", "uProject Path",
                data.get('uproject', ''),
                tool_tip="Path to uProject",
                category="General Settings",
            ),
            "engine_dir": DirectoryPathSetting(
                "engine_dir",
                "Engine Directory",
                data.get('engine_dir', ''),
                tool_tip="Path to UE 'Engine' directory",
                category="General Settings",
            ),
            'engine_sync_method': OptionSetting(
                "engine_sync_method",
                "Engine Sync Method",
                EngineSyncMethod.Use_Existing.value,
                possible_values=[p.value for p in EngineSyncMethod],
                category="Source Control Settings",
            ),
            "maps_path": StringSetting(
                "maps_path",
                "Map Path",
                data.get('maps_path', ''),
                placeholder_text="Maps",
                tool_tip="Relative path from Content folder that contains maps to launch into.",
                category="General Settings",
            ),
            "maps_filter": StringSetting(
                "maps_filter",
                "Map Filter",
                data.get('maps_filter', '*.umap'),
                placeholder_text="*.umap",
                tool_tip="Walk every file in the Map Path and run a fnmatch to filter the file names",
                category="General Settings",
            ),
            'maps_plugin_filters': StringListSetting(
                "maps_plugin_filters",
                "Map Plugin Filters",
                data.get('maps_plugin_filters', []),
                tool_tip=(
                    "DEPRECATED: Use 'Content Plugin Filters' instead' - "
                    "Plugins whose name matches any of these filters will "
                    "also be searched for maps."),
                show_ui=False,
                migrate_data=migrate_comma_separated_string_to_list,
                category="General Settings",
            ),
            'content_plugin_filters': StringListSetting(
                "content_plugin_filters",
                "Content Plugin Filters",
                # "Upgrade" from the deprecated "maps_plugin_filters".
                data.get(
                    'content_plugin_filters',
                    data.get('maps_plugin_filters', [])),
                tool_tip=(
                    "Plugins that match any of these filters will also be "
                    "searched when populating fields in Switchboard (e.g. "
                    "levels, nDisplay configs, etc.).\n"
                    "\n"
                    "Each value can be either a plugin name, or a relative or "
                    "absolute path to a plugin directory. Relative paths "
                    "should be relative to the directory containing the "
                    ".uproject file."),
                migrate_data=migrate_comma_separated_string_to_list,
                category="General Settings",
            ),
        }

        # Done here outside of the setting's initializer so that values different from the default are flagged in the UI (with a 'reset' option) 
        if 'engine_sync_method' in data:
            self.basic_project_settings["engine_sync_method"].update_value(data.get('engine_sync_method'))
        # To support backwards compatibility, we check the old 'build_engine' option (from before we had a 'PCB' option)
        elif data.get('build_engine', False):
            self.basic_project_settings["engine_sync_method"].update_value(EngineSyncMethod.Build_Engine.value)

        self.UPROJECT_PATH = self.basic_project_settings["uproject"]

        # Take note if this project had a cache when opened, as devices may want to trigger a cache
        if SBCache().query_project(self.UPROJECT_PATH.get_value()):
            self.PROJECTWASINCACHE = True
        else:
            self.PROJECTWASINCACHE = False

        self.PROJECT_NAME = self.basic_project_settings["project_name"]
        self.ENGINE_DIR = self.basic_project_settings["engine_dir"]
        self.ENGINE_SYNC_METHOD = self.basic_project_settings["engine_sync_method"]
        self.MAPS_PATH = self.basic_project_settings["maps_path"]
        self.MAPS_FILTER = self.basic_project_settings["maps_filter"]
        self.CONTENT_PLUGIN_FILTERS = (
            self.basic_project_settings["content_plugin_filters"])

        self.osc_settings = {
            'osc_server_port': IntSetting(
                attr_name='osc_server_port',
                nice_name='Internal OSC Server Port',
                value=data.get('osc_server_port', 6000),
                tool_tip=(
                    'The port on which Switchboard listens for incoming OSC '
                    'connections from other devices/applications.')
            )
        }

        self.OSC_SERVER_PORT = self.osc_settings["osc_server_port"]

        self.source_control_settings = {
            "p4_enabled": BoolSetting(
                "p4_enabled",
                "Perforce Enabled",
                data.get("p4_enabled", False),
                tool_tip="Toggle Perforce support for the entire application",
                category="Source Control Settings",
            ),
            "source_control_workspace": StringSetting(
                "source_control_workspace", "Workspace",
                data.get("source_control_workspace", ""),
                tool_tip="SourceControl Workspace/Branch",
                category="Source Control Settings",
            ),
            "project_workspace": StringSetting(
                "project_workspace", "Project Workspace Override",
                data.get("project_workspace", ""),
                tool_tip="Only use if different to the Engine workspace",
                category="Source Control Settings",
            ),
            "p4_sync_path": PerforcePathSetting(
                "p4_sync_path",
                "Perforce Project Path",
                data.get("p4_sync_path", ''),
                placeholder_text="//UE/Project",
                category="Source Control Settings",
            ),
            "p4_engine_path": PerforcePathSetting(
                "p4_engine_path",
                "Perforce Engine Path",
                data.get("p4_engine_path", ''),
                placeholder_text="//UE/Project/Engine",
                category="Source Control Settings",
            )
        }

        self.P4_ENABLED = self.source_control_settings["p4_enabled"]
        self.SOURCE_CONTROL_WORKSPACE = self.source_control_settings["source_control_workspace"]
        self.PROJECT_WORKSPACE = self.source_control_settings["project_workspace"]
        self.P4_PROJECT_PATH: PerforcePathSetting = self.source_control_settings["p4_sync_path"]
        self.P4_ENGINE_PATH: PerforcePathSetting = self.source_control_settings["p4_engine_path"]

        self.P4_PROJECT_PATH.truncate_files_with_extensions.append('.uproject')

    def init_unreal_insights(self, data={}):
        self.unreal_insight_settings = {
            "tracing_enabled": BoolSetting(
                "tracing_enabled",
                "Unreal Insights Tracing State",
                data.get("tracing_enabled", False),
            ),
            "tracing_args": StringSetting(
                "tracing_args",
                "Unreal Insights Tracing Args",
                data.get('tracing_args', 'default,concert,messaging,tasks')
            ),
            "tracing_stat_events": BoolSetting(
                "tracing_stat_events",
                "Unreal Insights Tracing with Stat Events",
                data.get('tracing_stat_events', True)
            )
        }

        self.INSIGHTS_TRACE_ENABLE = self.unreal_insight_settings["tracing_enabled"]
        self.INSIGHTS_TRACE_ARGS = self.unreal_insight_settings["tracing_args"]
        self.INSIGHTS_STAT_EVENTS = self.unreal_insight_settings["tracing_stat_events"]

    def default_mu_server_name(self):
        ''' Returns default server name based on current settings '''
        return f'{self.PROJECT_NAME.get_value()}_MU_Server'

    def default_mu_session_name(self):
        ''' Returns default session name based on current settings '''
        project_name = self.PROJECT_NAME.get_value()
        date_str = datetime.datetime.now().strftime('%y%m%d')
        return f'MU_{project_name}_{date_str}_1'

    def init_muserver(self, data={}):
        self.mu_settings = {
            "muserver_server_name": StringSetting(
                "muserver_server_name",
                "Server name",
                data.get('muserver_server_name', self.default_mu_server_name()),
                tool_tip="The name that will be given to the server"
            ),
            "muserver_command_line_arguments": StringSetting(
                "muserver_command_line_arguments",
                "Command Line Args",
                data.get('muserver_command_line_arguments', ''),
                tool_tip="Additional command line arguments to pass to multiuser"
            ),
            "muserver_endpoint": StringSetting(
                "muserver_endpoint",
                "Unicast Endpoint",
                data.get('muserver_endpoint', ':9030')
            ),
            "udpmessaging_multicast_endpoint": StringSetting(
                attr_name='udpmessaging_multicast_endpoint',
                nice_name='Multicast Endpoint',
                value=data.get('muserver_multicast_endpoint', '230.0.0.1:6666'),
                tool_tip=(
                    'Multicast group and port (-UDPMESSAGING_TRANSPORT_MULTICAST) '
                    'in the {address}:{port} endpoint format. The multicast group address '
                    'must be in the range 224.0.0.0 to 239.255.255.255.'),
            ),
            "multiuser_exe": StringSetting(
                "multiuser_exe",
                "Multiuser Executable Name",
                data.get('multiuser_exe', 'UnrealMultiUserServer')
            ),
            "multiuserslate_exe": StringSetting(
                "multiuserslate_exe",
                "Multiuser Slate Executable Name",
                data.get('multiuserslate_exe', 'UnrealMultiUserSlateServer')
            ),
            "muserver_archive_dir": DirectoryPathSetting(
                "muserver_archive_dir",
                "Directory for Saved Archives",
                data.get('muserver_archive_dir', '')
            ),
            "muserver_working_dir": DirectoryPathSetting(
                "muserver_working_dir",
                "Directory for Live Sessions",
                data.get('muserver_working_dir', '')
            ),
            "muserver_filesharing_srv": DirectoryPathSetting(
                "muserver_filesharing_srv",
                "Directory or URI for File Sharing Service",
                data.get('muserver_filesharing_srv', '')
            ),
            "muserver_filesharing_enabled": BoolSetting(
                "muserver_filesharing_enabled",
                "Use File Sharing Service",
                data.get('muserver_filesharing_enabled', False)
            ),
            "muserver_auto_launch": BoolSetting(
                "muserver_auto_launch",
                "Auto Launch",
                data.get('muserver_auto_launch', True)
            ),
            "muserver_slate_mode": BoolSetting(
                "muserver_slate_mode",
                "Launch Multi-user server in UI mode",
                data.get('muserver_slate_mode', True)
            ),
            "muserver_clean_history": BoolSetting(
                "muserver_clean_history",
                "Clean History",
                data.get('muserver_clean_history', False)
            ),
            "muserver_auto_build": BoolSetting(
                "muserver_auto_build",
                "Auto Build",
                data.get('muserver_auto_build', True)
            ),
            "muserver_auto_endpoint": BoolSetting(
                "muserver_auto_endpoint",
                "Auto Endpoint",
                data.get('muserver_auto_endpoint', True)
            ),
            "muserver_auto_join": BoolSetting(
                "muserver_auto_join",
                "Unreal Multi-user Server Auto-join",
                data.get('muserver_auto_join', True)
            ),
            "muserver_session_name": StringSetting(
                "muserver_session_name",
                "Session Name",
                data.get('muserver_session_name', self.default_mu_session_name()),
                tool_tip="The name of the multi-user session"
            )
        }

        self.MUSERVER_SERVER_NAME = self.mu_settings["muserver_server_name"]
        self.MUSERVER_COMMAND_LINE_ARGUMENTS = self.mu_settings["muserver_command_line_arguments"]
        self.MUSERVER_ENDPOINT = self.mu_settings["muserver_endpoint"]
        self.MUSERVER_MULTICAST_ENDPOINT = self.mu_settings["udpmessaging_multicast_endpoint"]
        self.MULTIUSER_SERVER_EXE = self.mu_settings["multiuser_exe"]
        self.MULTIUSER_SLATE_SERVER_EXE = self.mu_settings["multiuserslate_exe"]
        self.MUSERVER_AUTO_LAUNCH = self.mu_settings["muserver_auto_launch"]
        self.MUSERVER_SLATE_MODE = self.mu_settings["muserver_slate_mode"]
        self.MUSERVER_CLEAN_HISTORY = self.mu_settings["muserver_clean_history"]
        self.MUSERVER_AUTO_BUILD = self.mu_settings["muserver_auto_build"]
        self.MUSERVER_AUTO_ENDPOINT = self.mu_settings["muserver_auto_endpoint"]
        self.MUSERVER_AUTO_JOIN = self.mu_settings["muserver_auto_join"]
        self.MUSERVER_WORKING_DIR = self.mu_settings["muserver_working_dir"]
        self.MUSERVER_ARCHIVE_DIR = self.mu_settings["muserver_archive_dir"]
        self.MUSERVER_SESSION_NAME = self.mu_settings["muserver_session_name"]
        self.MUSERVER_FILESHARING_SRV = self.mu_settings["muserver_filesharing_srv"]
        self.MUSERVER_FILESHARING_ENABLED = self.mu_settings["muserver_filesharing_enabled"]
        
    def save_unreal_insights(self, data):
        data['tracing_enabled'] = self.INSIGHTS_TRACE_ENABLE.get_value()
        data['tracing_args'] = self.INSIGHTS_TRACE_ARGS.get_value()
        data['tracing_stat_events'] = self.INSIGHTS_STAT_EVENTS.get_value()

    def save_muserver(self, data):
        data["muserver_command_line_arguments"] = self.MUSERVER_COMMAND_LINE_ARGUMENTS.get_value()
        data["muserver_server_name"] = self.MUSERVER_SERVER_NAME.get_value()
        data["muserver_endpoint"] = self.MUSERVER_ENDPOINT.get_value()
        data["multiuser_exe"] = self.MULTIUSER_SERVER_EXE.get_value()
        data["multiuserslate_exe"] = self.MULTIUSER_SLATE_SERVER_EXE.get_value()
        data["muserver_auto_launch"] = self.MUSERVER_AUTO_LAUNCH.get_value()
        data["muserver_slate_mode"] = self.MUSERVER_SLATE_MODE.get_value()
        data["muserver_clean_history"] = self.MUSERVER_CLEAN_HISTORY.get_value()
        data["muserver_auto_build"] = self.MUSERVER_AUTO_BUILD.get_value()
        data["muserver_auto_endpoint"] = self.MUSERVER_AUTO_ENDPOINT.get_value()
        data["muserver_multicast_endpoint"] = self.MUSERVER_MULTICAST_ENDPOINT.get_value()
        data["muserver_auto_join"] = self.MUSERVER_AUTO_JOIN.get_value()
        data["muserver_archive_dir"] = self.MUSERVER_ARCHIVE_DIR.get_value()
        data["muserver_working_dir"] = self.MUSERVER_WORKING_DIR.get_value()
        data["muserver_session_name"] = self.MUSERVER_SESSION_NAME.get_value()
        data["muserver_filesharing_srv"] = self.MUSERVER_FILESHARING_SRV.get_value() 
        data["muserver_filesharing_enabled"] = self.MUSERVER_FILESHARING_ENABLED.get_value() 

        
    def load_plugin_settings(self, device_type, settings):
        ''' Updates plugin settings values with those read from the config file.
        '''

        loaded_settings = self._plugin_data_from_config.get(device_type, [])

        if loaded_settings:
            for setting in settings:
                if setting.attr_name in loaded_settings:
                    value = loaded_settings[setting.attr_name]
                    setting.update_value_from_config(value)
            del self._plugin_data_from_config[device_type]

    def register_plugin_settings(self, device_type, settings):

        self._plugin_settings[device_type] = settings

        for setting in settings:
            setting.signal_setting_changed.connect(lambda: self.save())
            setting.signal_setting_overridden.connect(
                self.on_device_override_changed)

    def register_device_settings(
            self, device_type, device_name, settings, overrides):
        self._device_settings[(device_type, device_name)] = (
            settings, overrides)

        for setting in settings:
            setting.signal_setting_changed.connect(lambda: self.save())

    def on_device_override_changed(self, device_name, old_value, override):
        # Only do a save operation when the device is known (has called
        # register_device_settings) otherwise it is still loading and we want
        # to avoid saving during device loading to avoid errors in the cfg
        # file.
        known_devices = [name for (_, name) in self._device_settings.keys()]
        if device_name in known_devices:
            self.save()

    def replace(self, new_config_path: Union[str, pathlib.Path]):
        """
        Move the file.

        If a file already exists at the new path, it will be overwritten.
        """
        new_config_path = CONFIG_MGR.get_absolute_config_path(new_config_path)

        if self.file_path:
            new_config_path.parent.mkdir(parents=True, exist_ok=True)
            shutil.move(self.file_path, new_config_path)

        self.file_path = new_config_path
        self.save()

    def save_as(self, new_config_path: Union[str, pathlib.Path]):
        """
        Copy the file.

        If a file already exists at the new path, it will be overwritten.
        """
        new_config_path = CONFIG_MGR.get_absolute_config_path(new_config_path)

        new_project_name = new_config_path.stem

        if self.file_path:
            new_config_path.parent.mkdir(parents=True, exist_ok=True)
            shutil.copyfile(src=self.file_path, dst=new_config_path)

            # Don't change project name if the user customized it
            if self.PROJECT_NAME.get_value() != self.file_path.stem:
                new_project_name = self.file_path.stem

        self.file_path = new_config_path

        self.PROJECT_NAME.update_value(new_project_name)

        self.save()

    def save(self):
        if not self.file_path or not self.saving_allowed:
            return

        data = {}

        # General settings
        data['project_name'] = self.PROJECT_NAME.get_value()
        data['uproject'] = self.UPROJECT_PATH.get_value()
        data['engine_dir'] = self.ENGINE_DIR.get_value()
        data['engine_sync_method'] = self.ENGINE_SYNC_METHOD.get_value()
        data["maps_path"] = self.MAPS_PATH.get_value()
        data["maps_filter"] = self.MAPS_FILTER.get_value()
        data["content_plugin_filters"] = (
            self.CONTENT_PLUGIN_FILTERS.get_value())
        data["listener_exe"] = self.LISTENER_EXE.get_value()
        data["sblhelper_exe"] = self.SBLHELPER_EXE.get_value()

        self.save_unreal_insights(data)

        # OSC settings
        data["osc_server_port"] = self.OSC_SERVER_PORT.get_value()

        # Source Control Settings
        data["p4_enabled"] = self.P4_ENABLED.get_value()
        data["p4_sync_path"] = self.P4_PROJECT_PATH.get_value()
        data["p4_engine_path"] = self.P4_ENGINE_PATH.get_value()
        data["source_control_workspace"] = (self.SOURCE_CONTROL_WORKSPACE.get_value())
        data["project_workspace"] = (self.PROJECT_WORKSPACE.get_value())

        self.save_muserver(data)

        # Level
        data["current_level"] = self.CURRENT_LEVEL

        # Devices
        data["devices"] = {}

        # Plugin settings
        for device_type, plugin_settings in self._plugin_settings.items():

            if not plugin_settings:
                continue

            settings = {}

            for setting in plugin_settings:
                settings[setting.attr_name] = setting.get_value_json()

            data["devices"][device_type] = {
                "settings": settings,
            }

        # Device settings
        for (device_type, device_name), (settings, overrides) in \
                self._device_settings.items():

            if device_type not in data["devices"].keys():
                data["devices"][device_type] = {}

            serialized_settings = {}

            for setting in settings:
                value = setting.get_value_json()
                serialized_settings[setting.attr_name] = value

            for setting in overrides:
                if setting.is_overridden(device_name):
                    value = setting.get_value_json(device_name)
                    serialized_settings[setting.attr_name] = value

            data["devices"][device_type][device_name] = serialized_settings

        # Save to file
        temp_buf = io.StringIO()
        try:
            json.dump(data, temp_buf, indent=4)
        except Exception as exc:
            LOGGER.error('Error while serializing config', exc_info=exc)
            return

        try:
            self.file_path.parent.mkdir(parents=True, exist_ok=True)
            with open(self.file_path, 'w', encoding='utf-8') as f:
                temp_buf.seek(0)
                shutil.copyfileobj(temp_buf, f)
                LOGGER.debug(f'Config File: {self.file_path} updated')
        except Exception as exc:
            LOGGER.error('Error writing config to disk', exc_info=exc)

    def on_device_name_changed(self, old_name, new_name):
        old_key = None

        # update the entry in device_settings as they are identified by name
        for (device_type, device_name), (_, overrides) in \
                self._device_settings.items():
            if device_name == old_name:
                old_key = (device_type, old_name)
                # we also need to patch the overrides for the same reason
                for setting in overrides:
                    setting.on_device_name_changed(old_name, new_name)
                break

        if old_key is not None:
            new_key = (old_key[0], new_name)
            self._device_settings[new_key] = self._device_settings.pop(old_key)
            self.save()
        else:
            LOGGER.warning(f"Device with name '{old_name}' not found in the config's _device_settings")

    def on_device_removed(self, _, device_type, device_name, update_config):
        if not update_config:
            return

        del self._device_settings[(device_type, device_name)]
        self.save()

    def get_project_dir(self) -> str:
        '''
        Get the root directory of the project.

        This is the directory in which the .uproject file lives.
        '''
        return os.path.dirname(self.UPROJECT_PATH.get_value().replace('"', ''))

    def get_project_content_dir(self) -> str:
        '''
        Get the "Content" directory of the project.
        '''
        return os.path.join(self.get_project_dir(), 'Content')

    def get_unreal_content_plugins(self) -> list[UnrealPlugin]:
        '''
        Get a list of Unreal Engine plugins that match the current
        Switchboard config's content plugin filter settings.

        By default, Switchboard searches inside project plugins as well as
        AdditionalPluginDirectories plugins for assets when populating UI
        fields such as the level or nDisplay config dropdown menus.

        Engine/other plugins in which to search for content can be selectively
        added though using the "Content Plugin Filters" setting.
        '''

        # Pop known plugin names from the config array, treat the rest as globs
        unreal_plugins: list[UnrealPlugin] = []
        filter_patterns: list[str] = []

        # Enabled project plugins are always enumerated for content
        unreal_plugins.extend(self.ue_plugin_mgr.enabled_project_plugins)

        for pattern in self.CONTENT_PLUGIN_FILTERS.get_value():
            if plugin := self.ue_plugin_mgr.get_plugin_by_name(pattern):
                unreal_plugins.append(plugin)
            else:
                filter_patterns.append(pattern)

        if filter_patterns:
            unreal_plugins.extend(UnrealPlugin.from_path_filters(
                self.UPROJECT_PATH.get_value(), filter_patterns))

        return unreal_plugins

    def resolve_content_path(
        self,
        file_path: Union[Path, str]
    ) -> Optional[PurePosixPath]:
        '''
        Resolve a file path on the file system to the corresponding content
        path in UE.
        '''

        return self.ue_plugin_mgr.file_to_content_path(file_path)

    def find_levels(self) -> list[str]:

        # show a progress bar if it is taking more a trivial amount of time
        progressDiag = QtWidgets.QProgressDialog(
            labelText='Finding levels...',
            cancelButtonText='Stop',
            minimum=0, 
            maximum=0, 
            parent=None
        )
        
        progressDiag.setWindowTitle('Unreal Level Finder')
        progressDiag.setModal(True)
        progressDiag.setMinimumDuration(1000)  # time before it shows up

        # Allow cancellation but still letting partial results through.
        stop_event = threading.Event()
        progressDiag.canceled.connect(stop_event.set)

        # Looks much better without the window frame.
        progressDiag.setWindowFlag(QtCore.Qt.FramelessWindowHint)

        # create an event object to signal when the function is done
        done_event = threading.Event()
        levels = []

        # our convenience worker function
        def find_level_work():
            self._find_levels(levels, stop_event)
            done_event.set()

        thread = threading.Thread(target=find_level_work)
        thread.start()

        # wait for the event to be set or the progress dialog to be canceled
        while not done_event.is_set() and not progressDiag.wasCanceled():
            progressDiag.setValue(progressDiag.value() + 1)
            QtWidgets.QApplication.processEvents()
            time.sleep(0.050)  # The worker thread will run faster if we sleep.

        progressDiag.close()

        thread.join()

        return levels

    def deduce_umap_gamepath_from_filepath(self, umap_filepath: Path) -> Optional[str]:
        '''
        Converts a .umap filepath a gamepath. It does this by finding its nearest "Content"
        ancestor and determining it it belongs to a project or to a uplugin, because that
        determines the root of the gamepath.

        Args:
            umap_filepath (Path):
                The filepath to the level.

        Returns:
            Optional[str]:
                The normalized gamepath string (e.g. "/Game/.../MyMap" or "/SomePlugin/.../MyMap"), 
                or None if something didn't work out.
        '''

        parts = umap_filepath.parts

        # Find the last occurrence of "Content" in the path
        try:
            idx = len(parts) - 1 - parts[::-1].index("Content")
        except ValueError:
            LOGGER.warning(f"No 'Content' folder in path: {umap_filepath}")
            return None

        # Use the relative paths under "Content" as those are in the gamepath
        rel_parts = parts[idx+1:]

        if not rel_parts:
            LOGGER.warning(f"No subpath under 'Content' in: {umap_filepath}")
            return None

        content_dir = Path(*parts[:idx+1])
        root_dir    = content_dir.parent

        # Figure out if it is a project or plugin map to establish the right prefix.

        prefix: Optional[str] = None

        if any(root_dir.glob("*.uproject")):
            prefix = "/Game"
        else:
            plugins = list(root_dir.glob("*.uplugin"))

            if plugins:

                # Use the first found uplugin as the plugin name used in the gamepath
                plugin_name = plugins[0].stem
                prefix = f"/{plugin_name}"

            else:
                LOGGER.warning(f"No .uproject or .uplugin under {root_dir}")
                return None

        # Strip off extension and join with forward slashes

        relativepath_without_ext = [
            part if not part.lower().endswith(".umap")
            else part[:-5]
            for part in rel_parts
        ]

        umap_gamepath = prefix + "/" + "/".join(relativepath_without_ext)

        return umap_gamepath

    def add_single_level(self, umap_gamepath: Path) -> List[str]:
        """
        Add the given to the SBCache.

        Args:
            umap_gamepath (Path):  
                Level's game path
        """

        if not umap_gamepath:
            return []

        path_str = str(umap_gamepath)

        current = self.get_levels()
        current = set(current)

        if path_str not in current:
            LOGGER.info(f'Level added via browse: "{path_str}"')
            levels = list(current) + [path_str]
        else:
            levels = list(current)

        # cache updated list of maps
        project = SBCache().query_or_create_project(self.UPROJECT_PATH.get_value())
        cmaps = [Map(id=None, project=None, gamepath=map) for map in levels]
        SBCache().update_project_maps(project=project, maps=cmaps)

        return levels

    def _find_levels(self, levels: list[str], stop_event: Optional[threading.Event] = None) -> None:
        '''
        Returns a list of full level paths in an Unreal Engine project and
        in plugins such as:
            [
                "/Game/Maps/MapName",
                "/MyPlugin/Levels/MapName"
            ]
        The slashes will always be "/" independent of the platform's separator.
        '''
        project_maps_path = os.path.normpath(
            os.path.join(
                self.get_project_content_dir(),
                self.MAPS_PATH.get_value()))

        # search_paths stores a list of tuples of the form
        # (unreal_plugin, directory_path). This allows us to differentiate
        # between maps in the project (unreal_plugin is None in that case) and
        # maps in a plugin.
        search_paths: list[tuple[Optional[UnrealPlugin], Path]] = [
            (None, Path(project_maps_path))
        ]

        for unreal_content_plugin in self.get_unreal_content_plugins():
            search_paths.append(
                (unreal_content_plugin,
                 unreal_content_plugin.plugin_content_path))

        def get_umap_files(dirpath) -> list:
            ''' Returns a list with all *.map files in the recursively searched dirpath

            Pre-filtering by .umap extension is faster than fnmatch on all files
            Currently there are no use cases where a different extension is of interest.

            Parameters
            ----------
            dirpath: str
                Directory to be recursively searched for .umap files

            Returns
            -------
                list
                    List of found map entries.
            '''
            umap_files = []

            for entry in os.scandir(dirpath):

                if stop_event and stop_event.is_set():
                    break

                if entry.is_file() and entry.name.endswith(".umap"):
                    umap_files.append(entry)
                elif entry.is_dir():
                    umap_files.extend(get_umap_files(entry.path))  # recursive

            return umap_files

        maps_filter = self.MAPS_FILTER.get_value()

        for (unreal_content_plugin, maps_path) in search_paths:

            # Stop the search if the user doesn't want to wait any longer.
            if stop_event and stop_event.is_set():
                break

            try:
                umaps = get_umap_files(maps_path)
            except FileNotFoundError:
                # This is normal for non-content plugins.
                continue

            for umap in umaps:
                if not fnmatch.fnmatch(umap.name, maps_filter):
                    continue

                if mapgamepath := self.resolve_content_path(umap.path):
                    mapgamepath = mapgamepath.with_suffix('')
                    if mapgamepath not in levels:
                        levels.append(str(mapgamepath))
                else:
                    LOGGER.warning(f"No game path for {umap.path}")

        # If the search was stopped, merge the search results with cache.
        # Otherwise replace the cache with the search results.

        project = SBCache().query_or_create_project(self.UPROJECT_PATH.get_value())

        if stop_event and stop_event.is_set():

            LOGGER.info("Stopped level scan early, this will merge partial search results with the existing cache.")
            
            existing_maps = SBCache().query_maps(project)
            existing_paths = {map.gamepath for map in existing_maps}

            merged_paths = sorted(existing_paths.union(levels))
            merged_maps = [Map(id=None, project=None, gamepath=path) for path in merged_paths]

            # update levels since the caller expects the merged results in there.
            levels.clear()
            levels.extend(merged_paths)
            
            SBCache().update_project_maps(project=project, maps=merged_maps)
        else:
            levels.sort()
            cmaps = [Map(id=None, project=None, gamepath=map) for map in levels]
            SBCache().update_project_maps(project=project, maps=cmaps)

    def get_levels(self) -> list[str]:
        ''' Returns a list with all *.map files from the cache '''
        project = SBCache().query_project(self.UPROJECT_PATH.get_value())
        if project is None:
            return []
        maps = SBCache().query_maps(project)
        return [map.gamepath for map in maps]

    def multiuser_server_path(self):
        if self.MUSERVER_SLATE_MODE.get_value():
            return self.engine_exe_path(
                self.ENGINE_DIR.get_value(), self.MULTIUSER_SLATE_SERVER_EXE.get_value())
        else:
            return self.engine_exe_path(
                self.ENGINE_DIR.get_value(), self.MULTIUSER_SERVER_EXE.get_value())

    def multiuser_server_session_directory_path(self):
        if self.MUSERVER_WORKING_DIR.get_value():
            return self.MUSERVER_WORKING_DIR.get_value()

        if self.MUSERVER_SLATE_MODE.get_value():
            return os.path.join(self.ENGINE_DIR.get_value(), "Programs", "UnrealMultiUserSlateServer", "Intermediate", "MultiUser")

        return os.path.join(self.ENGINE_DIR.get_value(), "Programs", "UnrealMultiUserServer", "Intermediate", "MultiUser")

    def multiuser_server_log_path(self):
        if self.MUSERVER_SLATE_MODE.get_value():
            return os.path.join(self.ENGINE_DIR.get_value(), "Programs", "UnrealMultiUserSlateServer", "Saved", "Logs", "UnrealMultiUserSlateServer.log")
        # else we get the path to the console server.
        return os.path.join(self.ENGINE_DIR.get_value(), "Programs", "UnrealMultiUserServer", "Saved", "Logs", "UnrealMultiUserServer.log")

    def listener_path(self):
        return self.engine_exe_path(
            self.ENGINE_DIR.get_value(), self.LISTENER_EXE.get_value())

    def sblhelper_path(self):
        return self.engine_exe_path(
            self.ENGINE_DIR.get_value(), self.SBLHELPER_EXE.get_value())

    # todo-dara: find a way to do this directly in the LiveLinkFace plugin code
    def unreal_device_addresses(self):
        unreal_addresses = []
        for (device_type, device_name), (settings, overrides) in \
                self._device_settings.items():
            if device_type == "Unreal":
                for setting in settings:
                    if setting.attr_name == "address":
                        unreal_addresses.append(setting.get_value(device_name))
        return unreal_addresses

    @staticmethod
    def engine_exe_path(engine_dir: str, exe_basename: str):
        '''
        Returns platform-dependent path to the specified engine executable.
        '''
        exe_name = exe_basename
        platform_bin_subdir = ''

        if sys.platform.startswith('win'):
            platform_bin_subdir = 'Win64'
            platform_bin_path = os.path.normpath(
                os.path.join(engine_dir, 'Binaries', platform_bin_subdir))
            given_path = os.path.join(platform_bin_path, exe_basename)
            if os.path.exists(given_path):
                return given_path

            # Use %PATHEXT% to resolve executable extension ambiguity.
            pathexts = os.environ.get(
                'PATHEXT', '.COM;.EXE;.BAT;.CMD').split(';')
            for ext in pathexts:
                testpath = os.path.join(
                    platform_bin_path, f'{exe_basename}{ext}')
                if os.path.isfile(testpath):
                    return testpath

            # Fallback despite non-existence.
            return given_path
        else:
            if sys.platform.startswith('linux'):
                platform_bin_subdir = 'Linux'
            elif sys.platform.startswith('darwin'):
                platform_bin_subdir = 'Mac'

            return os.path.normpath(
                os.path.join(
                    engine_dir, 'Binaries', platform_bin_subdir, exe_name))


class UserSettings(object):
    def init(self):
        user_settings_path = CONFIG_MGR.user_settings_file_path

        try:
            with open(user_settings_path, encoding='utf-8') as f:
                LOGGER.debug(f'Loading Settings {user_settings_path}')
                data = json.load(f)
        except FileNotFoundError:
            # Create a default user_settings
            LOGGER.debug('Creating default user settings')
            data = {}
        except ValueError:
            LOGGER.error(f'{user_settings_path} has invalid JSON format. ')

            backup_path = CONFIG_MGR.user_settings_backup_file_path

            answer = QtWidgets.QMessageBox.question(
                None,
                'Invalid User Settings',
                'User settings has invalid JSON format and will be replaced with a valid a new default JSON.\n\n'
                f'Do you want to save a backup named {ConfigManager.USER_SETTINGS_BACKUP_FILE_NAME} (overwrites existing)?',
                 QtWidgets.QMessageBox.Yes | QtWidgets.QMessageBox.No
            )
            if answer == QtWidgets.QMessageBox.Yes:
                if os.path.exists(backup_path):
                    os.remove(backup_path)
                shutil.copy(user_settings_path, backup_path)

            data = {}

        self.CONFIG = data.get('config')
        if self.CONFIG:
            try:
                self.CONFIG = CONFIG_MGR.get_absolute_config_path(self.CONFIG)
            except ConfigPathError as e:
                LOGGER.error(e)
                self.CONFIG = None

        if not self.CONFIG:
            config_paths = CONFIG_MGR.list_config_paths()
            self.CONFIG = config_paths[0] if config_paths else None

        # Address of the machine running Switchboard
        self.ADDRESS = LocalAddressSetting(
            "address",
            "Address",
            data.get("address", socket.gethostbyname(socket.gethostname()))
        )
        self.TRANSPORT_PATH = FilePathSetting(
                "transport_path",
                "Transport path",
                data.get('transport_path', '')
            )

        # UI Settings
        self.CURRENT_SEQUENCE = data.get('current_sequence', 'Default')
        self.CURRENT_SLATE = data.get('current_slate', 'Scene')
        self.CURRENT_TAKE = data.get('current_take', 1)
        self.CURRENT_LEVEL = data.get('current_level', None)
        self.LAST_BROWSED_PATH = data.get('last_browsed_path', None)

        # Save so any new defaults are written out
        self.save()

    def save(self):
        # Save will always happen after an initial load so let's validate that
        # the IP address is correct.
        #
        self.ADDRESS.check_and_fix_bad_ip()

        data = {
            'config': '',
            'address': self.ADDRESS.get_value(),
            'transport_path': self.TRANSPORT_PATH.get_value(),
            'current_sequence': self.CURRENT_SEQUENCE,
            'current_slate': self.CURRENT_SLATE,
            'current_take': self.CURRENT_TAKE,
            'current_level': self.CURRENT_LEVEL,
            'last_browsed_path': self.LAST_BROWSED_PATH,
        }

        if self.CONFIG:
            try:
                data['config'] = str(CONFIG_MGR.get_relative_config_path(self.CONFIG))
            except ConfigPathError as e:
                LOGGER.error(e)
            except ValueError as e:
                data['config'] = str(self.CONFIG)

        CONFIG_MGR.user_settings_file_path.parent.mkdir(parents=True, exist_ok=True)
        with open(CONFIG_MGR.user_settings_file_path, 'w', encoding='utf-8') as f:
            json.dump(data, f, indent=4)


# Get the user settings and load their config
SETTINGS = UserSettings()
CONFIG = Config()
