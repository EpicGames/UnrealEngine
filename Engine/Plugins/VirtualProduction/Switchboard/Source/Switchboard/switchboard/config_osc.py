# Copyright Epic Games, Inc. All Rights Reserved.

UE4_LAUNCH = '/UE4Launch'
UE4_LAUNCH_QUERY = '/UE4LaunchQuery'
UE4_LAUNCH_CONFIRM = '/UE4LaunchConfirm'

# Mobile
BATTERY = '/Battery'
BATTERY_QUERY = '/BatteryQuery'
THERMALS = '/Thermals'
THERMALS_QUERY = '/ThermalsQuery'
ARSESSION_START = '/ARSessionStart'
ARSESSION_START_CONFIRM = '/ARSessionStartConfirm'
ARSESSION_STOP = '/ARSessionStop'
ARSESSION_STOP_CONFIRM = '/ARSessionStopConfirm'
DISPLAY_OFF = '/VideoDisplayOff'
DISPLAY_ON = '/VideoDisplayOn'

# LiveLink
LIVE_LINK_CONNECT = '/LiveLink'
LIVE_LINK_SUBJECT = '/LiveLinkSubject'
ADD_LIVE_LINK_ADDRESS = '/AddLiveLinkAddress'
CLEAR_LIVE_LINK_ADDRESSES = '/ClearAllLiveLinkAddresses'
LIVE_LINK_STREAM_START = '/LiveLinkStreamStart'
LIVE_LINK_STREAM_STOP = '/LiveLinkStreamStop'

# Recording commands
RECORD_START = '/RecordStart'
RECORD_START_CONFIRM = '/RecordStartConfirm'
RECORD_STOP = '/RecordStop'
RECORD_STOP_CONFIRM = '/RecordStopConfirm'
RECORD_CANCEL = '/RecordCancel'
RECORD_CANCEL_CONFIRM = '/RecordCancelConfirm'
RECORD_ADD_MARK = '/RecordAddMark'
RECORD_ADD_MARK_CONFIRM = '/RecordAddMarkConfirm'
TAKE = '/Take'
TAKE_CONFIRM = '/TakeConfirm'
TAKE_QUERY = '/TakeQuery'
SLATE = '/Slate'
SLATE_CONFIRM = '/SlateConfirm'
SLATE_QUERY = '/SlateQuery'
SLATE_DESCRIPTION = '/SlateDescription'
DATA = '/Data'
DATA_QUERY = '/DataQuery'

# OSC settings
OSC_ADD_SEND_TARGET = '/OSCAddSendTarget'
OSC_ADD_SEND_TARGET_CONFIRM = '/OSCAddSendTargetConfirm'

# VPRC
VPRC_WILDCARD = '/VPRC*'
VPRC_RECIEVER_NAME = 'Record'

# MONITOR
MONITOR_WILDCARD = '/Monitor*'
MONITOR_RECIEVER_NAME = 'Monitor'
