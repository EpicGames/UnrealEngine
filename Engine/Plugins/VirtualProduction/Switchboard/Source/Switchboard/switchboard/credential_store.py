# Copyright Epic Games, Inc. All Rights Reserved.

from __future__ import annotations

from abc import abstractmethod, ABC
from dataclasses import dataclass
import json
import logging
import os
import pathlib
import stat
import sys
import tempfile
from typing import Optional


class CredentialStore(ABC):
    @dataclass
    class Credential:
        username: str
        blob: str

    @classmethod
    def encrypted_at_rest(cls) -> bool:
        return False

    @abstractmethod
    def get(
        self,
        key: str
    ) -> Optional[Credential]:
        pass

    @abstractmethod
    def set(
        self,
        key: str,
        val: Optional[Credential]
    ) -> None:
        pass

    @staticmethod
    def create() -> CredentialStore:
        if sys.platform.startswith('win'):
            return CredentialStoreWindows()
        elif os.name == 'posix':
            return CredentialStorePosix()

        return CredentialStoreGeneric()


class CredentialStoreGeneric(CredentialStore):
    def __init__(self):
        super().__init__()

        self._tempstore: dict[str, CredentialStore.Credential] = {}

    def get(
        self,
        key: str
    ) -> Optional[CredentialStore.Credential]:
        return self._tempstore.get(key)

    def set(
        self,
        key: str,
        val: Optional[CredentialStore.Credential]
    ) -> None:
        if val:
            self._tempstore[key] = val
        elif key in self._tempstore:
            self._tempstore.pop(key)


class CredentialStorePosix(CredentialStoreGeneric):
    PERMISSION_MASK = stat.S_IRWXU | stat.S_IRWXG | stat.S_IRWXO
    CRED_FILE_FLAGS = stat.S_IRUSR | stat.S_IWUSR  # 600
    CRED_DIR_FLAGS = stat.S_IRUSR | stat.S_IWUSR | stat.S_IXUSR  # 700

    def __init__(self):
        super().__init__()

        del self._tempstore

    @classmethod
    def _credential_dir(cls) -> pathlib.Path:
        return pathlib.Path(pathlib.Path.home(), '.switchboard')

    @classmethod
    def _credential_path(cls, key: str) -> pathlib.Path:
        return pathlib.Path(cls._credential_dir(), f'credential_{key}.json')

    def get(self, key: str) -> Optional[CredentialStore.Credential]:
        path = self._credential_path(key)
        if not path or not path.is_file():
            return None

        stat = os.lstat(path)
        current_uid = os.getuid()
        if stat.st_uid != current_uid:
            logging.error(
                f'Credential file {path} is not owned by the current user!')

        share_flags = stat.st_mode & self.PERMISSION_MASK
        if share_flags != self.CRED_FILE_FLAGS:
            logging.error(
                f'Credential file {path} has incorrect permissions!')

        with path.open() as f:
            cred = json.load(f)
            if ('user' in cred) and ('blob' in cred):
                return CredentialStore.Credential(cred['user'], cred['blob'])
            else:
                logging.error(f'Error parsing credential: {cred}')
                return None

    def set(self, key: str, val: Optional[CredentialStore.Credential]) -> None:
        cred_path = self._credential_path(key)
        if val is None:
            cred_path.unlink()
            return

        cred_dir = cred_path.parent
        cred_dir.mkdir(mode=0o700, parents=True, exist_ok=True)

        (tmp_fd, tmp_path) = tempfile.mkstemp(dir=cred_dir, text=True)
        with os.fdopen(tmp_fd, mode='w') as f:
            f.write(json.dumps({'user': val.username, 'blob': val.blob}))
        os.replace(tmp_path, cred_path)


if sys.platform.startswith('win'):
    import ctypes
    import ctypes.wintypes

    from .switchboard_ctypes import (CREDENTIALW, CredDeleteW, CredFree, CredReadW, CredWriteW,
                                     CRED_PERSIST_LOCAL_MACHINE, CRED_TYPE_GENERIC)


    class CredentialStoreWindows(CredentialStoreGeneric):
        def __init__(self):
            super().__init__()
            del self._tempstore

        @classmethod
        def encrypted_at_rest(cls) -> bool:
            return True

        def get(self, key: str) -> Optional[CredentialStore.Credential]:
            need_credfree = False
            cred = ctypes.pointer(CREDENTIALW())
            try:
                CredReadW(
                    TargetName=f'Switchboard_{key}',
                    Type=CRED_TYPE_GENERIC,
                    Flags=0,
                    Credential=ctypes.byref(cred)
                )

                need_credfree = True

                username = cred.contents.UserName
                blob_bytes = cred.contents.CredentialBlob
                blob_len = cred.contents.CredentialBlobSize
                blob_codepoints = [
                    int.from_bytes(blob_bytes[idx:idx+2], 'little')
                    for idx in range(0, blob_len, 2)]
                blob_str = ''.join(map(chr, blob_codepoints))

                return CredentialStore.Credential(username, blob_str)
            except KeyError:
                return None
            finally:
                if need_credfree:
                    CredFree(cred)

        def set(
            self,
            key: str,
            val: Optional[CredentialStore.Credential]
        ) -> None:
            target_name = f'Switchboard_{key}'
            if not val:
                CredDeleteW(target_name, CRED_TYPE_GENERIC)
                return

            cred = CREDENTIALW()
            cred.Type = CRED_TYPE_GENERIC
            cred.Persist = CRED_PERSIST_LOCAL_MACHINE
            cred.TargetName = target_name
            cred.UserName = val.username

            blob_bytearr = bytearray()
            for codepoint in val.blob:
                blob_bytearr.extend(int.to_bytes(ord(codepoint), 2, 'little'))

            blob_byte_len = len(blob_bytearr)
            cred.CredentialBlobSize = blob_byte_len
            cred.CredentialBlob = ((ctypes.wintypes.BYTE * blob_byte_len)
                                   .from_buffer(blob_bytearr))

            CredWriteW(ctypes.byref(cred), 0)


CREDENTIAL_STORE = CredentialStore.create()
