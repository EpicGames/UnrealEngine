# Copyright Epic Games, Inc. All Rights Reserved.

from __future__ import annotations

import gc

from PySide6.QtWidgets import QWidgetAction, QMenu
from typing import Callable, List

from switchboard.switchboard_logging import LOGGER

# Set as True to add a Debug menut to the Main Dialog
ADD_MENU = False


def test():
    ''' Placeholder for debugging tests. Will be called from Tools Menu item. '''
    pass


def register_menu_actions(menu: QMenu) -> None:
    '''
    Registers menu actions to the given QMenu object.

    This function adds a new action labeled "Test" to the provided QMenu. 
    When the action is triggered, the `test()` function is called.

    Args:
        menu (QMenu): The QMenu object to which the action should be added.
    '''

    action = register_menu_action(menu, "&Test")
    action.triggered.connect(lambda: test())


def register_menu_action(
        current_menu: QMenu,
        actionname: str,
        menunames: List[str] = [],
        actiontooltip: str = '') -> QWidgetAction:
    ''' Registers a QWidgetAction with the tools menu

    Args:
        current_menu: Current menu we're adding to.
        actionname: Name of the action to be added
        menunames: Submenu(s) where to place the given action

    Returns:
        QWidgetAction: The action that was created.
    '''

    # Find find submenu, and create the ones that don't exist along the way

    # iterate over menunames given
    for menuname in menunames:

        create_menu = True

        # try to find existing menu
        for current_action in current_menu.actions():

            submenu = current_action.menu()

            if submenu and (menuname == submenu.title()):
                current_menu = submenu
                create_menu = False
                break

        # if we didn't find the submenu, create it
        if create_menu:
            newmenu = QMenu(parent=current_menu)
            newmenu.setTitle(menuname)
            current_menu.addMenu(newmenu)
            current_menu = newmenu

    # add the given action
    action = QWidgetAction(current_menu)
    action.setText(actionname)
    action.setToolTip(actiontooltip)
    current_menu.setToolTipsVisible(True)
    current_menu.addAction(action)

    return action


def print_instances(cls: type, print_func: Callable[[object], str]) -> None:
    ''' Prints the number of instances and the identifying information of all instances of the given class,
    using the provided print_func for displaying identifying information. Useful when tracking down memory leaks.

    Example:
        print_instances(BoolSetting, lambda setting: setting.nice_name)

    Args:
        cls: The class type for which to print instances.
        print_func: A function that takes an object of type `cls` and returns a string for identifying it.
    '''
    instances = [obj for obj in gc.get_objects() if isinstance(obj, cls)]

    # Print the number of instances
    LOGGER.debug(f"{cls.__name__}: {len(instances)} instances")

    # Print the instances using the provided print_func
    for instance in sorted(instances, key=print_func):
        LOGGER.debug(f"{cls.__name__} instance: {print_func(instance)}")
