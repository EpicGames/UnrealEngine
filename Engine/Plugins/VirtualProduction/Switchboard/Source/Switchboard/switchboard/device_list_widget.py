# Copyright Epic Games, Inc. All Rights Reserved.

import weakref

import switchboard.switchboard_widgets as sb_widgets

from typing import Dict, List
from PySide6 import QtCore, QtGui, QtWidgets
from switchboard.devices.device_base import Device, DeviceStatus, PluginHeaderWidgets
from switchboard.devices.device_widget_base import DeviceAutoJoinMUServerUI


class DeviceListWidget(QtWidgets.QListWidget):
    signal_register_device_widget = QtCore.Signal(object)
    signal_remove_device = QtCore.Signal(object)
    signal_connect_all_plugin_devices_toggled = QtCore.Signal(str, bool)
    signal_open_all_plugin_devices_toggled = QtCore.Signal(str, bool)

    def __init__(self, name, parent=None):
        super().__init__(parent)

        self.setEditTriggers(QtWidgets.QAbstractItemView.NoEditTriggers)
        self.setFocusPolicy(QtCore.Qt.NoFocus)
        #self.setSelectionMode(QtWidgets.QAbstractItemView.NoSelection)
        self.setSelectionMode(QtWidgets.QAbstractItemView.MultiSelection)

        self._devices: Dict[int, Device] = {}  # key = device_hash

        self._header_by_category_name = {}

        self._autojoin_device_guard = False
        self.setMinimumSize(self.minimumSize().width(), 5)

    # Override all mouse events so that selection can be used for qss styling
    def mousePressEvent(self, e):
        pass

    def mouseReleaseEvent(self, e):
        pass

    def mouseDoubleClickEvent(self, e):
        pass

    def mouseMoveEvent(self, e):
        pass

    def contextMenuEvent(self, event):

        item = self.itemAt(event.pos())

        if not item:
            return

        item_widget = self.itemWidget(item)
        assert item_widget

        # No actions for clicking the header widget
        if isinstance(item_widget, DeviceWidgetHeader):
            return

        device_context_menu = QtWidgets.QMenu(self)

        device_context_menu.addAction("Remove device", lambda: self.ask_to_confirm_device_removal(event.pos()))
        item_widget.populate_context_menu(device_context_menu)

        device_context_menu.exec_(event.globalPos())
        event.accept()

    def ask_to_confirm_device_removal(self, pos):
        item = self.itemAt(pos)
        item_widget = self.itemWidget(item)

        confirmation = QtWidgets.QMessageBox.question(self, "Device Removal Confirmation", "Are you sure you want to delete this device?")
        if confirmation == QtWidgets.QMessageBox.Yes:
            self.signal_remove_device.emit(item_widget.device_hash)

    def devices_in_category(self, category):
        if category not in self._header_by_category_name:
            return []

        header_item = self._header_by_category_name[category]
        header_row = self.row(header_item)
        device_widgets = []
        for row in range(header_row+1, self.count()):
            widget = self.itemWidget(self.item(row))
            if type(widget) == DeviceWidgetHeader:
                break
            device_widgets.append(widget)
        return device_widgets

    def on_device_removed(self, device_hash, device_type, *args):
        self._devices.pop(device_hash)

        for i in range(self.count()):
            item = self.item(i)
            widget = self.itemWidget(item)
            if widget.device_hash == device_hash:
                self.takeItem(self.row(item))
                break

        category = device_type
        remaining_devices_in_category = self.devices_in_category(category)
        if len(remaining_devices_in_category) == 0:
            header_item = self._header_by_category_name[category]
            self.takeItem(self.row(header_item))
            self._header_by_category_name.pop(category)

        self.update_header_autojoin_state(category)

    def on_connect_all_toggled(self, plugin_name, state):
        self.signal_connect_all_plugin_devices_toggled.emit(plugin_name, state)

    def on_open_all_toggled(self, plugin_name, state):
        self.signal_connect_all_plugin_devices_toggled.emit(plugin_name, state)

    def add_header(self, label_name, show_open_button, show_connect_button, show_changelist, show_autojoin):
        header_widget = DeviceWidgetHeader(label_name, show_open_button, show_connect_button, show_changelist, show_autojoin)
        header_widget.signal_connect_all_toggled.connect(self.signal_connect_all_plugin_devices_toggled)
        header_widget.signal_open_all_toggled.connect(self.signal_open_all_plugin_devices_toggled)

        device_item = QtWidgets.QListWidgetItem()
        header_item_size = QtCore.QSize(header_widget.sizeHint().width(), sb_widgets.DEVICE_HEADER_LIST_WIDGET_HEIGHT)
        device_item.setSizeHint(header_item_size)
        self.addItem(device_item)
        self.setItemWidget(device_item, header_widget)

        self.attach_autojoin_button(header_widget, label_name)
        return device_item

    def update_header_autojoin_state(self, category):
        if self._autojoin_device_guard:
            return

        header_item = self._header_by_category_name.get(category)
        if not header_item:
            return

        header_widget = self.itemWidget(header_item)
        assert isinstance(header_widget, DeviceWidgetHeader)

        if header_widget.autojoin_mu is None:
            return

        any_set = False
        for device in self.devices():
            is_autojoin_set = device.autojoin_mu_server and device.autojoin_mu_server.get_value()
            any_set = any_set or is_autojoin_set

        wasBlockingSignals = header_widget.autojoin_mu.get_button().blockSignals(True)
        header_widget.autojoin_mu.get_button().setChecked(any_set)
        header_widget.autojoin_mu.get_button().blockSignals(wasBlockingSignals)

    def connect_to_device_autojoin(self, device, category):
        ''' Connects to the device autojoin setting with a weak reference to avoid strong reference cycles. '''
        self_weak_ref = weakref.ref(self)

        def child_device_changed_weak():
            self_instance = self_weak_ref()
            if self_instance is not None:
                self_instance.update_header_autojoin_state(category)

        if device.autojoin_mu_server:
            device.autojoin_mu_server.signal_setting_changed.connect(child_device_changed_weak)
            self.update_header_autojoin_state(category)

    def attach_autojoin_button(self, header_widget, category):
        if header_widget.autojoin_mu is None:
            return
        autojoin = header_widget.autojoin_mu

        def do_set_autojoin():
            self.set_autojoin_mu(autojoin, category)

        autojoin.signal_device_widget_autojoin_mu.connect(do_set_autojoin)

    def set_autojoin_mu(self, autojoin_ui, category):
        self._autojoin_device_guard = True

        is_checked = autojoin_ui.is_autojoin_enabled()
        for device in self.devices_in_category(category):
            device.autojoin_mu.set_autojoin_mu(is_checked)

        self._autojoin_device_guard = False

    def add_device_widget(self, device: Device):
        category = device.category_name
        if category not in self._header_by_category_name.keys():
            header_widget_config = device.plugin_header_widget_config()
            show_open_button = PluginHeaderWidgets.OPEN_BUTTON in header_widget_config
            show_connect_button = PluginHeaderWidgets.CONNECT_BUTTON in header_widget_config
            show_changelist = PluginHeaderWidgets.CHANGELIST_LABEL in header_widget_config
            show_autojoin = PluginHeaderWidgets.AUTOJOIN_MU in header_widget_config
            self._header_by_category_name[category] = self.add_header(category, show_open_button, show_connect_button, show_changelist, show_autojoin)

        header_item = self._header_by_category_name[category]
        header_row = self.row(header_item)

        device_item = QtWidgets.QListWidgetItem()
        device_item_size = QtCore.QSize(device.widget.sizeHint().width(), sb_widgets.DEVICE_LIST_WIDGET_HEIGHT)
        device_item.setSizeHint(device_item_size)
        self.insertItem(header_row+1, device_item)
        self.setItemWidget(device_item, device.widget)

        # Keep a dict for quick lookup
        self._devices[device.device_hash] = device

        self.signal_register_device_widget.emit(device.widget)
        # force the widget to update, to make sure status is correct
        device.widget.update_status(device.status, device.status, device)

        self.connect_to_device_autojoin(device, category)

        return device.widget

    def clear_widgets(self):
        self.clear()
        self._devices = {}
        self._header_by_category_name = {}

    def widget_item_by_device(self, device: Device):
        for i in range(self.count()):
            item = self.item(i)
            widget = self.itemWidget(item)

            if not widget:
                continue

            if widget.device_hash == device.device_hash:
                return item
        return None

    def device_widget_by_hash(self, device_hash: int):
        if device_hash not in self._devices:
            return None

        return self._devices[device_hash].widget

    def devices(self):
        return self._devices.values()

    def device_widgets(self):
        widgets = [device.widget for device in self._devices.values()]
        return widgets

    def update_category_status(
            self,
            category_name: str,
            devices: List[Device]):
        header_item = self._header_by_category_name.get(category_name)
        if not header_item:
            # The last device in the category may have just been deleted, in
            # which case there'll be no header item to update.
            return

        header_widget = self.itemWidget(header_item)
        assert isinstance(header_widget, DeviceWidgetHeader)

        def device_control_buttons(button_name):
            for device in devices:
                widget = self.device_widget_by_hash(device.device_hash)
                if widget:
                    yield widget.control_buttons[button_name]

        try:
            # New logic: base directly on devices' button states.
            connect_checked = any(
                b.isChecked()
                for b in device_control_buttons('connect'))
            connect_enabled = any(
                b.isEnabled() and (b.isChecked() == connect_checked)
                for b in device_control_buttons('connect'))

            open_checked = any(
                b.isChecked()
                for b in device_control_buttons('open'))
            open_enabled = any(
                b.isEnabled() and (b.isChecked() == open_checked)
                for b in device_control_buttons('open'))

            header_widget.update_connect_state(checked=connect_checked,
                                               enabled=connect_enabled)
            header_widget.update_open_state(checked=open_checked,
                                            enabled=open_enabled)
        except KeyError:
            # Couldn't index expected control_buttons; use old logic.
            any_connected = False
            any_opened = False
            for device in devices:
                any_connected |= (not device.is_disconnected)
                any_opened |= (device.status > DeviceStatus.CLOSED)

            header_widget.update_connect_state(checked=any_connected,
                                               enabled=True)
            header_widget.update_open_state(checked=any_opened,
                                            enabled=any_connected)

    def get_connect_and_open_all_button_states(self):
        # FIXME? Basically the same logic as update_category_status above.
        def category_header_widgets():
            for category, header_item in self._header_by_category_name.items():
                if not header_item:
                    continue

                header_widget = self.itemWidget(header_item)
                assert isinstance(header_widget, DeviceWidgetHeader)
                yield header_widget

        connect_checked = any(
            (w.connect_button and w.connect_button.isChecked())
            for w in category_header_widgets())
        connect_enabled = any(
            (w.connect_button and w.connect_button.isEnabled()
             and (w.connect_button.isChecked() == connect_checked))
            for w in category_header_widgets())

        open_checked = any(
            (w.open_button and w.open_button.isChecked())
            for w in category_header_widgets())
        open_enabled = any(
            (w.open_button and w.open_button.isEnabled()
             and (w.open_button.isChecked() == open_checked))
            for w in category_header_widgets())

        return connect_checked, connect_enabled, open_checked, open_enabled


class DeviceWidgetHeader(QtWidgets.QWidget):

    signal_connect_all_toggled = QtCore.Signal(str, bool) # params: plugin_name, toggle_state
    signal_open_all_toggled = QtCore.Signal(str, bool) # params: plugin_name, toggle_state

    def __init__(self, name, show_open_button, show_connect_button, show_changelist, show_autojoin, parent=None):
        super().__init__(parent)

        self.name = name
        self.address_label = None
        self.device_hash = 0 # When list widget is looking for devices it checks the device_has

        self.layout = QtWidgets.QHBoxLayout()
        self.layout.setContentsMargins(0, 6, 17, 6)
        self.setLayout(self.layout)

        def __label(label_text):
            label = QtWidgets.QLabel()
            label.setText(label_text)
            label.setObjectName('widget_header')
            return label

        self.name_label = __label(self.name + " Devices")
        self.address_label = __label('Address')
        self.changelist_label = __label('Changelist') if show_changelist else None
        self.layout.addWidget(self.name_label)
        self.layout.addWidget(self.address_label)

        spacer = QtWidgets.QSpacerItem(0, 20, QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Minimum)
        self.layout.addItem(spacer)

        if self.changelist_label:
            self.layout.addWidget(self.changelist_label)
            spacer = QtWidgets.QSpacerItem(0, 20, QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Minimum)
            self.layout.addItem(spacer)

        self.autojoin_mu = None
        if show_autojoin:
            self.autojoin_mu = DeviceAutoJoinMUServerUI("")
            self.autojoin_mu_button = self.autojoin_mu.make_button(self)
            self.autojoin_mu_button.setStyleSheet("padding-right: -6px;")
            self.layout.addWidget(self.autojoin_mu_button)

        self.open_button = None
        if show_open_button:
            self.open_button = sb_widgets.ControlQPushButton.create(
                icon_size=QtCore.QSize(21, 21),
                tool_tip=f'Start all connected {self.name} devices',
                hover_focus=False,
                name='open')

            self.layout.setAlignment(self.open_button, QtCore.Qt.AlignVCenter)
            self.open_button.clicked.connect(self.on_open_button_clicked)
            self.open_button.setEnabled(False)
            self.layout.addWidget(self.open_button)

        self.connect_button = None
        if show_connect_button:
            self.connect_button = sb_widgets.ControlQPushButton.create(
                icon_size=QtCore.QSize(21, 21),
                tool_tip=f'Connect all {self.name} devices',
                hover_focus=False,
                name='connect')

            self.layout.setAlignment(self.connect_button, QtCore.Qt.AlignVCenter)
            self.connect_button.clicked.connect(self.on_connect_button_clicked)
            self.layout.addWidget(self.connect_button)

        self.name_label.setMinimumSize(QtCore.QSize(235, 0))
        self.address_label.setMinimumSize(QtCore.QSize(100, 0))

    def showEvent(self, event):
        super().showEvent(event)

        self.find_address_label()

    def find_address_label(self):
        labels = self.findChildren(QtWidgets.QLabel)
        for label in labels:
            if label.text() == 'Address':
                self.address_label = label
                return

    def resizeEvent(self, event):
        super().resizeEvent(event)

        if not self.address_label:
            return

        width = event.size().width()

        if width < sb_widgets.DEVICE_WIDGET_HIDE_ADDRESS_WIDTH:
            self.address_label.hide()
        else:
            self.address_label.show()

    def paintEvent(self, event):
        opt = QtWidgets.QStyleOption()
        opt.initFrom(self)
        painter = QtGui.QPainter(self)
        self.style().drawPrimitive(QtWidgets.QStyle.PE_Widget, opt, painter, self)

    def on_open_button_clicked(self, checked):
        button_state = checked
        plugin_name = self.name
        self.signal_open_all_toggled.emit(plugin_name, button_state)

    def on_connect_button_clicked(self, checked):
        button_state = checked
        plugin_name = self.name
        self.signal_connect_all_toggled.emit(plugin_name, button_state)

    def update_connect_state(self, checked, enabled):
        if self.connect_button is None:
            return
        self.connect_button.setEnabled(enabled)
        self.connect_button.setChecked(checked)
        if checked:
            self.connect_button.setToolTip(f"Disconnect all connected {self.name} devices")
        else:
            self.connect_button.setToolTip(f"Connect all {self.name} devices")

    def update_open_state(self, checked, enabled):
        if self.open_button is None:
            return
        self.open_button.setEnabled(enabled)
        self.open_button.setChecked(checked)
        if checked:
            self.open_button.setToolTip(f"Stop all running {self.name} devices")
        else:
            self.open_button.setToolTip(f"Start all connected {self.name} devices")
