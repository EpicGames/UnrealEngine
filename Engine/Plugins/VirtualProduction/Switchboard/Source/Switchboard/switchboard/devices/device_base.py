# Copyright Epic Games, Inc. All Rights Reserved.

from __future__ import annotations

import os
import random
import socket
import threading
import time
import weakref

from enum import IntEnum, IntFlag, auto, unique
from typing import Optional, Type, Dict, OrderedDict, TYPE_CHECKING

from PySide6 import QtCore
from PySide6 import QtGui

import pythonosc.udp_client

from switchboard import config_osc as osc
from switchboard import recording
from switchboard.config import CONFIG, BoolSetting, AddressSetting, Setting
from switchboard.switchboard_logging import LOGGER

if TYPE_CHECKING:
    from switchboard.devices.device_widget_base import \
        AddDeviceDialog, DeviceWidget


@unique
class DeviceStatus(IntEnum):
    DELETE = auto()
    DISCONNECTED = auto()
    CONNECTING = auto()
    CLOSED = auto()
    CLOSING = auto()
    SYNCING = auto()
    BUILDING = auto()
    OPEN = auto()
    READY = auto()
    RECORDING = auto()
    PACKAGING = auto()


@unique
class PluginHeaderWidgets(IntFlag):
    OPEN_BUTTON = 1,
    CONNECT_BUTTON = 2,
    CHANGELIST_LABEL = 4,
    AUTOJOIN_MU = 8


class DeviceQtHandler(QtCore.QObject):
    signal_device_status_changed = QtCore.Signal(object, int)
    signal_device_connect_failed = QtCore.Signal(object)
    signal_device_client_disconnected = QtCore.Signal(object)
    signal_device_project_changelist_changed = QtCore.Signal(object)
    signal_device_engine_changelist_changed = QtCore.Signal(object)
    signal_device_built_changelist_changed = QtCore.Signal(object)
    signal_device_sync_failed = QtCore.Signal(object)
    signal_device_is_recording_device_changed = QtCore.Signal(object, int)

    # Signal parameters are: device, step, percent
    signal_device_build_update = QtCore.Signal(object, str, str)

    # Signal parameters are: device, percent
    signal_device_sync_update = QtCore.Signal(object, str)

    # Signal parameters are: device, step, percentstr
    signal_device_package_update = QtCore.Signal(object, str, str)


class Device(QtCore.QObject):
    # Falls back to the default AddDeviceDialog as specified in
    # device_widget_base.
    add_device_dialog: Optional[Type[AddDeviceDialog]] = None
    device_widget = None  # FIXME?: Unused?

    csettings = {
        'is_recording_device': BoolSetting(
            attr_name="is_recording_device",
            nice_name='Is Recording Device',
            value=True,
            tool_tip='Is this device used to record',
            category='Device Settings',
        )
    }

    def __init__(self, name: str, address: str, **kwargs):
        super().__init__()

        self._name = name  # Assigned name
        self.device_qt_handler = DeviceQtHandler()
        self.autojoin_mu_server = False
        self.setting_address = AddressSetting('address', 'Address', address, category="Network Settings")

        weak_self = weakref.ref(self)
        self.setting_address.signal_setting_changed.connect(
            lambda: weak_self() and weak_self()._check_recreate_osc_client())

        # override any setting that was passed via kwargs
        for setting in self.setting_overrides():
            if setting.attr_name in kwargs.keys():
                override = kwargs[setting.attr_name]
                setting.override_value_from_config(self.name, override)

        self._project_changelist: Optional[int | str] = None
        self._engine_changelist: Optional[int | str] = None

        self._status = DeviceStatus.DISCONNECTED

        # If the device should autoconnect on startup
        self.auto_connect = True

        # TODO: This is not really a hash. Could be changed to real hash based
        # on e.g. address and name.
        self.device_hash = random.getrandbits(64)

        # Lazily create the OSC client as needed
        self.osc_client: Optional[pythonosc.udp_client.SimpleUDPClient] = None

        self.device_recording = None

        # Runtime tracking
        self._runtime_start_time: Optional[float] = None

        self.widget: DeviceWidget = None  # Constructed during init()

    def init(self, widget_class, icons):

        self.widget = widget_class(self.name, self.device_hash, self.address, icons)

        self.widget.signal_device_name_changed.connect(self.on_device_name_changed)
        self.widget.signal_address_changed.connect(self.on_address_changed)

        # weakref to void an unintended strong reference that can prevent garbage collection

        self_weak_ref = weakref.ref(self)

        def on_address_changed_weak(old_address, new_address):
            self_instance = self_weak_ref()
            if self_instance is not None and self_instance.widget is not None:
                self_instance.widget.on_address_changed(new_address)

        self.setting_address.signal_setting_changed.connect(on_address_changed_weak)

        self.disable_override_style_on_device_only_settings()

        # Let the CONFIG class know what settings/properties this device has
        # so it can save the config file on changes.
        CONFIG.register_device_settings(
            self.device_type, self.name, self.device_settings(),
            self.setting_overrides())

    def on_device_name_changed(self, new_name):
        if self.name != new_name:
            old_name = self.name
            self.name = new_name
            CONFIG.on_device_name_changed(old_name, new_name)

    def on_address_changed(self, new_address):
        if self.address != new_address:
            self.address = new_address

    def disable_override_style_on_device_only_settings(self):
        ''' If a given setting does not exist in the plugin settings, then there is
        no point in supporting override highlights. If we don't do this, we may get
        override highlights on device instance properties that don't have a plugin
        value.
        '''
        for setting in self.device_settings():
            if setting.attr_name not in self.plugin_settings():
                setting.allow_override_style = False

    @property
    def device_type(self):
        return self.__class__.__name__.split("Device", 1)[1]

    @classmethod
    def load_plugin_icons(cls):
        plugin_name = cls.__name__.split("Device", 1)[1]
        plugin_dir = plugin_name.lower()
        icon_dir = os.path.join(os.path.dirname(__file__), plugin_dir, "icons")
        icons = {}
        for _, _, files in os.walk(icon_dir):
            for fname in files:
                state, _ = os.path.splitext(fname)
                icon_path = os.path.join(icon_dir, fname)
                icon = icons[state] = QtGui.QIcon()
                icon.addFile(icon_path)
        return icons

    @classmethod
    def reset_csettings(cls):
        for csetting in cls.csettings.values():
            csetting.reset()

    @classmethod
    def plugin_settings(cls) -> list[Setting]:
        # settings that are shared by all devices of a plugin
        return list(cls.csettings.values())

    def device_settings(self) -> list[Setting]:
        # settings that are specific to an instance of the device
        return [self.setting_address]

    def setting_overrides(self):
        # All settings that a device may override. these might be project or
        # plugin settings.
        return []

    @property
    def name(self) -> str:
        return self._name

    @name.setter
    def name(self, value: str):
        if self._name == value:
            return
        self._name = value

    @property
    def category_name(self):
        return self.device_type

    @classmethod
    def plugin_header_widget_config(cls):
        """
        Combination of widgets that will be visualized in the plugin header.
        """
        return PluginHeaderWidgets.CONNECT_BUTTON

    @property
    def is_recording_device(self):
        return Device.csettings['is_recording_device'].get_value(self.name)

    @is_recording_device.setter
    def is_recording_device(self, value):
        Device.csettings['is_recording_device'].update_value(value)
        self.device_qt_handler.signal_device_is_recording_device_changed.emit(
            self, value)

    @property
    def address(self) -> str:
        return self.setting_address.get_value()

    @address.setter
    def address(self, value: str):
        self.setting_address.update_value(value)

    @property
    def status(self):
        return self._status

    @status.setter
    def status(self, value: DeviceStatus):
        previous_status = self._status
        self._status = value
        
        # Track runtime when device starts/stops running
        if value == DeviceStatus.OPEN and previous_status != DeviceStatus.OPEN:
            self._runtime_start_time = time.time()
        elif previous_status == DeviceStatus.OPEN and value != DeviceStatus.OPEN:
            self._runtime_start_time = None
            
        self.device_qt_handler.signal_device_status_changed.emit(
            self, previous_status)

    @property
    def is_disconnected(self):
        return self.status in {
            DeviceStatus.DISCONNECTED, DeviceStatus.CONNECTING}

    def get_runtime_string(self) -> Optional[str]:
        ''' Returns a string representing how long the device has been running.
        If the device is not actually running, returns None.

        Shows hours when >= 1 hour, and days when >= 24 hours.

        e.g.
                  "01:02" (1 min, 2 seconds)
               "01:02:03" (1 hour, 2 minutes, 3 seconds)
            "1d 02:03:04" (1 day, 2 hours, 3 minutes, 4 seconds)
        '''

        if self.status != DeviceStatus.OPEN or self._runtime_start_time is None:
            return None
            
        runtime_seconds = int(time.time() - self._runtime_start_time)
        days = runtime_seconds // 86400
        hours = (runtime_seconds % 86400) // 3600
        minutes = (runtime_seconds % 3600) // 60
        seconds = runtime_seconds % 60
        
        if days > 0:
            return f"{days}d {hours:02d}:{minutes:02d}:{seconds:02d}"
        elif hours > 0:
            return f"{hours:02d}:{minutes:02d}:{seconds:02d}"
        else:
            return f"{minutes:02d}:{seconds:02d}"

    @property
    def project_changelist(self):
        return self._project_changelist

    @project_changelist.setter
    def project_changelist(self, value: Optional[int | str]):
        self._project_changelist = value
        self.device_qt_handler.signal_device_project_changelist_changed.emit(
            self)

    @property
    def engine_changelist(self):
        return self._engine_changelist

    @engine_changelist.setter
    def engine_changelist(self, value: Optional[int | str]):
        self._engine_changelist = value
        self.device_qt_handler.signal_device_engine_changelist_changed.emit(
            self)

    @property
    def device_osc_port(self) -> int:
        raise NotImplementedError()

    def set_slate(self, value):
        pass

    def set_take(self, value):
        pass

    @QtCore.Slot()
    def connecting_listener(self):
        # Ensure this code is run from the main thread
        if threading.current_thread() is not threading.main_thread():
            QtCore.QMetaObject.invokeMethod(
                self, 'connecting_listener', QtCore.Qt.QueuedConnection)
            return

        # If the device was disconnected, set to connecting.
        # We can only transition to CONNECTING from DISCONNECTED.
        if self.status == DeviceStatus.DISCONNECTED:
            self.status = DeviceStatus.CONNECTING

    @QtCore.Slot()
    def connect_listener(self):
        # Ensure this code is run from the main thread
        if threading.current_thread() is not threading.main_thread():
            QtCore.QMetaObject.invokeMethod(
                self, 'connect_listener', QtCore.Qt.QueuedConnection)
            return

        # If the device was disconnected, set to closed.
        # We can transition to CLOSED from either DISCONNECTED or CONNECTING.
        if self.is_disconnected:
            self.status = DeviceStatus.CLOSED

    @QtCore.Slot()
    def disconnect_listener(self):
        # Ensure this code is run from the main thread
        if threading.current_thread() is not threading.main_thread():
            QtCore.QMetaObject.invokeMethod(
                self, 'disconnect_listener', QtCore.Qt.QueuedConnection)
            return

        self.status = DeviceStatus.DISCONNECTED

    def setup_osc_client(self, osc_port):
        try:
            LOGGER.osc(
                f'{self.name}: (Re)creating OSC client '
                f'{self.address}:{osc_port}')

            self.osc_client = pythonosc.udp_client.SimpleUDPClient(
                self.address, osc_port)
        except socket.gaierror as exc:
            LOGGER.error(f'{self.name}: Invalid OSC server address',
                         exc_info=exc)

    def _check_recreate_osc_client(self):
        if self.osc_client is not None:
            address_changed = self.osc_client._address != self.address
            port_changed = self.osc_client._port != self.device_osc_port
            if address_changed or port_changed:
                self.setup_osc_client(self.device_osc_port)

    def send_osc_message(self, command, value, log=True):
        if not self.osc_client:
            self.setup_osc_client(self.device_osc_port)

        if log:
            LOGGER.osc(
                f'OSC Server: Sending {command} {value} '
                f'to {self.name} ({self.address})')

        try:
            self.osc_client.send_message(command, value)
        except socket.gaierror as e:
            LOGGER.error(
                f'{self.name}: Incorrect address when sending '
                f'OSC message. {e}')

    def record_start(self, slate, take, description):
        if self.is_disconnected or not self.is_recording_device:
            return

        self.send_osc_message(osc.RECORD_START, [slate, take, description])

    def record_stop(self):
        if (self.status != DeviceStatus.RECORDING or
                not self.is_recording_device):
            return

        self.send_osc_message(osc.RECORD_STOP, 1)

    def record_start_confirm(self, timecode):
        self.device_recording = self.new_device_recording(
            self.name, self.device_type, timecode)
        self.status = DeviceStatus.RECORDING

    def record_stop_confirm(self, timecode, paths=None):
        if self.device_recording:
            self.device_recording.timecode_out = timecode
            if paths:
                self.device_recording.paths = paths

        self.status = DeviceStatus.READY

    def get_device_recording(self):
        return self.device_recording

    def transport_paths(self, device_recording):
        """
        Return the transport paths for the passed in device_recording
        This is used to create TransportJobs. If the device does not specify
        which paths it should transport, then no jobs will be moved to the
        transport path.
        """
        return device_recording.paths

    def transport_file(self, device_path, output_dir):
        pass

    def process_file(self, device_path, output_path):
        pass

    def device_widget_registered(self, device_widget):
        ''' Called when the device's widget is registered
            You could connect to its signals here.
        '''
        pass

    def should_allow_exit(self, close_req_id: int) -> bool:
        '''
        Returning `False` will interrupt an attempt to exit Switchboard.
        Can be used as an opportunity to prompt with a message box, etc.
        `close_req_id` is unique per exit request/attempt.
        '''
        return True

    @staticmethod
    def new_device_recording(device_name, device_type, timecode_in):
        """
        Create a new DeviceRecording
        """
        device_recording = recording.DeviceRecording()
        device_recording.device_name = device_name
        device_recording.device_type = device_type
        device_recording.timecode_in = timecode_in

        return device_recording

    @classmethod
    def plug_into_ui(cls, menubar, tabs):
        ''' Plugins can take this opportunity to add their own options to the UI
        '''
        pass

    @classmethod
    def close_all_devices(cls, devices):
        '''
        Handle closing all devices of this type when "Stop all" button is pressed.
        Override this method to customize behavior for specific device types.
        Default implementation closes each device individually.
        '''
        for device in devices:
            try:
                open_button = device.widget.open_button
                if open_button.isEnabled() and open_button.isChecked():
                    device.widget._close()
            except Exception:
                pass

    @classmethod
    def added_device(cls, device):
        ''' DeviceManager informing that this plugin device has been created and added.
        '''
        pass

    @classmethod
    def removed_device(cls, device):
        ''' DeviceManager informing that this plugin device has been removed.
        '''
        pass

    @classmethod
    def all_devices_added(cls):
        ''' Called after the device manager adds all the devices of this type
        '''
        pass

    @classmethod
    def sort_setting_categories(cls, categories: Dict) -> OrderedDict:
        ''' Sort the settings categories so that they are shown in the desired order '''
        return OrderedDict(categories)
