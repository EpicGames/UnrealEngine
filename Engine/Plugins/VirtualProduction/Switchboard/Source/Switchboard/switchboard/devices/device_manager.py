# Copyright Epic Games, Inc. All Rights Reserved.

import importlib
import inspect
import os
import traceback
from typing import Dict, List, Type

from PySide6 import QtCore, QtGui

from switchboard.switchboard_logging import LOGGER
from switchboard.devices.device_base import Device, DeviceStatus
from switchboard.devices.device_widget_base import AddDeviceDialog, DeviceWidget
import switchboard.switchboard_utils as utils


DEVICE_PLUGIN_PACKAGE = "switchboard.devices"
DEVICE_PLUGIN_PATH = os.path.join(os.path.dirname(__file__))


class DeviceManager(QtCore.QObject):
    signal_device_added = QtCore.Signal(object)
    signal_device_removed = QtCore.Signal(object, str, str, bool)

    def __init__(self):
        super().__init__()

        self._plugins, self._plugin_widgets, self._plugin_icons = \
            self.find_available_device_plugins()

        self._devices: Dict[int, Device] = {}  # key is device_hash
        self._device_name_validator = DeviceNameValidator(self)

    def add_devices(self, device_config):
        for device_type, devices in device_config.items():
            for device in devices:
                self._add_device(device_type, device["name"], device["address"], **device["kwargs"])

        # notify the plugins that all devices were added (gets called even if no devices were added)
        for device_cls in self._plugins.values():
            device_cls.all_devices_added()

    def _add_device(self, device_type, name, address, **kwargs):
        device_cls_name = device_type
        if device_cls_name not in self._plugins:
            LOGGER.error(f"Could not find plugin for {device_type} in {DEVICE_PLUGIN_PACKAGE}")
            return None

        device = self._plugins[device_cls_name](name, address, **kwargs)
        widget_class = self._plugin_widgets[device_cls_name]
        icons = self._plugin_icons[device_cls_name] if device_cls_name in self._plugin_icons.keys() else None
        device.init(widget_class, icons)
        self._devices[device.device_hash] = device

        device.widget.set_name_validator(self._device_name_validator)

        # Notify the plugin
        device.__class__.added_device(device)

        self.signal_device_added.emit(device)
        return device

    def find_available_device_plugins(self):
        plugin_modules = self._find_plugin_modules()

        found_plugins: Dict[str, Type[Device]] = {}
        plugin_widgets: Dict[str, Type[DeviceWidget]] = {}

        for plugin in plugin_modules:
            try:
                plugin_module = importlib.import_module(plugin)
            except:
                LOGGER.error(f"Error while loading plugin: {plugin}\n\n=== Traceback BEGIN ===\n{traceback.format_exc()}=== Traceback END ===\n")
                continue

            members = inspect.getmembers(plugin_module, inspect.isclass)

            for (name, c) in members:
                # only add classes that are a sub-class of Device but not Device itself
                if issubclass(c, Device) and (c is not Device):
                    display_name = utils.remove_prefix(name, "Device")
                    found_plugins[display_name] = c
                elif issubclass(c, DeviceWidget) and (c is not DeviceWidget):
                    display_name = utils.remove_prefix(name, "DeviceWidget")
                    plugin_widgets[display_name] = c

        plugin_icons = {}
        
        for name, plugin in found_plugins.items():
            plugin_icons[name] = plugin.load_plugin_icons()

        return found_plugins, plugin_widgets, plugin_icons

    def _find_plugin_modules(self):
        ''' Discovers and returns qualified import paths for plugins. '''
        plugin_modules: List[str] = []
        device_subdirs = next(os.walk(DEVICE_PLUGIN_PATH))[1]

        for subdir in device_subdirs:

            module_name = f"plugin_{subdir}"
            path = os.path.join(DEVICE_PLUGIN_PATH, subdir, module_name + ".py")

            if os.path.exists(path):
                module = ".".join([DEVICE_PLUGIN_PACKAGE, subdir, module_name])
                plugin_modules.append(module)

        return plugin_modules

    def get_device_add_dialog(self, device_type):

        if self._plugins[device_type].add_device_dialog is None:
            dialog = AddDeviceDialog(device_type, self.devices())
        else:
            dialog = self._plugins[device_type].add_device_dialog(self.devices())

        dialog.add_name_validator(self._device_name_validator)
        return dialog

    def remove_device(self, device, update_config=True):
        # Remove the device from the dict
        self._devices.pop(device.device_hash)

        # Disconnect the device
        device.disconnect_listener()

        # Set status to delete
        device.status = DeviceStatus.DELETE

        # Notify the plugin
        device.__class__.removed_device(device)

        self.signal_device_removed.emit(device.device_hash, device.device_type, device.name, update_config)

    def remove_device_by_hash(self, device_hash):
        device = self.device_with_hash(device_hash)
        self.remove_device(device)

    def clear_device_list(self):
        ''' Removes all device instances. 
        '''
        devices_being_removed = list(self._devices.values())

        for device in devices_being_removed:
            self.remove_device(device, update_config=False)

        if len(self._devices):
            LOGGER.error(f"{inspect.currentframe().f_code.co_name} failed to remove all devices one by one")
            self._devices.clear()

    def reset_plugins_settings(self, config):
        ''' Resets all plugins' settings, including their values and overrides.
        This function should be called right after a new config is being loaded or created.
        '''
        for plugin in self._plugins.values():
            plugin.reset_csettings()

        for name, plugin in self._plugins.items():
            plugin_settings = plugin.plugin_settings()
            config.register_plugin_settings(name, plugin_settings)
            config.load_plugin_settings(name, plugin_settings)

    def available_device_plugins(self):
        return self._plugins

    def plugin_settings(self, device_type):
        return self._plugins[device_type].plugin_settings()

    def plugin_icons(self, device_type):
        return self._plugin_icons[device_type]

    def auto_connect(self):
        for device in self._devices.values():
            # Auto connect any devices
            if device.auto_connect:
                device.connect_listener()

    def devices(self):
        return list(self._devices.values())

    def devices_of_type(self, device_type):
        return [device for device in self._devices.values() if device.device_type == device_type]

    def device_with_hash(self, device_hash):
        if device_hash not in self._devices:
            return None

        return self._devices[device_hash]

    def device_with_address(self, address):
        for device in self._devices.values():
            if device.address == address:
                return device
        return None

    def device_with_name(self, name):
        for device in self._devices.values():
            if device.name == name:
                return device
        return None

    def is_name_unique(self, name):
        for device in self._devices.values():
            if device.name.lower() == name.lower():
                return False
        return True

    def plug_into_ui(self, menubar, tabs):
        for plugin in self._plugins.values():
            plugin.plug_into_ui(menubar, tabs)


class DeviceNameValidator(QtGui.QValidator):
    def __init__(self, device_manager, parent=None):
        super().__init__(parent)
        self.device_manager = device_manager

    def validate(self, input, pos):
        if self.device_manager.is_name_unique(input):
            return QtGui.QValidator.Acceptable
        return QtGui.QValidator.Intermediate
