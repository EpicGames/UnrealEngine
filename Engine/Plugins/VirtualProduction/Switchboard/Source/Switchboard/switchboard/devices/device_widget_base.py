# Copyright Epic Games, Inc. All Rights Reserved.
from typing import Dict, Optional

from PySide6 import QtWidgets, QtGui, QtCore

from switchboard.config import CONFIG
from switchboard.switchboard_widgets import FramelessQLineEdit
from switchboard.devices.device_base import DeviceStatus, Device
import switchboard.switchboard_widgets as sb_widgets


class DeviceRunningTimeTooltip(QtWidgets.QWidget):
    ''' Custom tooltip widget that can constantly update its display of the device's running time.
    
    It should only apply to devices currently with OPEN status.
    '''
    
    def __init__(self, parent=None):
        super().__init__(parent, QtCore.Qt.ToolTip | QtCore.Qt.FramelessWindowHint)

        self.setAttribute(QtCore.Qt.WA_ShowWithoutActivating)
        
        self.label = QtWidgets.QLabel()
        self.label.setWordWrap(True)
        layout = QtWidgets.QVBoxLayout()
        layout.setContentsMargins(0, 0, 0, 0)
        layout.addWidget(self.label)
        self.setLayout(layout)
        
        # Timer to update the display every second

        self.update_timer = QtCore.QTimer()
        self.update_timer.timeout.connect(self.update_content)
        self.update_timer.setInterval(1000)
        
        self.device: Optional[Device] = None
        
    def show_for_device(self, device, position):
        '''Show the tooltip for the given device at the given position.
        '''
        self.device = device
        self.update_content()
        self.move(position)
        self.show()
        self.update_timer.start()
        
    def hide_tooltip(self):
        ''' Hide the tooltip and stop updates.
        '''
        self.update_timer.stop()
        self.hide()
        self.device = None
        
    def update_content(self):
        ''' Update the tooltip content the device's current running time.
        '''

        if not self.device or self.device.status != DeviceStatus.OPEN:
            self.hide_tooltip()
            return

        # Read the device's running time as a string to display in the tooltip            
        runtime_str = self.device.get_runtime_string()

        if runtime_str:
            tooltip_text = f"Device has been running for {runtime_str}"
        else:
            tooltip_text = "Device has been started"
        
        self.label.setText(tooltip_text)
        self.adjustSize()


class DeviceWidgetItem(QtWidgets.QWidget):
    """
    Custom class to get QSS working correctly to achieve a look.
    This allows the QSS to set the style of the DeviceWidgetItem and change its color when recording
    """
    def __init__(self, parent=None):
        super().__init__(parent)

    def paintEvent(self, event):
        opt = QtWidgets.QStyleOption()
        opt.initFrom(self)
        painter = QtGui.QPainter(self)
        self.style().drawPrimitive(QtWidgets.QStyle.PE_Widget, opt, painter, self)


class DeviceAutoJoinMUServerUI(QtCore.QObject):
    signal_device_widget_autojoin_mu = QtCore.Signal(object)

    autojoin_mu_default = True

    def __init__(self, name, parent = None):
        super().__init__(parent)
        self.name = name
        self._button = None

    def __del__(self):
        try:
            CONFIG.MUSERVER_AUTO_JOIN.signal_setting_changed.disconnect(self.disable_enable_based_on_global)
        except (RuntimeError, TypeError):
            pass

    def is_autojoin_enabled(self):
        return self._button.isChecked()

    def set_autojoin_mu(self, is_checked):
        if is_checked is not self.is_autojoin_enabled():
            self._button.setChecked(is_checked)
            self._set_autojoin_mu(is_checked)

    def _set_autojoin_mu(self, is_checked):
        self.signal_device_widget_autojoin_mu.emit(self)

    def get_button(self):
        return self._button

    def disable_enable_based_on_global(self):
        self._button.setEnabled(CONFIG.MUSERVER_AUTO_JOIN.get_value())

    def make_button(self, parent):
        """
        Make a new device setting push button.
        """
        self._button = sb_widgets.ControlQPushButton.create(
                icon_size=QtCore.QSize(15, 15),
                tool_tip=f'Toggle Auto-join for Multi-user Server',
                hover_focus=False,
                name='autojoin'
        )

        self.set_autojoin_mu(self.autojoin_mu_default)
        self._button.toggled.connect(self._set_autojoin_mu)
        CONFIG.MUSERVER_AUTO_JOIN.signal_setting_changed.connect(self.disable_enable_based_on_global)
        self.disable_enable_based_on_global()
        return self._button


class DeviceWidget(QtWidgets.QWidget):
    signal_device_widget_connect = QtCore.Signal(object)
    signal_device_widget_disconnect = QtCore.Signal(object)
    signal_device_widget_open = QtCore.Signal(object)
    signal_device_widget_close = QtCore.Signal(object)
    signal_device_widget_sync = QtCore.Signal(object)
    signal_device_widget_build = QtCore.Signal(object)
    signal_device_widget_trigger_start_toggled = QtCore.Signal(object, bool)
    signal_device_widget_trigger_stop_toggled = QtCore.Signal(object, bool)

    signal_device_name_changed = QtCore.Signal(str)
    signal_address_changed = QtCore.Signal(str)

    hostname_validator = sb_widgets.HostnameValidator()

    def __init__(self, name, device_hash, address, icons, parent=None):
        super().__init__(parent)

        # Lookup device by a hash instead of name/address
        self.device_hash = device_hash
        self.icons = icons

        # Status Label
        self.status_icon = QtWidgets.QLabel()
        self.status_icon.setGeometry(0, 0, 11, 1)
        pixmap = QtGui.QPixmap(":/icons/images/status_blank_disabled.png")
        self.status_icon.setPixmap(pixmap)
        self.status_icon.resize(pixmap.width(), pixmap.height())
        
        # Enable mouse tracking for hover events
        self.status_icon.setMouseTracking(True)
        self.status_icon.enterEvent = self._on_status_icon_enter
        self.status_icon.leaveEvent = self._on_status_icon_leave

        # Device icon
        self.device_icon = QtWidgets.QLabel()
        self.device_icon.setGeometry(0, 0, 40, 40)
        pixmap = self.icon_for_state("enabled").pixmap(QtCore.QSize(40, 40))
        self.device_icon.setPixmap(pixmap)
        self.device_icon.resize(pixmap.width(), pixmap.height())
        self.device_icon.setMinimumSize(QtCore.QSize(60, 40))
        self.device_icon.setAlignment(QtCore.Qt.AlignCenter)

        self.name_validator = None
        self.help_tool_tip = QtWidgets.QToolTip()

        # Device name
        self.name_line_edit = FramelessQLineEdit()
        self.name_line_edit.textChanged[str].connect(self.on_name_changed)
        self.name_line_edit.editingFinished.connect(self.on_name_edited)

        self.name_line_edit.setText(name)
        self.name_line_edit.setObjectName('device_name')
        self.name_line_edit.setMaximumSize(QtCore.QSize(150, 40))
        # 20 + 11 + 60 + 150

        # Address Label
        self.address_line_edit = FramelessQLineEdit()
        self.address_line_edit.setObjectName('device_address')
        self.address_line_edit.setValidator(DeviceWidget.hostname_validator)
        self.address_line_edit.editingFinished.connect(self.on_address_edited)
        self.address_line_edit.setText(address)
        self.address_line_edit.setAlignment(QtCore.Qt.AlignCenter)
        self.address_line_edit.setMaximumSize(QtCore.QSize(100, 40))

        # Create a widget where the body of the item will go
        # This is made to allow the edit buttons to sit "outside" of the item
        self.widget = DeviceWidgetItem()
        self.edit_layout = QtWidgets.QHBoxLayout()
        self.edit_layout.setContentsMargins(0,0,0,0)
        self.setLayout(self.edit_layout)
        self.edit_layout.addWidget(self.widget)

        # Main layout where the contents of the item will live
        self.layout = QtWidgets.QHBoxLayout()
        self.layout.setContentsMargins(20, 2, 20, 2)
        self.layout.setSpacing(2)
        self.widget.setLayout(self.layout)

        self.add_widget_to_layout(self.status_icon)
        self.add_widget_to_layout(self.device_icon)
        self.add_widget_to_layout(self.name_line_edit)
        self.add_widget_to_layout(self.address_line_edit)

        spacer = QtWidgets.QSpacerItem(0, 20, QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Minimum)
        self.add_item_to_layout(spacer)

        # Store previous status for efficiency
        #self.previous_status = DeviceStatus.DISCONNECTED

        # Set style as disconnected
        for label in [self.name_line_edit, self.address_line_edit]:
            sb_widgets.set_qt_property(label, 'disconnected', True)

        # Store the control buttons by name ("connect", "open", etc.)
        self.control_buttons: Dict[str, sb_widgets.ControlQPushButton] = {}

        # Runtime tracking
        self._current_device = None
        self._runtime_tooltip = DeviceRunningTimeTooltip(self)
        
        # Timer for tooltip delay
        self._tooltip_delay_timer = QtCore.QTimer()
        self._tooltip_delay_timer.setSingleShot(True)
        self._tooltip_delay_timer.timeout.connect(self._show_delayed_tooltip)
        self._pending_tooltip_pos = None

        self._add_control_buttons()

    def add_widget_to_layout(self, widget):
        ''' Adds a widget to the layout '''

        self.layout.addWidget(widget)

    def add_item_to_layout(self, item):
        ''' Adds an item to the layout '''

        self.layout.addItem(item)

    def can_sync(self):
        return False

    def can_build(self):
        return False

    def icon_for_state(self, state):
        if state in self.icons.keys():
            return self.icons[state]
        else:
            if "enabled" in self.icons.keys():
                return self.icons["enabled"]
            else:
                return QtGui.QIcon()

    def set_name_validator(self, name_validator):
        self.name_validator = name_validator

    def on_name_changed(self, text):
        if not self.name_validator:
            return

        if self.name_validator.validate(text, self.name_line_edit.cursorPosition()) != QtGui.QValidator.State.Acceptable:
            rect = self.name_line_edit.parent().mapToGlobal(self.name_line_edit.geometry().topRight())
            self.help_tool_tip.showText(rect, "Names must be unique")

            sb_widgets.set_qt_property(self.name_line_edit, "input_error", True)
            self.name_line_edit.is_valid = False
        else:
            self.name_line_edit.is_valid = True
            sb_widgets.set_qt_property(self.name_line_edit, "input_error", False)
            self.help_tool_tip.hideText()

    def on_name_edited(self):
        new_value = self.name_line_edit.text()

        if self.name_line_edit.is_valid and self.name_line_edit.current_text != new_value:
            sb_widgets.set_qt_property(self.name_line_edit, "input_error", False)

            self.signal_device_name_changed.emit(new_value)

    def on_address_edited(self):
        new_value = self.address_line_edit.text()

        if self.address_line_edit.is_valid and self.address_line_edit.current_text != new_value:
            sb_widgets.set_qt_property(self.address_line_edit, "input_error", False)

            self.signal_address_changed.emit(new_value)

    def on_address_changed(self, new_address):
        self.address_line_edit.setText(new_address)

    def _add_control_buttons(self):
        pass

    def _on_status_icon_enter(self, event):
        ''' Show custom tooltip when mouse enters status icon.
        '''

        if self._current_device and self._current_device.status == DeviceStatus.OPEN:
            # Position tooltip similar to the regular tooltip
            self._pending_tooltip_pos = QtGui.QCursor.pos() + QtCore.QPoint(10, 10)

            # Start delay timer. Without the timer the tooltip appears too fast and before you center the mouse
            self._tooltip_delay_timer.start(700)
        else:
            # For non-OPEN devices, show the regular tooltip
            self._show_regular_tooltip()

    def _on_status_icon_leave(self, event):
        ''' Hide custom tooltip when mouse leaves status icon.
        '''
        # Cancel pending tooltip
        self._tooltip_delay_timer.stop()
        self._runtime_tooltip.hide_tooltip()
        
    def _show_delayed_tooltip(self):
        ''' Show the custom tooltip after delay.
        '''
        if self._current_device and self._current_device.status == DeviceStatus.OPEN and self._pending_tooltip_pos:
            self._runtime_tooltip.show_for_device(self._current_device, self._pending_tooltip_pos)

    def _show_regular_tooltip(self):
        ''' Show the appropriate regular tooltip based on device status.
        '''

        if not self._current_device:
            return
            
        status = self._current_device.status
        if status >= DeviceStatus.READY:
            self.status_icon.setToolTip("Ready to start recording")
        elif status == DeviceStatus.DISCONNECTED:
            self.status_icon.setToolTip("Disconnected")
        elif status == DeviceStatus.CONNECTING:
            self.status_icon.setToolTip("Connecting...")
        else:
            self.status_icon.setToolTip("Connected")

    def update_status(self, status, previous_status, device=None):
        # Store device reference for runtime updates
        self._current_device = device
        
        # Status Icon
        if status >= DeviceStatus.READY:
            self.status_icon.setPixmap(QtGui.QPixmap(":/icons/images/status_green.png"))
            self.status_icon.setToolTip("Ready to start recording")
        elif status == DeviceStatus.DISCONNECTED:
            pixmap = QtGui.QPixmap(":/icons/images/status_blank_disabled.png")
            self.status_icon.setPixmap(pixmap)
            self.status_icon.setToolTip("Disconnected")
        elif status == DeviceStatus.CONNECTING:
            pixmap = QtGui.QPixmap(":/icons/images/status_orange.png")
            self.status_icon.setPixmap(pixmap)
            self.status_icon.setToolTip("Connecting...")
        elif status == DeviceStatus.OPEN:
            pixmap = QtGui.QPixmap(":/icons/images/status_orange.png")
            self.status_icon.setPixmap(pixmap)
            # Clear regular tooltip for running devices (custom tooltip will handle this)
            self.status_icon.setToolTip("")
        else:
            self.status_icon.setPixmap(QtGui.QPixmap(":/icons/images/status_cyan.png"))
            self.status_icon.setToolTip("Connected")

        # Device icon
        if status in {DeviceStatus.DISCONNECTED, DeviceStatus.CONNECTING}:
            for label in [self.name_line_edit, self.address_line_edit]:
                sb_widgets.set_qt_property(label, 'disconnected', True)

            pixmap = self.icon_for_state("disabled").pixmap(QtCore.QSize(40, 40))
            self.device_icon.setPixmap(pixmap)

            if status == DeviceStatus.DISCONNECTED:
                # Make the name and address editable when disconnected.
                self.name_line_edit.setReadOnly(False)
                self.address_line_edit.setReadOnly(False)
            elif status == DeviceStatus.CONNECTING:
                # Make the name and address non-editable while connecting.
                self.name_line_edit.setReadOnly(True)
                self.address_line_edit.setReadOnly(True)
        elif ((previous_status in {DeviceStatus.DISCONNECTED, DeviceStatus.CONNECTING}) and
                status > DeviceStatus.CONNECTING):
            for label in [self.name_line_edit, self.address_line_edit]:
                sb_widgets.set_qt_property(label, 'disconnected', False)

            pixmap = self.icon_for_state("enabled").pixmap(QtCore.QSize(40, 40))
            self.device_icon.setPixmap(pixmap)

            # Make the name and address non-editable when connected.
            self.name_line_edit.setReadOnly(True)
            self.address_line_edit.setReadOnly(True)

        # Handle coloring List Widget items if they are recording
        if status == DeviceStatus.RECORDING:
            sb_widgets.set_qt_property(self.widget, 'recording', True)
        else:
            sb_widgets.set_qt_property(self.widget, 'recording', False)

    def resizeEvent(self, event):
        super().resizeEvent(event)

        width = event.size().width()

        if width < sb_widgets.DEVICE_WIDGET_HIDE_ADDRESS_WIDTH:
            self.address_line_edit.hide()
        else:
            self.address_line_edit.show()

    def assign_button_to_name(self, name, button):
        if name:
            self.control_buttons[name] = button

    def add_control_button(self, *args, name: Optional[str] = None, **kwargs):
        button = sb_widgets.ControlQPushButton.create(*args, name=name,
                                                      **kwargs)
        self.add_widget_to_layout(button)

        self.assign_button_to_name(name, button)
        return button

    def populate_context_menu(self, cmenu: QtWidgets.QMenu):
        ''' Called to populate the given context menu with any desired actions'''
        pass


class AddDeviceDialog(QtWidgets.QDialog):
    def __init__(self, device_type, existing_devices, parent=None):
        super().__init__(parent=parent, f=QtCore.Qt.WindowCloseButtonHint)

        self.device_type = device_type
        self.setWindowTitle(f"Add {self.device_type} Device")

        self.name_field = QtWidgets.QLineEdit(self)
        self.name_field.textChanged.connect(lambda: self.refresh_validity())

        self.address_field = QtWidgets.QLineEdit(self)
        self.address_field.setValidator(DeviceWidget.hostname_validator)
        self.address_field.textChanged.connect(lambda: self.refresh_validity())

        self.form_layout = QtWidgets.QFormLayout()
        self.form_layout.addRow("Name", self.name_field)
        self.form_layout.addRow("Address", self.address_field)

        layout = QtWidgets.QVBoxLayout()
        layout.insertLayout(0, self.form_layout)

        self.button_box = QtWidgets.QDialogButtonBox(QtWidgets.QDialogButtonBox.Ok | QtWidgets.QDialogButtonBox.Cancel)
        self.ok_btn = self.button_box.button(QtWidgets.QDialogButtonBox.StandardButton.Ok)
        self.cancel_btn = self.button_box.button(QtWidgets.QDialogButtonBox.StandardButton.Cancel)

        self.button_box.accepted.connect(lambda: self.accept())
        self.button_box.rejected.connect(lambda: self.reject())
        layout.addWidget(self.button_box)

        self.setLayout(layout)
        self.refresh_validity()

    def add_name_validator(self, validator):
        if self.name_field:
            self.name_field.setValidator(validator)

    def refresh_validity(self):
        name_valid = self.name_field.hasAcceptableInput()
        addr_valid = self.address_field.hasAcceptableInput()

        sb_widgets.set_qt_property(self.name_field, 'validation', 'acceptable' if name_valid else 'intermediate')
        sb_widgets.set_qt_property(self.address_field, 'validation', 'acceptable' if addr_valid else 'intermediate')

        self.ok_btn.setEnabled(name_valid and addr_valid)

    def devices_to_add(self):
        return [{"type": self.device_type, "name": self.name_field.text(), "address": self.address_field.text(), "kwargs": {}}]

    def devices_to_remove(self):
        return []
