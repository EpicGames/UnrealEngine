# Copyright Epic Games, Inc. All Rights Reserved.
from functools import wraps
import os

from PySide6 import QtCore

from switchboard import recording
from switchboard.config import BoolSetting
from switchboard.devices.device_base import Device, DeviceStatus
from switchboard.devices.device_widget_base import DeviceWidget
from switchboard.switchboard_logging import LOGGER
import switchboard.switchboard_utils as utils

from .thirdparty.aja.embedded.rest import kipro


def unresponsive_kipro(f):
    """
    Decorator to gracefully disconnect if a KiPro command comes back with
    kipro.UnresponsiveTargetError.
    """
    @wraps(f)
    def wrapped(self, *args, **kwargs):
        try:
            from .thirdparty.aja.embedded.rest import kipro
            return f(self, *args, **kwargs)
        except kipro.UnresponsiveTargetError:
            self.device_qt_handler.signal_device_connect_failed.emit(self)
            return None
        except ModuleNotFoundError:
            LOGGER.error(
                'Could not connect to KiPro because the module was not '
                'installed')
            return None

    return wrapped


class DeviceKiPro(Device):
    def __init__(self, name, address, **kwargs):
        self.setting_auto_play = BoolSetting(
            "auto_play", "Auto Play After Stop", False)

        super().__init__(name, address, **kwargs)

        self.client = None

        self._interval = 0.1
        self._timeout = 4

        self.repeat_function = None

    def device_settings(self):
        settings = super().device_settings()
        settings.append(self.setting_auto_play)
        return settings

    @property
    def auto_play(self):
        return self.setting_auto_play.get_value()

    @auto_play.setter
    def auto_play(self, value):
        if self.setting__auto_play.get_value() == value:
            return

        self.setting_auto_play.update_value(value)

    # TODO: Make sure KiPro is not in transport mode
    @unresponsive_kipro
    def connect_listener(self):
        self.client = kipro.Client(f'http://{self.address}')

        self.status = DeviceStatus.READY

    def disconnect_listener(self):
        self.status = DeviceStatus.DISCONNECTED
        self.client = None

    @unresponsive_kipro
    def set_slate(self, value):
        self.client.setParameter('eParamID_CustomClipName', value)

    @unresponsive_kipro
    def set_take(self, value):
        self.client.setParameter('eParamID_CustomTake', value)

    @unresponsive_kipro
    def record_start(self, slate, take, description):
        if self.is_disconnected or not self.is_recording_device:
            return

        self.set_media_state_for_record_play()

        # Set Slate/Take
        self.set_slate(slate)
        self.set_take(take)

        # Start a new recording
        self.device_recording = recording.DeviceRecording()
        self.device_recording.device_name = self.name
        self.device_recording.device_type = self.device_type

        # If there is already a repeating function trying to record/stop/play,
        # stop it.
        if self.repeat_function:
            self.repeat_function.stop()
            self.repeat_function = None

        # Start device Recording
        self.repeat_function = utils.RepeatFunction(
            self._interval, self._timeout, self.record)
        self.repeat_function.add_finish_callback(
            self.record_start_confirm, None)  # TODO: Don't bake
        self.repeat_function.start(
            results_function=lambda results: results[-1] == 'Recording')

    @unresponsive_kipro
    def record_stop(self):
        # If there is already a repeating function trying to record, stop it
        if self.repeat_function:
            self.repeat_function.stop()
            self.repeat_function = None

        self.repeat_function = utils.RepeatFunction(
            self._interval, self._timeout, self.stop)
        self.repeat_function.add_finish_callback(
            self.record_stop_confirm, self.timecode())  # TODO: Don't bake TC
        self.repeat_function.start(
            results_function=lambda results: results[-1] == 'Idle')

    @unresponsive_kipro
    def record_stop_confirm(self, timecode, paths=None):
        # storage_path = self.client.getParameter('eParamID_StoragePath')
        try:
            clip_name = self.client.getCurrentClipName()
            self.device_recording.paths = [os.path.join('media/', clip_name)]
            self.device_recording.status = recording.RecordingStatus.ON_DEVICE
            self.device_recording.timecode_out = timecode
        except Exception as e:
            LOGGER.error(f'{self.name}: record_stop_confirm error - {e}')

        if self.repeat_function:
            self.repeat_function.stop()
            self.repeat_function = None

        if self.auto_play:
            self.repeat_function = utils.RepeatFunction(
                self._interval, self._timeout, self.play)
            self.repeat_function.start(
                results_function=lambda results:
                    results[-1] == 'Playing Forward')

        # self.transport_file(
        #     self.device_recording.paths[0],
        #     'C:/Users/jacob.buck/Downloads/Test/test.mov')

        self.status = DeviceStatus.READY

    @unresponsive_kipro
    def timecode(self):
        """
        Figure out issues with timecode. When going into playback mode it will
        report the timecode for the playback and not reset to the current
        timecode until some time after start/stop occurs.
        """
        return self.client.getTimecodeWithSynchronousCall()

    @unresponsive_kipro
    def play(self):
        if self.is_disconnected:
            return

        self.set_media_state_for_record_play()
        self.client.play()

        return self.client.getTransporterState()

    @unresponsive_kipro
    def stop(self):
        # if self.is_disconnected:
        #     return

        self.set_media_state_for_record_play()
        self.client.stop()

        return self.client.getTransporterState()

    @unresponsive_kipro
    def record(self):
        if self.is_disconnected:
            return

        self.set_media_state_for_record_play()
        self.client.record()

        return self.client.getTransporterState()

    @unresponsive_kipro
    def fast_forward(self):
        if self.is_disconnected:
            return

        self.set_media_state_for_record_play()
        self.client.fastForward()

        return self.client.getTransporterState()

    @unresponsive_kipro
    def fast_reverse(self):
        if self.is_disconnected:
            return

        self.set_media_state_for_record_play()
        self.client.fastReverse()

        return self.client.getTransporterState()

    @unresponsive_kipro
    def set_media_state_for_record_play(self):
        """
        KiPro can be put into a state to download videos. When in this mode it
        will no longer accept Record or Play functions. This function makes
        sure the state is set back to accept record/play.
        """
        _, description = self.client.getParameter('eParamID_MediaState')

        if description == 'Data - LAN':
            self.client.setParameter('eParamID_MediaState', 0)

        # {'eParamID_MediaState':
        #     'Define whether the Ki Pro device is in normal Record-Play or '
        #     'Data Transfer mode.'}
        # (200, 'Data - LAN') client.setParameter('eParamID_MediaState', 1)
        # (200, 'Record-Play') client.setParameter('eParamID_MediaState', 0)

    def transport_file(self, device_path, output_dir):
        self.client.setParameter('eParamID_MediaState', 1)

        device_path = f'http://{self.address}/{device_path}'
        output_path = os.path.join(output_dir, os.path.basename(device_path))

        # Put all drives into file transfer mode
        utils.download_file(device_path, output_path)


class DeviceWidgetKiPro(DeviceWidget):
    signal_device_widget_play = QtCore.Signal(object)
    signal_device_widget_stop = QtCore.Signal(object)

    def __init__(self, name, device_hash, address, icons, parent=None):
        super().__init__(name, device_hash, address, icons, parent=parent)

    def _add_control_buttons(self):
        super()._add_control_buttons()

        # Play Button
        self.play_button = self.add_control_button(
            ':/icons/images/icon_play.png',
            icon_hover=':/icons/images/icon_play_hover.png',
            icon_disabled=':/icons/images/icon_play_disabled.png',
            icon_on=':/icons/images/icon_play.png',
            icon_hover_on=':/icons/images/icon_play_hover.png',
            icon_disabled_on=':/icons/images/icon_play_disabled.png',
            tool_tip='Connect/Disconnect from listener')

        self.play_button.clicked.connect(self.play_button_clicked)

        # Stop Button
        self.stop_button = self.add_control_button(
            ':/icons/images/icon_stop.png',
            icon_hover=':/icons/images/icon_stop_hover.png',
            icon_disabled=':/icons/images/icon_stop_disabled.png',
            icon_on=':/icons/images/icon_stop.png',
            icon_hover_on=':/icons/images/icon_stop_hover.png',
            icon_disabled_on=':/icons/images/icon_stop_disabled.png',
            tool_tip='Connect/Disconnect from listener')

        self.stop_button.clicked.connect(self.stop_button_clicked)

        # Connect Button
        self.connect_button = self.add_control_button(
            ':/icons/images/icon_connect.png',
            icon_hover=':/icons/images/icon_connect_hover.png',
            icon_disabled=':/icons/images/icon_connect_disabled.png',
            icon_on=':/icons/images/icon_connected.png',
            icon_hover_on=':/icons/images/icon_connected_hover.png',
            icon_disabled_on=':/icons/images/icon_connected_disabled.png',
            tool_tip='Connect/Disconnect from listener')

        self.connect_button.clicked.connect(self.connect_button_clicked)

    def update_status(self, status, previous_status, device=None):
        super().update_status(status, previous_status, device)

        if status in {DeviceStatus.DISCONNECTED, DeviceStatus.CONNECTING}:
            self.play_button.setDisabled(True)
            self.stop_button.setDisabled(True)
        elif status >= DeviceStatus.CLOSED:
            self.play_button.setDisabled(False)
            self.stop_button.setDisabled(False)

    def connect_button_clicked(self):
        if self.connect_button.isChecked():
            self._connect()
        else:
            self._disconnect()

    def play_button_clicked(self):
        self.signal_device_widget_play.emit(self)

    def stop_button_clicked(self):
        self.signal_device_widget_stop.emit(self)

    def _connect(self):
        # Make sure the button is in the correct state
        self.connect_button.setChecked(True)

        # Emit Signal to Switchboard
        self.signal_device_widget_connect.emit(self)

    def _disconnect(self):
        # Make sure the button is in the correct state
        self.connect_button.setChecked(False)

        # Emit Signal to Switchboard
        self.signal_device_widget_disconnect.emit(self)
