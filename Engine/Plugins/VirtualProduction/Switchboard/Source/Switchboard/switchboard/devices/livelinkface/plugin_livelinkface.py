# Copyright Epic Games, Inc. All Rights Reserved.

import datetime

from PySide6 import QtCore, QtGui

from switchboard import config_osc as osc
from switchboard.config import CONFIG, SETTINGS, IntSetting
from switchboard.devices.device_base import Device, DeviceStatus
from switchboard.devices.device_widget_base import DeviceWidget
from switchboard.switchboard_logging import LOGGER


class DeviceLiveLinkFace(Device):
    csettings = {
        'osc_port': IntSetting(
            attr_name='osc_port',
            nice_name='OSC Port',
            value=8000,
            tool_tip=(
                'Must match the port on which the Live Link Face app is '
                'configured to listen for OSC connections.'),
        )
    }

    def __init__(self, name, address, **kwargs):
        super().__init__(name, address, **kwargs)

        self.auto_connect = False
        self._battery = 1.0
        self.look_for_device = True

        self.query_timer = QtCore.QTimer(self)
        self.query_timer.timeout.connect(self.query_mobile)
        sleep_time_in_ms = 1000 * 5
        self.query_timer.start(sleep_time_in_ms)
        self._query_time = datetime.datetime.now()

        self.display_on = True
        osc_port_setting = DeviceLiveLinkFace.csettings['osc_port']
        osc_port_setting.signal_setting_changed.connect(
            lambda: self._check_recreate_osc_client())
        osc_port_setting.signal_setting_overridden.connect(
            lambda: self._check_recreate_osc_client())

    def setting_overrides(self):
        return super().setting_overrides() + [
            DeviceLiveLinkFace.csettings['osc_port'],
        ]

    @property
    def device_osc_port(self) -> int:
        return DeviceLiveLinkFace.csettings['osc_port'].get_value(self.name)

    @property
    def battery(self):
        return self._battery

    @battery.setter
    def battery(self, value):
        self._battery = value

        # Keep track of when the mobile battery was queried so the connection can time out
        self._query_time = datetime.datetime.now()

    def set_take(self, value):
        self.send_osc_message(osc.TAKE, value)

    def set_slate(self, value):
        self.send_osc_message(osc.SLATE, value)

    def connect_listener(self):
        # If the device was disconnected, set to closed
        if self.is_disconnected:
            self.status = DeviceStatus.CLOSED

        self.status = DeviceStatus.READY

    def disconnect_listener(self):
        super().disconnect_listener()

    def launch(self):
        unreal_devices = CONFIG.unreal_device_addresses()

        if not unreal_devices:
            LOGGER.warning(f'{self.name}: does not have an Unreal device to stream to. Please add one')
            self.status = DeviceStatus.DISCONNECTED
            return

        # Start the arkit session
        self.send_osc_message(osc.LIVE_LINK_SUBJECT, self.name)
        self.send_osc_message(osc.ARSESSION_START, 1)

        self.send_osc_message(osc.CLEAR_LIVE_LINK_ADDRESSES, 1)
        # Loop through all of the Unreal Addresses to add them
        for unreal_address in unreal_devices:
            self.send_osc_message(osc.ADD_LIVE_LINK_ADDRESS, [unreal_address, 11111])

        # Start the LiveLinkStream
        self.send_osc_message(osc.LIVE_LINK_STREAM_START, 1)

    def close(self):
        # Stop the arkit session
        self.send_osc_message(osc.ARSESSION_STOP, 1)

    def query_mobile(self):
        """
        Poll the mobile every 5 seconds by sending it two OSC commands osc.OSC_ADD_SEND_TARGET and osc.BATTERY_QUERY
        osc.OSC_ADD_SEND_TARGET gives the mobile an address to send OSC commands back to
        osc.BATTERY_QUERY send the battery % but also is used to make sure the app is still alive by resetting a timer
        The mobile will not be connected until it recieves a osc.BATTERY command
        """
        if self.status > DeviceStatus.DISCONNECTED and (datetime.datetime.now() - self._query_time) > datetime.timedelta(seconds=11):
            self.disconnect_listener()

        if self.look_for_device:
            if len(CONFIG.unreal_device_addresses()) > 0:
                if self.status > DeviceStatus.DISCONNECTED:
                    self.battery_query()
                else:
                    self.osc_add_send_target()
                    self.battery_query()
            else:
                LOGGER.warning(f'{self.name}: is trying to connect but the "Take Recorder" machine is not set in the Settings')

        if self.status == DeviceStatus.DELETE:
            self.query_timer.stop()

    # Send OSC message to ask for how much battery the phone has
    def battery_query(self):
        self.send_osc_message(osc.BATTERY_QUERY, 1, log=False)

    # Tell the mobile which address is running Switchboard
    def osc_add_send_target(self):
        self.send_osc_message(osc.OSC_ADD_SEND_TARGET, [SETTINGS.ADDRESS.get_value(), CONFIG.OSC_SERVER_PORT.get_value()], log=True)

    def osc_add_send_target_confirm(self):
        # If the mobile is left on and CP is relaunched the normal bootup sequence doesn't happen 
        if self.is_disconnected:
            #self.close() # Close the session if it was left open
            self.status = DeviceStatus.OPEN
            self.launch()

    def record_start(self, slate, take, description):
        if self.is_disconnected or not self.is_recording_device:
            return

        self.send_osc_message(osc.RECORD_START, [slate, take])

    #Toggle Display
    def toggle_display(self):
        if self.display_on:
            self.send_osc_message(osc.DISPLAY_OFF, 1)
            self.display_on = False
        else:
            self.send_osc_message(osc.DISPLAY_ON, 1)
            self.display_on = True


class DeviceWidgetLiveLinkFace(DeviceWidget):
    signal_device_widget_toggle_display = QtCore.Signal(object)

    def __init__(self, name, device_hash, address, icons, parent=None):
        super().__init__(name, device_hash, address, icons, parent=parent)

    def _add_control_buttons(self):
        super()._add_control_buttons()
        self.look_for_button = self.add_control_button(':/icons/images/icon_radar_disabled.png',
                                           icon_hover=':/icons/images/icon_radar_hover.png',
                                        icon_disabled=':/icons/images/icon_radar_disabled.png',
                                              icon_on=':/icons/images/icon_radar.png',
                                        icon_hover_on=':/icons/images/icon_radar_hover.png',
                                     icon_disabled_on=':/icons/images/icon_radar_disabled.png',
                                             tool_tip='Look for mobile',
                                             checkable=True, checked=True)

        self.disp_button = self.add_control_button(':/icons/images/icon_record_disabled.png',
                                           icon_hover=':/icons/images/icon_record_hover.png',
                                        icon_disabled=':/icons/images/icon_record_disabled.png',
                                              icon_on=':/icons/images/icon_record.png',
                                        icon_hover_on=':/icons/images/icon_record_hover.png',
                                     icon_disabled_on=':/icons/images/icon_record_disabled.png',
                                             tool_tip='Toggle Device Display',
                                             checkable=True, checked=True)


        self.look_for_button.clicked.connect(self.look_for_button_clicked)
        self.disp_button.clicked.connect(self.toggle_display)

    def set_battery(self, value):
        rounded_value = int(round(value * 100 / 10.0)) * 10

        qrc_path = f":/icons/images/icon_livelinkface_{rounded_value}.png"
        pixmap = QtGui.QPixmap(qrc_path)

        if pixmap.isNull():
            pixmap = self.icon_for_state("enabled").pixmap(QtCore.QSize(30, 30))

        self.device_icon.setPixmap(pixmap)

    def toggle_display(self, value):
        self.signal_device_widget_toggle_display.emit(self)
        

    def look_for_button_clicked(self):
        if self.look_for_button.isChecked():
            self.signal_device_widget_connect.emit(self)
        else:
            self.signal_device_widget_disconnect.emit(self)
