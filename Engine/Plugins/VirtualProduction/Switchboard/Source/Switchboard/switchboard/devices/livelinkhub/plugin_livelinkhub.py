# Copyright Epic Games, Inc. All Rights Reserved.

from __future__ import annotations

from enum import IntEnum
import pathlib
from typing import Callable
from uuid import UUID

from PySide6 import QtCore, QtWidgets

from switchboard import message_protocol
from switchboard.config import (
    CONFIG, Setting, BoolSetting, FilePathListSetting, FilePathSetting, FloatSetting, StringSetting)
from switchboard.devices.device_base import Device, DeviceStatus, PluginHeaderWidgets
from switchboard.devices.unreal.plugin_unreal import DeviceUnreal, DeviceWidgetUnreal, ProgramStartQueueItem
from switchboard.switchboard_logging import LOGGER


class LiveLinkHubTopologyMode(IntEnum):
    Hub = 0
    Spoke = 1


class DeviceLiveLinkHub(DeviceUnreal):
    LLH_PROG_NAME = 'livelinkhub'

    csettings: dict[str, Setting] = {
        'executable_name': StringSetting(
            attr_name='executable_name',
            nice_name='Live Link Hub executable filename',
            value='LiveLinkHub.exe',
            category='General Settings',
        ),
        'command_line_arguments': StringSetting(
            attr_name='command_line_arguments',
            nice_name='Command Line Arguments',
            value='',
            tool_tip='Additional command line arguments for Live Link Hub',
            category='General Settings',
        ),
        'session_path': FilePathSetting(
            attr_name='session_path',
            nice_name='Config File Path',
            value='',
            tool_tip='Path to the Live Link Hub session file to load on startup',
            category='General Settings',
            file_path_filter="JSON files (*.json);;All Files (*)",
        ),
        'retrieve_logs': BoolSetting(
            attr_name='retrieve_logs',
            nice_name='Retrieve Logs',
            value=True,
            tool_tip='When checked, retrieves logs after Live Link Hub terminates',
            category='General Settings',
        ),
        'soft_kill_timeout': FloatSetting(
            attr_name='soft_kill_timeout',
            nice_name='Soft Kill Timeout',
            value=3.0,
            tool_tip=(
                'Timeout in seconds for graceful process shutdown before force termination. '
                'Live Link Hub processes have this amount of time to save state and clean up before being forcefully killed.'
            ),
            category='General Settings',
            show_ui=False,  # For now, avoiding the extra clutter.
        ),
        'disable_crash_recovery': BoolSetting(
            attr_name='disable_crash_recovery',
            nice_name='Disable Crash Recovery',
            value=False,
            tool_tip='Stops Live Link Hub from prompting to recover after an unclean shutdown.',
            category='General Settings',
        ),
        'vendor_plugin_paths': FilePathListSetting(
            attr_name='vendor_plugin_paths',
            nice_name='Vendor Plugin Paths',
            value=[],
            tool_tip=(
                'List of vendor / third-party .uplugin files to compile into LiveLinkHub '
                'and enable at runtime. Paths may be absolute or relative to the engine '
                'workspace root. See InstalledLiveLinkHubBuild.xml for examples.'
            ),
            category='General Settings',
            file_path_filter='Unreal Plugin (*.uplugin);;All Files (*)',
        ),
        'port': DeviceUnreal.csettings['port'],
    }

    def __init__(self, name, address, *args, **kwargs):
        super().__init__(name, address, *args, **kwargs)

        self.widget: DeviceWidgetLiveLinkHub
        self.setting_spoke_mode = BoolSetting(
            attr_name='spoke_mode',
            nice_name='Spoke Mode',
            value=kwargs.pop('spoke_mode', False),
            tool_tip='Whether to start this instance of Live Link Hub in "spoke" topology mode',
        )

    @property
    def llh_proj_dir(self) -> pathlib.Path:
        ''' Returns the directory containing LiveLinkHub.uproject (and logs, etc). '''
        engine_dir = pathlib.Path(CONFIG.ENGINE_DIR.get_value(self.name))
        return engine_dir / 'Source' / 'Programs' / 'LiveLinkHub'

    def _resolved_vendor_plugin_paths(self) -> list[str]:
        '''
        Returns the configured vendor plugin paths with empty entries removed.
        Relative paths are resolved against the workspace root (parent of ENGINE_DIR),
        matching the convention used by InstalledLiveLinkHubBuild.xml.
        '''
        raw_paths = DeviceLiveLinkHub.csettings['vendor_plugin_paths'].get_value(self.name) or []
        workspace_root = pathlib.Path(CONFIG.ENGINE_DIR.get_value(self.name)).parent

        resolved: list[str] = []
        for entry in raw_paths:
            entry = (entry or '').strip()
            if not entry:
                continue
            path = pathlib.Path(entry)
            if not path.is_absolute():
                path = workspace_root / path
            resolved.append(str(path))
        return resolved

    #@override
    @classmethod
    def plugin_settings(cls):
        return list(cls.csettings.values())

    #@override
    def device_settings(self):
        return Device.device_settings(self) + [
            self.setting_spoke_mode
        ]

    #@override
    def setting_overrides(self):
        return [
            DeviceUnreal.csettings['udpmessaging_unicast_endpoint'],
            DeviceUnreal.csettings['udpmessaging_extra_static_endpoints'],
            DeviceLiveLinkHub.csettings['command_line_arguments'],
            DeviceLiveLinkHub.csettings['session_path'],
            DeviceLiveLinkHub.csettings['vendor_plugin_paths'],
            CONFIG.ENGINE_DIR,
            CONFIG.SOURCE_CONTROL_WORKSPACE,
        ]

    #@override
    @classmethod
    def plugin_header_widget_config(cls):
        return super().plugin_header_widget_config() & ~PluginHeaderWidgets.AUTOJOIN_MU

    #@override
    def device_widget_registered(self, device_widget: DeviceWidgetLiveLinkHub):
        super().device_widget_registered(device_widget)

        # Reflect current mode in the widget
        if self.setting_spoke_mode.get_value():
            current_mode = LiveLinkHubTopologyMode.Spoke
        else:
            current_mode = LiveLinkHubTopologyMode.Hub

        device_widget.mode_dropdown.setCurrentIndex(current_mode)

        # Respond to mode changes via the widget
        device_widget.signal_device_widget_spoke_mode.connect(
            lambda new_is_spoke: self.setting_spoke_mode.update_value(new_is_spoke)
        )

    #@override
    @property
    def executable_filename(self):
        return DeviceLiveLinkHub.csettings['executable_name'].get_value()

    #@override
    @property
    def extra_cmdline_args_setting(self):
        return DeviceLiveLinkHub.csettings['command_line_arguments'].get_value(self.name)

    @property
    def session_path(self):
        return DeviceLiveLinkHub.csettings['session_path'].get_value(self.name)

    #@override
    def setup_osc_client(self, osc_port):
        ''' Live Link Hub devices should not send or receive OSC '''
        return

    #@override
    def send_osc_message(self, command, value, log=True):
        ''' Live Link Hub devices should not send or receive OSC '''
        return

    #@override
    def get_remote_log_path(self):
        return self.llh_proj_dir / 'Saved' / 'Logs'

    #@override
    def generate_unreal_command_line_args(self, map_name: str):
        command_line_args: list[str] = []
        command_line_args.append(f'{self.extra_cmdline_args_setting}')

        command_line_args.append(f'Log={self.log_filename}')

        if self.setting_spoke_mode.get_value() == False:
            command_line_args.extend(['-hub',
                                      '-UDPMESSAGING_SHARE_KNOWN_NODES=1'])
        else:
            command_line_args.append('-spoke')

        # UdpMessaging endpoints
        command_line_args.extend(self.build_udpmessaging_args())

        if self.session_path:
            command_line_args.append(f' -SessionPath="{self.session_path}"')

        # Enable any user-specified vendor plugins at runtime. LiveLinkHub disables
        # engine plugins by default, so plugins compiled in via -Plugin=<path> at
        # build time still need to be explicitly enabled when launching.
        vendor_plugin_paths = self._resolved_vendor_plugin_paths()
        if vendor_plugin_paths:
            plugin_names = ','.join(pathlib.Path(p).stem for p in vendor_plugin_paths)
            command_line_args.append(f'-EnablePlugins={plugin_names}')

        if DeviceLiveLinkHub.csettings['disable_crash_recovery'].get_value():
            command_line_args.append(
                ' -ini:Engine:'
                '[/Script/LiveLinkHub.LiveLinkHubSettings]:'
                'bEnableCrashRecovery=false'
            )

        return ' '.join(command_line_args)

    #@override
    def launch(self, map_name: str):
        map_name = ''

        exe_path, args = self.generate_unreal_command_line(map_name)
        LOGGER.info(f"Launching Live Link Hub: {exe_path} {args}")
        self.last_launch_command.update_value(f'{exe_path} {args}')

        puuid, msg = message_protocol.create_start_process_message(
            prog_path=exe_path,
            prog_args=args,
            prog_name=self.LLH_PROG_NAME,
            caller=self.name,
            update_clients_with_stdout=False,
        )

        self.program_start_queue.add(
            ProgramStartQueueItem(
                name=self.LLH_PROG_NAME,
                puuid_dependency=None,
                puuid=puuid,
                msg_to_unreal_client=msg,
            ),
            unreal_client=self.unreal_client,
        )

    #@override
    def close(self, force=False, soft_kill_timeout=None):
        llh_puuids = self.program_start_queue.running_puuids_named(self.LLH_PROG_NAME)

        # Get soft kill timeout for this device (use provided value or setting)
        if soft_kill_timeout is None:
            soft_kill_timeout = DeviceLiveLinkHub.csettings['soft_kill_timeout'].get_value(self.name)

        if not llh_puuids:
            self.status = DeviceStatus.CLOSED
        else:
            self.status = DeviceStatus.CLOSING
            for llh_puuid in llh_puuids:
                _, msg = message_protocol.create_kill_process_message(llh_puuid, soft_kill_timeout)
                self.unreal_client.send_message(msg)

    #@override
    def do_program_running_update(self, prog: ProgramStartQueueItem):
        super().do_program_running_update(prog)

        if prog.name == self.LLH_PROG_NAME:
            self.status = DeviceStatus.OPEN

    #@override
    def do_program_ended_update(
        self,
        *,
        program_name: str,
        returncode: int,
        get_stdout_str: Callable[[], str],
        get_stderr_str: Callable[[], str],
    ):
        super().do_program_ended_update(
            program_name=program_name,
            returncode=returncode,
            get_stdout_str=get_stdout_str,
            get_stderr_str=get_stderr_str
        )

        if program_name == self.LLH_PROG_NAME:
            self.status = DeviceStatus.CLOSED
            if DeviceLiveLinkHub.csettings['retrieve_logs'].get_value():
                self.start_retrieve_log(returncode)

    #@override
    def _request_roles_file(self):
        # Not relevant to our device type.
        return

    #@override
    def _queue_all_builds(
        self,
        requesting_device: DeviceUnreal,
        puuid_dependency: UUID | None = None,
    ) -> UUID | None:
        ubt_args = f'LiveLinkHub {requesting_device.target_platform} Development -Progress'
        # Enable additional plugins by name (matches the -EnablePlugin=<Name> pattern
        # used in InstalledLiveLinkHubBuild.xml). Note: do NOT use UBT's -Plugin=<path>
        # form here — that puts UBT into single foreign-plugin build mode and replaces
        # the target's EnablePlugins list, which would drop all the LiveLink* and
        # MetaHumanLiveLink* modules baked into LiveLinkHub.Target.cs.
        for plugin_path in self._resolved_vendor_plugin_paths():
            plugin_name = pathlib.Path(plugin_path).stem
            ubt_args += f' -EnablePlugin={plugin_name}'
        return requesting_device._queue_build('livelinkhub', ubt_args, puuid_dependency)


class DeviceWidgetLiveLinkHub(DeviceWidgetUnreal):
    signal_device_widget_spoke_mode = QtCore.Signal(bool)

    def __init__(self, *args, **kwargs):
        super().__init__(*args, **kwargs)

        self.mode_dropdown: QtWidgets.QComboBox

    #@override
    @property
    def app_name(self):
        return 'Live Link Hub'

    #@override
    def _add_control_buttons(self):
        self._autojoin_visible = False
        super()._add_control_buttons()

    #@override
    def add_widget_to_layout(self, widget):
        if widget == self.name_line_edit:
            self._add_mode_dropdown()

            # shorten the widget to account for the inserted dropdown
            added_width = (
                self.mode_dropdown.minimumSize().width()
                + 2 * self.layout.spacing()
                + self.mode_dropdown.contentsMargins().left()
                + self.mode_dropdown.contentsMargins().right())

            le_maxwidth = self.name_line_edit.maximumWidth() - added_width
            self.name_line_edit.setMaximumWidth(le_maxwidth)

        super().add_widget_to_layout(widget)

    #@override
    def populate_context_menu(self, cmenu: QtWidgets.QMenu):
        cmenu.addAction('Include in build' if self.exclude_from_build else 'Exclude from build',
                        lambda: self.signal_exclude_from_build_toggled.emit())
        cmenu.addAction('Open fetched log', lambda: self.signal_open_last_log.emit(self))
        cmenu.addAction('Copy last launch command', lambda: self.signal_copy_last_launch_command.emit(self))

    #@override
    def _update_connected_ui(self):
        super()._update_connected_ui()
        self.mode_dropdown.setDisabled(False)

    #@override
    def _update_disconnected_ui(self):
        super()._update_disconnected_ui()
        self.mode_dropdown.setDisabled(True)

    def _add_mode_dropdown(self):
        '''
        Adds to the layout a dropdown to select which topology mode the device should be in.
        '''

        self.mode_dropdown = QtWidgets.QComboBox(self)
        self.mode_dropdown.setFixedSize(40, 32)
        self.mode_dropdown.view().setFixedWidth(100)
        for mode in LiveLinkHubTopologyMode:
            self.mode_dropdown.addItem(mode.name)
            self.mode_dropdown.setItemData(mode.value, self.icons[f'{mode.name}Mode'],
                                           QtCore.Qt.ItemDataRole.DecorationRole)

        self.add_widget_to_layout(self.mode_dropdown)

        self.mode_dropdown.currentIndexChanged.connect(self._on_mode_dropdown_current_index_changed)

    def _on_mode_dropdown_current_index_changed(self, idx: int):
        self.signal_device_widget_spoke_mode.emit(idx == LiveLinkHubTopologyMode.Spoke)
