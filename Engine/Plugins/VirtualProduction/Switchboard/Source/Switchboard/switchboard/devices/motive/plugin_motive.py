# Copyright Epic Games, Inc. All Rights Reserved.

from collections import deque
import datetime
import select
import socket
import struct
from threading import Thread
import time

from switchboard.config import IntSetting
from switchboard.devices.device_base import Device, DeviceStatus
from switchboard.devices.device_widget_base import DeviceWidget
from switchboard.switchboard_logging import LOGGER
import switchboard.switchboard_utils as utils


class DeviceMotive(Device):
    NAT_CONNECT = 0
    NAT_SERVERINFO = 1
    NAT_REQUEST = 2
    NAT_RESPONSE = 3
    NAT_REQUEST_MODELDEF = 4
    NAT_MODELDEF = 5
    NAT_REQUEST_FRAMEOFDATA = 6
    NAT_FRAMEOFDATA = 7
    NAT_MESSAGESTRING = 8
    NAT_DISCONNECT = 9
    NAT_KEEPALIVE = 10
    NAT_DISCONNECTBYTIMEOUT = 11
    NAT_ECHOREQUEST = 12
    NAT_ECHORESPONSE = 13
    NAT_DISCOVERY = 14
    NAT_UNRECOGNIZED_REQUEST = 100

    setting_motive_port = IntSetting(
        "motive_port", "Motive Command Port", 1510)

    def __init__(self, name, address, **kwargs):
        super().__init__(name, address, **kwargs)

        self.trigger_start = True
        self.trigger_stop = True

        self.client = None

        self._slate = 'slate'
        self._take = 1

        # Stores pairs of (queued message, command name).
        self.message_queue = deque()

        self.socket = None
        self.close_socket = False
        self.last_activity = datetime.datetime.now()
        self.awaiting_echo_response = False
        self.command_response_callbacks = {
            "Connect": self.on_motive_connect_response,
            "Disconnect": lambda _: None,
            "Echo": self.on_motive_echo_response,
            "SetRecordTakeName": self.on_motive_record_take_name_set,
            "StartRecording": self.on_motive_recording_started,
            "StopRecording": self.on_motive_recording_stopped}
        self.motive_connection_thread = None

    @staticmethod
    def plugin_settings():
        return Device.plugin_settings() + [DeviceMotive.setting_motive_port]

    def send_request_to_motive(self, request, *args):
        """ Sends a request message to Motive's command port. """
        message_str = request
        if args:
            message_str += ',' + ','.join(args)

        message_str_len = len(message_str) + 1
        data = struct.pack(
            f"HH{message_str_len}s", self.NAT_REQUEST,
            4 + message_str_len, message_str.encode('utf-8'))
        self.message_queue.appendleft((data, request))

    def send_echo_request(self):
        """
        Sends an echo request to Motive if not already waiting for an echo
        response.
        """
        if self.awaiting_echo_response:
            return

        self.awaiting_echo_response = True

        message = self.NAT_ECHOREQUEST
        message_str = "Ping"
        packet_size = len(message_str) + 1

        data = message.to_bytes(2, byteorder='little')
        data += packet_size.to_bytes(2, byteorder='little')

        data += message_str.encode('utf-8')
        data += b'\0'
        self.message_queue.appendleft((data, "Echo"))

    def on_motive_echo_response(self, response):
        """
        Callback that is exectued when Motive has responded to an echo request.
        """
        self.awaiting_echo_response = False

    @property
    def is_connected(self):
        return self.socket is not None

    def connect_listener(self):
        """ Connect to Motive's socket """
        self.close_socket = False

        self.socket = socket.socket(socket.AF_INET, socket.SOCK_DGRAM)  # UDP
        self.socket.bind(('', 0))

        self.last_activity = datetime.datetime.now()

        self.motive_connection_thread = Thread(target=self.motive_connection)
        self.motive_connection_thread.start()

        self.awaiting_echo_response = False

        connect_message = struct.pack("HH", self.NAT_CONNECT, 0)
        self.message_queue.appendleft((connect_message, "Connect"))

    def on_motive_connect_response(self, response):
        """
        Callback executed when a response to a connect message is received.
        """
        message_id = int.from_bytes(response[0:2], byteorder='little')
        if message_id != self.NAT_SERVERINFO:
            LOGGER.error(f"Unexpected connect response {message_id}")
            return

        # Because the socket is UDP, we can only be sure that a "connection"
        # has been established when the connect response has beeen received.
        # That's why the connection state is updated here instead of in
        # connect_listener().
        if self.is_disconnected:
            self.status = DeviceStatus.CLOSED

    def disconnect_listener(self):
        """ Disconnect from Motive """
        if self.is_connected:
            disconnect_message = struct.pack("HH", self.NAT_DISCONNECT, 0)
            self.message_queue.appendleft((disconnect_message, "Disconnect"))

            self.close_socket = True
            self.motive_connection_thread.join()

            # updates device state
            super().disconnect_listener()

    def motive_connection(self):
        """
        Thread procedure for socket connection between Switchboard and Motive.
        """
        ping_interval = 1.0
        disconnect_timeout = 3.0

        while self.is_connected:
            try:
                rlist = [self.socket]
                wlist = []
                xlist = []
                read_timeout = 0.2

                if len(self.message_queue):
                    message_bytes, cmd_name = self.message_queue.pop()

                    self._flush_read_sockets()
                    self.socket.sendto(
                        message_bytes,
                        (self.address,
                         self.setting_motive_port.get_value()))

                    read_sockets, _, _ = select.select(
                        rlist, wlist, xlist, read_timeout)
                    for rs in read_sockets:
                        received_data = rs.recv(4096)
                        self.process_message(received_data, cmd_name)
                else:
                    time.sleep(0.01)

                activity_delta = datetime.datetime.now() - self.last_activity

                if activity_delta.total_seconds() > disconnect_timeout:
                    raise Exception("Connection timeout")
                elif activity_delta.total_seconds() > ping_interval:
                    self.send_echo_request()

                if self.close_socket and len(self.message_queue) == 0:
                    self.socket.shutdown(socket.SHUT_RDWR)
                    self.socket.close()
                    self.socket = None
                    break

            except Exception as e:
                LOGGER.warning(f"{self.name}: Disconnecting due to: {e}")
                self.device_qt_handler.signal_device_client_disconnected.emit(
                    self)
                break

    def _flush_read_sockets(self):
        """ Flushes all sockets used for receiving data. """
        rlist = [self.socket]
        wlist = []
        xlist = []
        read_timeout = 0
        read_sockets, _, _ = select.select(rlist, wlist, xlist, read_timeout)
        for rs in read_sockets:
            orig_timeout = rs.gettimeout()
            rs.settimeout(0)
            data = b'foo'
            try:
                while len(data) > 0:
                    data = rs.recv(32768)
            except socket.error:
                pass
            finally:
                rs.settimeout(orig_timeout)

    def process_message(self, data, cmd_name):
        """ Processes incoming messages sent by Motive. """

        self.last_activity = datetime.datetime.now()
        message_id = int.from_bytes(data[0:2], byteorder='little')

        if message_id in (
                self.NAT_SERVERINFO, self.NAT_RESPONSE, self.NAT_ECHORESPONSE):
            if cmd_name in self.command_response_callbacks:
                self.command_response_callbacks[cmd_name](data)
            else:
                LOGGER.error(
                    f"{self.name}: Could not find callback for "
                    f"{cmd_name} request")
                assert(False)
        elif message_id == self.NAT_UNRECOGNIZED_REQUEST:
            LOGGER.error(
                f"{self.name}: Server did not recognize {cmd_name} request")
            assert(False)

    def set_slate(self, value):
        """ Notify Motive when slate name was changed. """
        self._slate = value
        self.send_request_to_motive(
            "SetRecordTakeName", utils.capture_name(self._slate, self._take))

    def set_take(self, value):
        """ Notify Motive when Take number was changed. """
        self._take = value
        self.send_request_to_motive(
            "SetRecordTakeName", utils.capture_name(self._slate, self._take))

    def on_motive_record_take_name_set(self, response):
        """ Callback that is executed when the take name was set in Motive. """
        pass

    def record_start(self, slate, take, description):
        """
        Called by switchboard_dialog when recording was started, will start
        recording in Motive.
        """
        if self.is_disconnected or not self.trigger_start:
            return

        self.set_slate(slate)
        self.set_take(take)

        self.send_request_to_motive('StartRecording')

    def on_motive_recording_started(self, response):
        """ Callback that is exectued when Motive has started recording. """
        self.record_start_confirm(self.timecode())

    def record_stop(self):
        """
        Called by switchboard_dialog when recording was stopped, will stop
        recording in Motive.
        """
        if self.is_disconnected or not self.trigger_stop:
            return

        self.send_request_to_motive('StopRecording')

    def on_motive_recording_stopped(self, response):
        """ Callback that is exectued when Motive has stopped recording. """
        self.record_stop_confirm(self.timecode(), paths=None)

    def timecode(self):
        return '00:00:00:00'


class DeviceWidgetMotive(DeviceWidget):
    def __init__(self, name, device_hash, address, icons, parent=None):
        super().__init__(name, device_hash, address, icons, parent=parent)

    def _add_control_buttons(self):
        super()._add_control_buttons()
        self.trigger_start_button = self.add_control_button(
            ':/icons/images/icon_trigger_start_disabled.png',
            icon_hover=':/icons/images/icon_trigger_start_hover.png',
            icon_disabled=':/icons/images/icon_trigger_start_disabled.png',
            icon_on=':/icons/images/icon_trigger_start.png',
            icon_hover_on=':/icons/images/icon_trigger_start_hover.png',
            icon_disabled_on=':/icons/images/icon_trigger_start_disabled.png',
            tool_tip='Trigger when recording starts',
            checkable=True, checked=True)

        self.trigger_stop_button = self.add_control_button(
            ':/icons/images/icon_trigger_stop_disabled.png',
            icon_hover=':/icons/images/icon_trigger_stop_hover.png',
            icon_disabled=':/icons/images/icon_trigger_stop_disabled.png',
            icon_on=':/icons/images/icon_trigger_stop.png',
            icon_hover_on=':/icons/images/icon_trigger_stop_hover.png',
            icon_disabled_on=':/icons/images/icon_trigger_stop_disabled.png',
            tool_tip='Trigger when recording stops',
            checkable=True, checked=True)

        self.connect_button = self.add_control_button(
            ':/icons/images/icon_connect.png',
            icon_hover=':/icons/images/icon_connect_hover.png',
            icon_disabled=':/icons/images/icon_connect_disabled.png',
            icon_on=':/icons/images/icon_connected.png',
            icon_hover_on=':/icons/images/icon_connected_hover.png',
            icon_disabled_on=':/icons/images/icon_connected_disabled.png',
            tool_tip='Connect/Disconnect from listener')

        self.trigger_start_button.clicked.connect(self.trigger_start_clicked)
        self.trigger_stop_button.clicked.connect(self.trigger_stop_clicked)
        self.connect_button.clicked.connect(self.connect_button_clicked)

        # Disable the buttons
        self.trigger_start_button.setDisabled(True)
        self.trigger_stop_button.setDisabled(True)

    def trigger_start_clicked(self):
        if self.trigger_start_button.isChecked():
            self.signal_device_widget_trigger_start_toggled.emit(self, True)
        else:
            self.signal_device_widget_trigger_start_toggled.emit(self, False)

    def trigger_stop_clicked(self):
        if self.trigger_stop_button.isChecked():
            self.signal_device_widget_trigger_stop_toggled.emit(self, True)
        else:
            self.signal_device_widget_trigger_stop_toggled.emit(self, False)

    def connect_button_clicked(self):
        if self.connect_button.isChecked():
            self._connect()
        else:
            self._disconnect()

    def _connect(self):
        # Make sure the button is in the correct state
        self.connect_button.setChecked(True)

        # Enable the buttons
        self.trigger_start_button.setDisabled(False)
        self.trigger_stop_button.setDisabled(False)

        # Emit Signal to Switchboard
        self.signal_device_widget_connect.emit(self)

    def _disconnect(self):
        # Make sure the button is in the correct state
        self.connect_button.setChecked(False)

        # Disable the buttons
        self.trigger_start_button.setDisabled(True)
        self.trigger_stop_button.setDisabled(True)

        # Emit Signal to Switchboard
        self.signal_device_widget_disconnect.emit(self)
