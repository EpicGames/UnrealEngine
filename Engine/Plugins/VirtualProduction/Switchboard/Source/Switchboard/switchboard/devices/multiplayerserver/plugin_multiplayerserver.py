# Copyright Epic Games, Inc. All Rights Reserved.

from switchboard.devices.unreal.plugin_unreal import DeviceUnreal, DeviceWidgetUnreal
from switchboard.config import (CONFIG, FilePathSetting, StringSetting,
                                IntSetting, get_game_launch_level_path)


class DeviceMultiplayerServer(DeviceUnreal):
    settings = {
        'executable_path': FilePathSetting(
            attr_name="executable_path",
            nice_name="Executable Path",
            value=""
        ),
        'level_path': StringSetting(
            attr_name="level_path",
            nice_name="Level Path",
            value=""
        ),
        'multiplayer_port': IntSetting(
            attr_name="multiplayer_port",
            nice_name="Multiplayer Port",
            value=7777,
            tool_tip="Port to use for multiplayer connections"
        )
    }

    def __init__(self, name, address, **kwargs):

        super().__init__(name, address, **kwargs)

    def generate_unreal_exe_path(self):
        exe_path = self.settings['executable_path'].get_value()
        if exe_path == '':
            return super().generate_unreal_exe_path()

        return exe_path

    def generate_unreal_command_line_args(self, map_name):
        command_line_args = ''
        uproject_path = CONFIG.UPROJECT_PATH.get_value()

        custom_level_path = self.settings['level_path'].get_value().strip()
        level_path = custom_level_path if custom_level_path != '' else get_game_launch_level_path()

        if self.settings['executable_path'].get_value() == '':
            command_line_args = f'{uproject_path} {level_path} -server'
        else:
            command_line_args = level_path

        command_line_args += f' -log -port={self.settings["multiplayer_port"].get_value()}'

        return command_line_args

    @classmethod
    def plugin_settings(cls):
        return list(cls.settings.values()) + [DeviceUnreal.csettings['port']]

    def device_settings(self):
        return [self.setting_address]

    def setting_overrides(self):
        return []


class DeviceWidgetMultiplayerServer(DeviceWidgetUnreal):
    def __init__(self, name, device_type, device_hash, address, parent=None):

        super().__init__(name, device_type, device_hash, address, parent=parent)

        self.autojoin_mu._button.setVisible(False)
        self.build_button.setDisabled(True)
        self.build_button.setVisible(False)
