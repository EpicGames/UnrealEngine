# Copyright Epic Games, Inc. All Rights Reserved.

from collections import OrderedDict
from itertools import count, islice
import time
import threading
import traceback

from PySide6 import QtCore
from PySide6.QtCore import Qt, QTimer, Q_ARG
from PySide6.QtGui import QColor, QIcon, QStandardItemModel, QStandardItem

from switchboard import message_protocol
from switchboard.message_protocol import SyncStatusRequestFlags
from switchboard.switchboard_logging import LOGGER
from switchboard.devices.device_base import Device
from switchboard.devices.unreal.plugin_unreal import UnrealJobs


class nDisplayMonitor(QStandardItemModel):
    '''
    This will monitor the status of the nDisplay nodes, in particular regarding
    sync. It polls the listener at the specified rate and the UI should update
    with this info.
    '''

    BG_COLOR_WARNING = QColor(0x70, 0x40, 0x00)
    BG_COLOR_NORMAL = QColor(0x3d, 0x3d, 0x3d)
    FG_COLOR_DISCONNECTED = QColor(0x7F, 0x7F, 0x7F)
    FG_COLOR_NORMAL = QColor(0xd8, 0xd8, 0xd8)
    CORE_OVERLOAD_THRESH = 90  # percent utilization

    # Special meaning when received from listener for PresentMode, but
    # sometimes also used for other values' display for consistency.
    DATA_MISSING = 'n/a'

    def __init__(self, parent):
        QStandardItemModel.__init__(self, parent)

        # If false, the button to disable full screen optimizations is hidden
        # and periodic polling of the state of this feature is disabled.
        self.show_disable_fso_btn = False

        # Determines if GPU stats are regularly queried or not.
        self.poll_gpu_stats = False

        # The time it takes to round trip the cluster and poll the status of the same device
        # Used to detect stale devices.
        self.roundtrip_polling_period_ms = 0

        # Keeps track of the round-robin polling of device sync status
        self._next_roundrobin_poll_deviceIdx = 0

        # ordered so that we can map row indices to devices
        self.devicedatas = OrderedDict()

        self.timer = QTimer(self)
        self.timer.timeout.connect(self.poll_variable_sync_status)

        HEADER_DATA = {
            'Connected': 'If we are connected to the listener of this device',
            'Node': 'The cluster name of this device',
            'Host': 'The URL of the remote PC',
            'Driver': 'GPU driver version',
            'PresentMode':
                'Current presentation mode. Only available once the render '
                'node process is running. Expects "Hardware Composed: '
                'Independent Flip"',
            'Gpus': 'Informs if GPUs are synced.',
            'Displays': 'Detected displays and whether they are in sync',
            'SyncRate': 'Sync Frame Rate',
            'HouseSync':
                'Presence of an external sync signal connected to the remote '
                'Quadro Sync card',
            'SyncSource': 'The source of the GPU sync signal',
            'Mosaics': 'Display grids and their resolutions',
            'Taskbar':
                'Whether the taskbar is set to auto hide or always on top. It '
                'is recommended to be consistent across the cluster',
            'InFocus':
                'Whether nDisplay instance window is in focus. It is '
                'recommended to be in focus.',
            'CpuUtilization':
                'CPU utilization average. The number of overloaded cores (>'
                f'{self.CORE_OVERLOAD_THRESH}% load) will be displayed in '
                'parentheses.\n\n'
                'in addition, simultaneous multi-threading (SMT), also known\n'
                'as Hyper-Threading, is known to potentially cause hitches.\n'
                'If SMT is enabled, that will also be flagged as a warning.',
            'MemUtilization': 'Physical memory, utilized / total.',
            'GpuUtilization':
                'GPU utilization. The GPU clock speed is displayed in '
                'parentheses.',
            'GpuTemperature':
                'GPU temperature in degrees celsius. (Max across all '
                'sensors.)',
            'FSO':
                'It is recommended to disable Full Screen Optimizations in the\n'
                'unreal executable because it has been associated with tearing.\n'
                'Only available once the render node process is running. \n'
                '(Expected value is "no")',
            'OsVer': 'Operating system version',
        }

        self.colnames = list(HEADER_DATA.keys())
        self.colnamesdisplay = ["" if name == 'Connected' else name for name in self.colnames]
        self.tooltips = list(HEADER_DATA.values())

        # Load connection status icons
        self.icon_unconnected = QIcon(":/icons/images/status_blank_disabled.png")
        self.icon_connected = QIcon(":/icons/images/status_cyan.png")
        self.icon_running = QIcon(":/icons/images/status_orange.png")

    def color_for_column(self, colname, value, data, is_program_running: bool):
        ''' Returns the background color for the given cell '''
        if data['Connected'].lower() == 'no':
            return self.BG_COLOR_NORMAL

        if colname == 'PresentMode':
            ok_values = [
                'Hardware Composed: Independent Flip',
                'Hardware: Independent Flip',
            ]
            is_good = (not is_program_running) or any(ok_value in value for ok_value in ok_values)
            return self.BG_COLOR_NORMAL if is_good else self.BG_COLOR_WARNING

        if colname == 'Gpus':
            is_synced = ('Synced' in value) and ('Free' not in value)
            return self.BG_COLOR_NORMAL if is_synced else self.BG_COLOR_WARNING

        if colname == 'InFocus':
            is_good = (not is_program_running) or ('yes' in value)
            return self.BG_COLOR_NORMAL if is_good else self.BG_COLOR_WARNING

        if colname == 'FSO':
            is_good = (not is_program_running) or ('no' in value)
            return self.BG_COLOR_NORMAL if is_good else self.BG_COLOR_WARNING

        if colname == 'Displays':
            is_normal = (('Follower' in value or 'Leader' in value)
                         and ('Unsynced' not in value))
            return self.BG_COLOR_NORMAL if is_normal else self.BG_COLOR_WARNING

        if colname == 'CpuUtilization':
            # "(# cores > threshold%)" or "(SMT ENABLED)"
            no_caveats = '(' not in value
            return self.BG_COLOR_NORMAL if no_caveats else self.BG_COLOR_WARNING

        return self.BG_COLOR_NORMAL

    def friendly_osver(self, device):
        ''' Returns a display-friendly string for the device OS version '''
        if device.os_version_label.startswith('Windows'):
            try:
                # Destructuring according to FWindowsPlatformMisc::GetOSVersion
                # (ex: "10.0.19041.1.256.64bit")
                [major, minor, build, product_type, suite_mask, arch] = \
                    device.os_version_number.split('.')

                if major == '10' and minor == '0':
                    # FWindowsPlatformMisc::GetOSVersions (with an s) returns a
                    # label that includes this "release ID"/"SDK version", but
                    # the mechanism for retrieving it seems fragile.
                    # Based on: https://docs.microsoft.com/en-us/windows/release-health/release-information  # noqa
                    BUILD_TO_SDK_VERSION = {
                        '19044': '21H2',
                        '19043': '21H1',
                        '19042': '20H2',
                        '19041': '2004',
                        '18363': '1909',
                        '17763': '1809',
                        '17134': '1803',
                        '14393': '1607',
                        '10240': '1507',
                    }

                    sdk_version = BUILD_TO_SDK_VERSION.get(build)
                    if sdk_version:
                        return f'Windows 10, version {sdk_version}'

            except ValueError:
                # Mismatched count of splits vs. destructure
                pass  # Fall through to default

        # Default / fallback
        friendly = device.os_version_label

        if device.os_version_label_sub != '':
            friendly += " " + device.os_version_label_sub

        if device.os_version_number != '':
            friendly += " " + device.os_version_number

        if friendly == '':
            friendly = self.DATA_MISSING

        return friendly

    def reset_device_data(self, device, data):
        ''' Sets device data to unconnected state '''
        for colname in self.colnames:
            data[colname] = self.DATA_MISSING

        data['Host'] = str(device.address)
        data['Node'] = device.name
        data['Connected'] = \
            'yes' if device.is_connected_and_authenticated() else 'no'

        # extra data not in columns
        data['TimeLastFlipGlitch'] = time.time()

    def added_device(self, device):
        ''' Called by the plugin when a new nDisplay device has been added. '''
        data = {}

        self.reset_device_data(device, data)
        self.devicedatas[device.device_hash] = {
            'device': device, 'data': data, 'time_last_update': 0,
            'stale': True
        }

        # notify the UI of the change
        self.rebuild_table()

        # Adding a device may change our polling strategy
        self.update_polling_timer()

    def get_col_count(self) -> int:
        ''' Returns the number of columns that the table should have.'''
        return len(self.colnames)

    def get_row_count(self) -> int:
        ''' Returns the number of rows that the table should have.
        It is based on the number of devices that have been added.
        '''
        return len(self.devicedatas)

    def rebuild_table(self) -> None:
        ''' Rebuilds all the table items from scratch. '''
        self.setColumnCount(self.get_col_count())
        self.setRowCount(self.get_row_count())
        self.rebuild_headers()

        # Update the row data
        for row in range(self.get_row_count()):
            self.refresh_display_for_row(row)

    def rebuild_headers(self) -> None:
        ''' Repopulates from scratch the header items.'''
        # set header labels
        self.setHorizontalHeaderLabels(self.colnamesdisplay)

        # set tooltips
        for col in range(self.get_col_count()):
            self.setHeaderData(
                col,
                Qt.Orientation.Horizontal,
                self.tooltips[col],
                Qt.ItemDataRole.ToolTipRole
            )

    def removed_device(self, device):
        ''' Called by the plugin when an nDisplay device has been removed. '''
        self.devicedatas.pop(device.device_hash)

        self.rebuild_table()  # To reflect this in the UI
        self.update_polling_timer()  # This may change our polling strategy

    def update_polling_timer(self) -> None:
        ''' Re-calculates what the polling timer should be set to.'''

        devs = [devicedata['device'] for devicedata in self.devicedatas.values()]

        # No need for a timer if there aren't any devices in the table
        if len(devs) == 0:
            self.timer.stop()
            return

        num_pollable_nodes = sum(1 for dev in devs if self.can_poll_device(dev))

        # turn off the timer if there are no devices left
        if num_pollable_nodes:
            min_ms_between_nodes = 250  # Fastest allowed update rate per node
            min_ms_roundtrip = 1000     # Fastest allowed update rate of the same node

            polling_period_ms = max(min_ms_roundtrip / num_pollable_nodes, min_ms_between_nodes)

            self.roundtrip_polling_period_ms = num_pollable_nodes * polling_period_ms

            # start/continue polling since there is at least one device
            self.timer.setInterval(polling_period_ms)
        else:
            maintenance_timer_tick_ms = 2000
            self.timer.setInterval(maintenance_timer_tick_ms)

        if not self.timer.isActive():
            self.timer.start()

    def handle_stale_device(self, devicedata, deviceIdx):
        '''
        Detects if the device is stale, and resets the data if so as to not
        mislead the user.
        '''
        # if already flagged as stale, no need to do anything
        if devicedata['stale']:
            return

        # check if it has been too long
        time_elapsed_since_last_update = \
            time.time() - devicedata['time_last_update']

        timeout_factor = 4
        timeout = timeout_factor * self.roundtrip_polling_period_ms * 1e-3
        if time_elapsed_since_last_update < timeout:
            return

        # if we're here, it has been too long since the last update
        self.reset_device_data(devicedata['device'], devicedata['data'])
        devicedata['stale'] = True

        # notify the UI
        row = deviceIdx
        self.refresh_display_for_row(row)

    def handle_connection_change(self, devicedata, deviceIdx):
        '''
        Detects if device connection changed and notifies the UI if a
        disconnection happens.
        '''
        device = devicedata['device']
        data = devicedata['data']

        is_connected = device.is_connected_and_authenticated()
        was_connected = True if data['Connected'] == 'yes' else False

        data['Connected'] = 'yes' if is_connected else 'no'

        if was_connected != is_connected:
            if is_connected:
                # Poll the sync status to get an immediate update on its state
                self.poll_sync_status_for_device(device, SyncStatusRequestFlags.all())
            else:
                self.reset_device_data(device, data)

            row = deviceIdx
            self.refresh_display_for_row(row)

            # Since we only poll connected devices, our polling timer needs to be updated.
            self.update_polling_timer()

    def default_program_id(self):
        ''' Default value for program id when unreal is not running.
        Used for messages that need this argument'''

        return '00000000-0000-0000-0000-000000000000'

    def program_id_from_device(self, device: Device):
        ''' Returns the program id of the running nDisplay unreal instance '''
        try:
            program_id = device.program_start_queue.running_puuids_named(UnrealJobs.Unreal.value)[-1]
        except IndexError:
            program_id = self.default_program_id()

        return program_id

    def can_poll_device(self, device: Device) -> bool:
        ''' Returns True if polling this device is allowed. Currently,
        this is based on whether the listener is ready to take commands.'''
        # can't poll if not connected to listener
        return device.is_connected_and_authenticated()

    @QtCore.Slot(Device)
    def on_device_connected(self, device: Device) -> None:
        ''' Called when a node has connected to its listener. '''
        # Ensure this code is run from the main thread
        if threading.current_thread() is not threading.main_thread():
            QtCore.QMetaObject.invokeMethod(
                self,
                'on_device_connected',
                QtCore.Qt.QueuedConnection,
                Q_ARG(Device, device)
            )
            return

        try:
            deviceIdx, devicedata = self.devicedata_from_device(device)
            self.handle_connection_change(devicedata=devicedata, deviceIdx=deviceIdx)
        except KeyError:
            LOGGER.warning(f"nDisplay Monitor could not find find device {device.name} upon connection")

    @QtCore.Slot(Device)
    def on_device_disconnected(self, device: Device) -> None:
        ''' Called when a node has disconnected from its listener. '''
        # Ensure this code is run from the main thread
        if threading.current_thread() is not threading.main_thread():
            QtCore.QMetaObject.invokeMethod(
                self,
                'on_device_disconnected',
                QtCore.Qt.QueuedConnection,
                Q_ARG(Device, device)
            )
            return

        try:
            deviceIdx, devicedata = self.devicedata_from_device(device)
            self.handle_connection_change(devicedata=devicedata, deviceIdx=deviceIdx)
        except KeyError:
            LOGGER.warning(f"nDisplay Monitor could not find find device {device.name} upon disconnection")

    def poll_sync_status_for_device(self, device: Device, request_flags: SyncStatusRequestFlags):
        ''' Polls sync status for the given device '''

        # no point in continuing if not connected to listener
        if not self.can_poll_device(device):
            return

        # create message

        program_id = self.program_id_from_device(device)

        _, msg = message_protocol.create_get_sync_status_message(
            program_id, request_flags)

        # send get sync status message
        device.unreal_client.send_message(msg)

    def poll_sync_status(self, request_flags: SyncStatusRequestFlags, all=False) -> None:
        ''' Polls sync status for all nDisplay devices in round robin '''

        numdevs = len(self.devicedatas)

        for _ in range(numdevs):
            if self._next_roundrobin_poll_deviceIdx >= numdevs:
                self._next_roundrobin_poll_deviceIdx = 0

            # Get the device we're interested in polling and advance the round robin index
            deviceIdx = self._next_roundrobin_poll_deviceIdx
            devicedata = next(islice(self.devicedatas.values(), deviceIdx, deviceIdx + 1))
            self._next_roundrobin_poll_deviceIdx += 1
            device = devicedata['device']

            # We still need to do some maintenance on all the devices. By iterating
            # over them even in sections per polling timout we keep them up to date.

            # detect connection changes (a disconnection invalidates data)
            self.handle_connection_change(devicedata, deviceIdx)

            # detect stale devices
            self.handle_stale_device(devicedata, deviceIdx)

            # Poll the device if we can.
            if self.can_poll_device(device):
                # request status
                self.poll_sync_status_for_device(device, request_flags)
                if not all:
                    return

    def poll_variable_sync_status(self) -> None:
        ''' Poll sync status but only include items that may change and that are
        not likely to cause hitches in the target machine. '''

        request_flags = SyncStatusRequestFlags.all()

        # We don't poll status that can cause hitches, or that do not typically change.

        request_flags &= ~SyncStatusRequestFlags.SyncTopos  # <-- Causes hitches when presenting frames.
        request_flags &= ~SyncStatusRequestFlags.MosaicTopos
        request_flags &= ~SyncStatusRequestFlags.DriverInfo

        if not self.poll_gpu_stats:
            request_flags &= ~SyncStatusRequestFlags.GpuCoreClockKhz
            request_flags &= ~SyncStatusRequestFlags.GpuTemperature
            request_flags &= ~SyncStatusRequestFlags.GpuUtilization

        self.poll_sync_status(request_flags)

    def devicedata_from_device(self, device):
        ''' Retrieves the devicedata and index for given device '''
        for device_idx, hash_devicedata in enumerate(self.devicedatas.items()):
            device_hash, devicedata = hash_devicedata[0], hash_devicedata[1]
            if device_hash == device.device_hash:
                return (device_idx, devicedata)

        raise KeyError

    def populate_sync_data(self, devicedata, message):
        '''
        Populates model data with message contents, which comes from 'get sync
        data' command.
        '''

        data = devicedata['data']
        device = devicedata['device']

        request_flags = SyncStatusRequestFlags(message['request_flags'])
        sync_status = message['syncStatus']

        #
        # Sync Topology
        #
        if SyncStatusRequestFlags.SyncTopos in request_flags:
            sync_topos = sync_status['syncTopos']

            # Build list informing which Gpus in each Sync group are in sync
            gpus = []

            for sync_topo in sync_topos:
                gpu_sync_oks = [gpu['bIsSynced'] for gpu in sync_topo['syncGpus']]
                gpu_syncs = map(lambda x: "Synced" if x else 'Free', gpu_sync_oks)
                gpus.append('%s' % (', '.join(gpu_syncs)))

            data['Gpus'] = '\n'.join(gpus) if len(gpus) > 0 else self.DATA_MISSING

            # Build list informing which Display in each Sync group are in sync.
            displays = []

            bpc_strings = {1: 6, 2: 8, 3: 10, 4: 12, 5: 16}

            for sync_topo in sync_topos:
                display_sync_states = [
                    f"{syncDisplay['syncState']}"
                    f"({bpc_strings.get(syncDisplay['bpc'], '??')}bpc)"
                    for syncDisplay in sync_topo['syncDisplays']]
                displays.append(', '.join(display_sync_states))

            if len(displays) > 0:
                data['Displays'] = '\n'.join(displays)
            else:
                data['Displays'] = self.DATA_MISSING

            # Build Fps
            refresh_rates = \
                [f"{syncTopo['syncStatusParams']['refreshRate']*1e-4:.3f}"
                    for syncTopo in sync_topos]

            if len(refresh_rates) > 0:
                data['Fps'] = '\n'.join(refresh_rates)
            else:
                data['Fps'] = self.DATA_MISSING

            # Build House Sync
            house_fpss = [syncTopo['syncStatusParams']['houseSyncIncoming']*1e-4
                          for syncTopo in sync_topos]
            house_syncs = [syncTopo['syncStatusParams']['bHouseSync']
                           for syncTopo in sync_topos]
            house_sync_fpss = list(
                map(
                    lambda x: f"{x[1]:.3f}" if x[0] else 'no',
                    zip(house_syncs, house_fpss)
                )
            )

            if len(house_sync_fpss) > 0:
                data['HouseSync'] = '\n'.join(house_sync_fpss)
            else:
                data['HouseSync'] = self.DATA_MISSING

            # Build Sync Source
            source_str = {0: 'Vsync', 1: 'House'}
            sync_sources = [sync_topo['syncControlParams']['source']
                            for sync_topo in sync_topos]
            sync_sources = [source_str.get(sync_source, 'Unknown')
                            for sync_source in sync_sources]
            bInternalSecondaries = [
                sync_topo['syncStatusParams']['bInternalSecondary']
                for sync_topo in sync_topos]

            sync_followers = []

            for i in range(len(sync_sources)):
                if bInternalSecondaries[i] and sync_sources[i] == 'Vsync':
                    sync_followers.append('Vsync(daisy)')
                else:
                    sync_followers.append(sync_sources[i])

            if len(sync_followers) > 0:
                data['SyncSource'] = '\n'.join(sync_followers)
            else:
                data['SyncSource'] = self.DATA_MISSING

        #
        # Mosaic Topology
        #
        if SyncStatusRequestFlags.MosaicTopos in request_flags:
            mosaic_topos = sync_status['mosaicTopos']

            mosaic_topo_lines = []

            for mosaic_topo in mosaic_topos:
                display_settings = mosaic_topo['displaySettings']
                width_per_display = display_settings['width']
                height_per_display = display_settings['height']

                width = mosaic_topo['columns'] * width_per_display
                height = mosaic_topo['rows'] * height_per_display

                # Ignoring displaySettings['freq'] because it seems to be fixed
                # and ignores sync frequency.
                line = f"{width}x{height} {display_settings['bpp']}bpp"
                mosaic_topo_lines.append(line)

            data['Mosaics'] = '\n'.join(mosaic_topo_lines)

        #
        # Build PresentMode
        #
        if SyncStatusRequestFlags.FlipModeHistory in request_flags:
            flip_history = sync_status['flipModeHistory']

            if len(flip_history) > 0:
                data['PresentMode'] = flip_history[-1]

            # Detect PresentMode glitches
            if len(set(flip_history)) > 1:
                data['PresentMode'] = 'GLITCH!'
                data['TimeLastFlipGlitch'] = time.time()

            # Write time since last glitch
            if data['PresentMode'] != self.DATA_MISSING:
                time_since_flip_glitch = time.time() - data['TimeLastFlipGlitch']

                # For 1 minute, let the user know that there was a flip mode glitch
                if time_since_flip_glitch < 1*60:
                    data['PresentMode'] = data['PresentMode'].split('\n')[0] \
                        + '\n' + str(int(time_since_flip_glitch))

        # Window in focus or not
        if SyncStatusRequestFlags.PidInFocus in request_flags:
            data['InFocus'] = 'no'
            for prg in device.program_start_queue.running_programs_named(UnrealJobs.Unreal.value):
                if prg.pid and prg.pid == sync_status['pidInFocus']:
                    data['InFocus'] = 'yes'
                    break

        # Show Exe flags (like Disable Fullscreen Optimization)
        if SyncStatusRequestFlags.ProgramLayers in request_flags:
            layers = '\n'.join([layer for layer in sync_status['programLayers'][1:]])
            if 'DISABLEDXMAXIMIZEDWINDOWEDMODE' in layers:
                data['FSO'] = 'no'
            elif len(device.program_start_queue.running_programs_named(UnrealJobs.Unreal.value)):
                data['FSO'] = 'yes'
            else:
                data['FSO'] = 'n/a'

        # Driver version
        if SyncStatusRequestFlags.DriverInfo in request_flags:
            try:
                driver = sync_status['driverVersion']
                data['Driver'] = f'{int(driver/100)}.{driver % 100}'
            except (KeyError, TypeError):
                data['Driver'] = self.DATA_MISSING

        # Taskbar visibility
        if SyncStatusRequestFlags.Taskbar in request_flags:
            data['Taskbar'] = sync_status.get('taskbar', self.DATA_MISSING)

        # Operating system version
        data['OsVer'] = self.friendly_osver(device)

        # CPU utilization
        if SyncStatusRequestFlags.CpuUtilization in request_flags:
            try:
                num_cores = len(sync_status['cpuUtilization'])
                num_overloaded_cores = 0
                cpu_load_avg = 0.0
                for core_load in sync_status['cpuUtilization']:
                    cpu_load_avg += float(core_load) * (1.0 / num_cores)
                    if core_load > self.CORE_OVERLOAD_THRESH:
                        num_overloaded_cores += 1

                data['CpuUtilization'] = f"{cpu_load_avg:.0f}%"
                if num_overloaded_cores > 0:
                    data['CpuUtilization'] += f' ({num_overloaded_cores} cores >' \
                        f' {self.CORE_OVERLOAD_THRESH}%)'

                if device.processor_smt:
                    data['CpuUtilization'] += ' (SMT ENABLED)'
            except (KeyError, ValueError):
                data['CpuUtilization'] = self.DATA_MISSING

        # Host memory utilization
        if SyncStatusRequestFlags.AvailablePhysicalMemory in request_flags:
            try:
                gb = 1024 * 1024 * 1024
                mem_total = device.total_phys_mem
                mem_avail = sync_status.get('availablePhysicalMemory', 0)
                mem_utilized = mem_total - mem_avail
                data['MemUtilization'] = \
                    f'{mem_utilized/gb:.1f} / {mem_total/gb:.0f} GB'
            except TypeError:
                data['MemUtilization'] = self.DATA_MISSING

        if self.poll_gpu_stats:

            # GPU utilization + clocks
            if (SyncStatusRequestFlags.GpuUtilization | SyncStatusRequestFlags.GpuCoreClockKhz) in request_flags:
                try:
                    gpu_stats = list(map(
                        lambda x: f"#{x[0]}: {x[1]:.0f}% ({x[2] / 1000:.0f} MHz)",
                        zip(count(), sync_status['gpuUtilization'],
                            sync_status['gpuCoreClocksKhz'])))

                    if len(gpu_stats) > 0:
                        data['GpuUtilization'] = '\n'.join(gpu_stats)
                    else:
                        data['GpuUtilization'] = self.DATA_MISSING
                except (KeyError, TypeError):
                    data['GpuUtilization'] = self.DATA_MISSING

            # GPU temperature
            if SyncStatusRequestFlags.GpuTemperature in request_flags:
                try:
                    temps = [t if t != -2147483648 else self.DATA_MISSING for t in sync_status['gpuTemperature']]

                    if len(temps) > 0:
                        data['GpuTemperature'] = '\n'.join(
                            map(lambda x: f"#{x[0]}: {x[1]}° C", zip(count(), temps)))
                    else:
                        data['GpuTemperature'] = self.DATA_MISSING
                except (KeyError, TypeError):
                    data['GpuTemperature'] = self.DATA_MISSING
        else:
            # If we're generally not polling stats, show n/a as to not be misleading
            # since it is such a volatile metric.
            data['GpuTemperature'] = 'n/a'
            data['GpuUtilization'] = 'n/a'

    def on_get_sync_status(self, device, message):
        '''
        Called when the listener has sent a message with the sync status
        '''
        try:
            if message['bAck'] is False:
                return
        except KeyError:
            LOGGER.error('Error parsing "get sync status" (missing "bAck")')
            return

        # Parse and update the model.
        deviceIdx, devicedata = self.devicedata_from_device(device)
        devicedata['time_last_update'] = time.time()
        devicedata['stale'] = False

        try:
            self.populate_sync_data(devicedata=devicedata, message=message)
        except (KeyError, ValueError):
            LOGGER.error(
                'Error parsing "get sync status" message and populating'
                'model data\n\n=== Traceback BEGIN ===\n'
                f'{traceback.format_exc()}=== Traceback END ===\n')
            return

        row = deviceIdx
        self.refresh_display_for_row(row)

    def try_issue_console_exec(self, exec_str, executor=''):
        ''' Issues a console exec to the cluster '''
        devices = [devicedata['device']
                   for devicedata in self.devicedatas.values()]

        if len(devices):
            try:
                devices[0].console_exec_cluster(devices, exec_str, executor)
                return True
            except Exception as exc:
                LOGGER.warning(f'Could not issue console exec ({exc})',
                               exc_info=exc)

        return False

    def make_default_table_item(self):
        ''' Creates a standard table item with default configuration '''
        item = QStandardItem()
        item.setFlags(item.flags() & ~QtCore.Qt.ItemIsEditable & ~QtCore.Qt.ItemIsSelectable)
        return item

    def refresh_display_for_row(self, row: int):
        ''' Refreshes the data displayed in the given row. Zero-based index. '''
        # Gather the data
        _, devicedata = list(self.devicedatas.items())[row]
        device = devicedata['device']
        data = devicedata['data']
        is_program_running = self.program_id_from_device(device) != self.default_program_id()

        if data['Connected'].lower() == 'yes':
            is_connected = True
        else:
            is_connected = False

        # Update the item
        for col, colname in enumerate(self.colnames):

            item = self.item(row, col)

            if not item:
                item = self.make_default_table_item()
                self.setItem(row, col, item)

            # Set text
            value = data[colname]
            item.setText(value)

            # Set background color
            bg_color = self.color_for_column(
                colname=colname,
                value=value,
                data=data,
                is_program_running=is_program_running
            )

            item.setBackground(bg_color)

            # Set foreground color
            if is_connected:
                item.setForeground(self.FG_COLOR_NORMAL)
            else:
                item.setForeground(self.FG_COLOR_DISCONNECTED)

            # Set the connection icon
            if colname == 'Connected':
                icon = self.icon_unconnected

                if value.lower() == 'yes':
                    if is_program_running:
                        icon = self.icon_running
                    else:
                        icon = self.icon_connected

                item.setIcon(icon)
                item.setText("")

            # Set the alignment
            if colname in ('CpuUtilization', 'GpuUtilization'):
                alignment = Qt.AlignmentFlag.AlignLeft | Qt.AlignmentFlag.AlignVCenter
            else:
                alignment = Qt.AlignmentFlag.AlignCenter

            item.setTextAlignment(alignment)

    def update_device_name_and_address(self, in_device: Device):
        ''' Refeshes the node names and addresses displayed in the table '''

        for deviceIdx, devicedata in enumerate(self.devicedatas.values()):
            device = devicedata['device']

            # See if it is the right device based on its hash.
            if in_device.device_hash == device.device_hash:

                # Update name and address
                data = devicedata['data']
                data['Host'] = str(device.address)
                data['Node'] = device.name

                # Refresh the table with the new data
                row = deviceIdx
                self.refresh_display_for_row(row)
                return

    @QtCore.Slot()
    def on_refresh_table_clicked(self):
        ''' Refreshes the data in the table, which involves sending commands to
        the devices.'''

        for devicedata in self.devicedatas.values():
            device = devicedata['device']
            device.refresh_mosaics()
            self.update_device_name_and_address(device)

        # Request a full update on the sync status.
        self.poll_sync_status(SyncStatusRequestFlags.all(), all=True)

    @QtCore.Slot()
    def on_disable_fso_clicked(self):
        ''' Tries to force the correct UnrealEditor.exe flags '''
        for devicedata in self.devicedatas.values():
            device = devicedata['device']
            data = devicedata['data']

            # Generally, want FSO to be disabled.
            if data['FSO'] != 'no':
                device.disable_fso()

    @QtCore.Slot()
    def on_soft_kill_clicked(self):
        ''' Kills the cluster by sending a message to the primary. '''
        devices = [devicedata['device']
                   for devicedata in self.devicedatas.values()]

        if len(devices):
            try:
                devices[0].soft_kill_cluster(devices)
            except Exception:
                LOGGER.warning("Could not soft kill cluster")

    @QtCore.Slot()
    def on_minimize_windows_clicked(self):
        ''' Tries to minimize all windows in the nodes. '''
        for devicedata in self.devicedatas.values():
            device = devicedata['device']
            device.minimize_windows()

    @QtCore.Slot()
    def on_gpu_stats_toggled(self, state: int):
        ''' Called when the GPU Stats checkbox is toggled. We update the state variable accordingly. '''
        if state == Qt.Checked.value:
            self.poll_gpu_stats = True
        else:
            self.poll_gpu_stats = False
