# Copyright Epic Games, Inc. All Rights Reserved.

from collections import OrderedDict
from typing import Optional

from PySide6.QtCore import Signal, QSortFilterProxyModel, Qt
from PySide6.QtWidgets import QComboBox, QCompleter, QHBoxLayout, \
    QHeaderView, QLabel, QPushButton, QSizePolicy, QStyledItemDelegate, \
    QTableView, QVBoxLayout, QWidget, QCheckBox

from switchboard.devices.ndisplay.ndisplay_monitor import nDisplayMonitor


class nDisplayMonitorUI(QWidget):

    signal_exec_history_changed = Signal()

    def __init__(self, parent: QWidget, monitor: nDisplayMonitor):
        QWidget.__init__(self, parent)

        self.monitor = monitor

        # Console exec combo entries; key is lowercased, value is as-entered.
        # TODO: Frecency sorting?
        self.exec_history = OrderedDict[str, str]()

        self.cmb_console_exec = QComboBox()
        self.exec_model = QSortFilterProxyModel(self.cmb_console_exec)
        self.exec_completer = QCompleter(self.exec_model)

        layout = QVBoxLayout(self)
        layout.addLayout(self.create_button_row_layout())
        layout.addWidget(self.create_table_view())

    def create_button_row_layout(self) -> QHBoxLayout:
        layout_buttons = QHBoxLayout()

        # Create console exec label/combo/button
        self.cmb_console_exec.setEditable(True)
        self.cmb_console_exec.setInsertPolicy(
            QComboBox.NoInsert)  # done manually
        self.cmb_console_exec.lineEdit().returnPressed.connect(
            self.on_console_return_pressed)
        size = QSizePolicy(QSizePolicy.Preferred, QSizePolicy.Preferred)
        size.setHorizontalStretch(3)
        self.cmb_console_exec.setSizePolicy(size)
        self.cmb_console_exec.setMinimumWidth(150)

        # Set up auto-completion
        self.exec_model.setFilterCaseSensitivity(Qt.CaseInsensitive)
        self.exec_model.setSourceModel(self.cmb_console_exec.model())
        self.exec_completer.setCompletionMode(QCompleter.PopupCompletion)
        self.exec_completer.popup().setItemDelegate(QStyledItemDelegate())
        self.exec_completer.activated.connect(
            lambda: self.try_issue_console_exec(),
            Qt.QueuedConnection)  # Queued to work around edit clear timing.
        self.cmb_console_exec.setCompleter(self.exec_completer)
        self.cmb_console_exec.lineEdit().textEdited.connect(
            self.exec_model.setFilterFixedString)

        btn_console_exec = QPushButton('Exec')
        btn_console_exec.setToolTip(
            'Issues a console command via nDisplay cluster event')
        btn_console_exec.clicked.connect(
            lambda: self.try_issue_console_exec())

        # Create remaining buttons

        # Refresh Table button to query all fields in the monitor
        btn_refresh_table = QPushButton('Refresh Table')
        btn_refresh_table.setToolTip(
            'Updates the cached mosaic topologies.')
        btn_refresh_table.clicked.connect(
            self.monitor.on_refresh_table_clicked)

        # This button disables full screen optimizations on the executable
        # This has been linked in the past to being 1 frame out of sync.
        btn_disable_fso = QPushButton('Disable FSO')
        btn_disable_fso.setToolTip(
            'Disables Full Screen Optimizations on the Unreal Engine executable.'
            '\nNote: The Remote executable must be running.')
        btn_disable_fso.clicked.connect(
            self.monitor.on_disable_fso_clicked)

        # Minimize button to minimize all windows on the remote node.
        # This has been useful in the past when there are windows that
        # otherwise stay on top of the nDisplay window.
        btn_minimize_windows = QPushButton('Minimize')
        btn_minimize_windows.setToolTip(
            'Minimizes all windows in the nodes.')
        btn_minimize_windows.clicked.connect(
            self.monitor.on_minimize_windows_clicked)

        # GPU Stats checkbox to enable GPU stats monitoring
        chk_gpu_stats = QCheckBox('GPU Stats')
        chk_gpu_stats.setToolTip(
            'GPU Stats add overhead to the remote node that can cause hitches. Only enable when needed.')
        chk_gpu_stats.stateChanged.connect(
            self.monitor.on_gpu_stats_toggled)

        layout_buttons.addWidget(QLabel('Console:'))
        layout_buttons.addWidget(self.cmb_console_exec)
        layout_buttons.addWidget(btn_console_exec)
        layout_buttons.addStretch(1)
        layout_buttons.addWidget(chk_gpu_stats)
        layout_buttons.addWidget(btn_refresh_table)

        if self.monitor.show_disable_fso_btn:
            layout_buttons.addWidget(btn_disable_fso)

        layout_buttons.addWidget(btn_minimize_windows)

        return layout_buttons

    def create_table_view(self) -> QTableView:
        tableview = QTableView()

        # the monitor is the model of this tableview.
        tableview.setModel(self.monitor)

        size = QSizePolicy(QSizePolicy.Preferred, QSizePolicy.Preferred)
        size.setHorizontalStretch(1)
        tableview.setSizePolicy(size)

        # configure resize modes on headers
        tableview.horizontalHeader().setSectionResizeMode(
            QHeaderView.ResizeToContents)
        tableview.horizontalHeader().setStretchLastSection(False)
        tableview.verticalHeader().setSectionResizeMode(
            QHeaderView.ResizeToContents)
        tableview.verticalHeader().setVisible(False)

        return tableview

    def on_console_return_pressed(self):
        # If accepting from the autocomplete popup, don't exec twice.
        is_completion = self.exec_completer.popup().currentIndex().isValid()
        if not is_completion:
            self.try_issue_console_exec()

    def try_issue_console_exec(self, exec_str: Optional[str] = None):
        if exec_str is None:
            exec_str = self.cmb_console_exec.currentText().strip()

        if not exec_str:
            return

        self.update_exec_history(exec_str)
        issued = self.monitor.try_issue_console_exec(exec_str)
        if issued:
            self.cmb_console_exec.clearEditText()
            self.cmb_console_exec.setCurrentIndex(-1)

    def update_exec_history(self, exec_str: str):
        ''' Invoked when a command is executed, either adding or moving to most recent '''
        # Reinsert (case-insensitive) duplicates as most recent.
        exec_str_lower = exec_str.lower()
        if exec_str_lower in self.exec_history:
            del self.exec_history[exec_str_lower]
        self.exec_history[exec_str_lower] = exec_str

        MAX_HISTORY_ENTRIES = 30
        if len(self.exec_history) > MAX_HISTORY_ENTRIES:
            self.exec_history.popitem(last=False)

        self._rebuild_console_exec_items()

        self.signal_exec_history_changed.emit()

    def replace_exec_history(self, history: list[str]):
        ''' Can be used to restore the history model from a persisted version. '''
        self.exec_history = OrderedDict({
            exec.lower(): exec
            for exec in history
        })

        self._rebuild_console_exec_items()

    def _rebuild_console_exec_items(self):
        # Most recently used at the top.
        self.cmb_console_exec.clear()
        self.cmb_console_exec.addItems(list(reversed(self.exec_history.values())))
