# Copyright Epic Games, Inc. All Rights Reserved.

import json
from PySide6 import QtGui, QtWidgets
from PySide6.QtCore import Qt
from PySide6.QtWidgets import (
    QDialog, QDialogButtonBox, QFormLayout, QGridLayout,
    QHBoxLayout, QLabel, QLineEdit, QMessageBox, QPushButton,
    QSizePolicy, QToolButton, QVBoxLayout, QWidget, QGroupBox
)

from switchboard import switchboard_widgets as sb_widgets
from switchboard.config import CONFIG
from switchboard.sbcache import SBCache
from switchboard.switchboard_logging import LOGGER
from switchboard.devices.unreal.plugin_unreal import DeviceUnreal


class CustomCommandDialog(QDialog):
    '''Dialog for adding/editing custom exec commands'''

    def __init__(self, parent, name='', command=''):
        super().__init__(parent)

        self.setWindowTitle('Custom Command' if not name else 'Edit Custom Command')
        self.setMinimumWidth(400)

        # Main layout
        layout = QVBoxLayout(self)

        # Form layout
        form_layout = QFormLayout()

        # Name input
        self.name_edit = QLineEdit(name)
        self.name_edit.setPlaceholderText('e.g. Trigger Event')
        form_layout.addRow('Name:', self.name_edit)

        # Command input
        self.command_edit = QLineEdit(command)
        self.command_edit.setPlaceholderText('e.g. ke * MyBlueprintCustomEventName')
        self.command_edit.setToolTip('Console command to execute on the nDisplay cluster')
        form_layout.addRow('Command:', self.command_edit)

        layout.addLayout(form_layout)

        # Buttons
        self.button_box = QDialogButtonBox(
            QDialogButtonBox.Ok | QDialogButtonBox.Cancel,
            parent=self
        )
        self.button_box.accepted.connect(self.validate_and_accept)
        self.button_box.rejected.connect(self.reject)
        layout.addWidget(self.button_box)

    def validate_and_accept(self):
        '''Validates input and accepts dialog'''
        if not self.name_edit.text().strip():
            QMessageBox.warning(self, 'Invalid Input', 'Please enter a name to use for the button label.')
            return
        if not self.command_edit.text().strip():
            QMessageBox.warning(self, 'Invalid Input', 'Please enter a command to execute.')
            return
        self.accept()

    def get_values(self):
        '''Returns the entered name and command'''
        return self.name_edit.text().strip(), self.command_edit.text().strip()


class nDisplayToolsUI(QWidget):
    '''
    UI for nDisplay Tools tab that allows setting Media Profiles and
    applying Live Link presets to the nDisplay cluster at runtime.
    '''

    # Configuration
    CUSTOM_COMMANDS_NUM_COLUMNS = 4

    def __init__(self, parent: QWidget, plugin_ndisplay):
        QWidget.__init__(self, parent)

        self.plugin_ndisplay = plugin_ndisplay

        # Main layout
        layout = QVBoxLayout(self)
        layout.setSpacing(10)

        # Media Profile section
        media_profile_group = self.create_media_profile_section()
        layout.addWidget(media_profile_group)

        # Live Link Preset section
        livelink_preset_group = self.create_livelink_preset_section()
        layout.addWidget(livelink_preset_group)

        # Custom Commands section
        custom_commands_group = self.create_custom_commands_section()
        layout.addWidget(custom_commands_group)

        # Add stretch to push everything to the top
        layout.addStretch()

        # Listen for clipboard changes to auto-refresh
        QtWidgets.QApplication.clipboard().dataChanged.connect(
            self.on_clipboard_changed)

        # Listen for asset additions from clipboard operations
        DeviceUnreal.static_signals.clipboard_assets_added_signal.connect(
            self.on_assets_added_via_clipboard)

        # Listen for project asset scans/updates
        DeviceUnreal.static_signals.project_assets_updated_signal.connect(
            self.on_project_assets_updated)

        # Listen for nDisplay config file changes
        self.plugin_ndisplay.csettings['ndisplay_config_file'].signal_setting_changed.connect(
            self.on_config_changed)

        # Listen for default media profile/livelink preset changes
        self.plugin_ndisplay.csettings['mediaprofile'].signal_setting_changed.connect(
            self.on_default_settings_changed)
        self.plugin_ndisplay.csettings['livelink_preset'].signal_setting_changed.connect(
            self.on_default_settings_changed)

        # Listen for custom commands changes
        self.plugin_ndisplay.csettings['custom_exec_commands'].signal_setting_changed.connect(
            self.on_custom_commands_changed)

        # Initial population of combo boxes
        self.refresh_media_profiles()
        self.refresh_livelink_presets()

    def scan_project_assets(self):
        '''Scans project for assets and updates cache, following UAssetSetting pattern'''
        DeviceUnreal.analyze_project_assets()

    def on_project_assets_updated(self):
        '''Called when project assets are updated from any source'''
        self.refresh_media_profiles()
        self.refresh_livelink_presets()

    def on_config_changed(self, old_value, new_value):
        '''Called when nDisplay config file changes - update default selections'''
        # When config changes, refresh dropdown contents from SBCache first
        # as the loaded config might be from a different project with different assets
        self.refresh_media_profiles()
        self.refresh_livelink_presets()
        self.refresh_custom_commands()

        # Then update dropdown selections to match defaults
        self._set_default_media_profile()
        self._set_default_livelink_preset()

    def on_default_settings_changed(self, old_value, new_value):
        '''Called when default media profile or livelink preset settings change'''
        # Update dropdown selections to reflect new defaults
        self._set_default_media_profile()
        self._set_default_livelink_preset()

    def on_custom_commands_changed(self, old_value, new_value):
        '''Called when custom commands settings change'''
        self.refresh_custom_commands()

    def _set_default_media_profile(self):
        '''Sets the media profile combo box to the default value from nDisplay device settings'''
        # Get default media profile from nDisplay class settings
        default_gamepath = self.plugin_ndisplay.csettings['mediaprofile'].get_value()

        if default_gamepath:
            # Find the item in combo box that matches this gamepath
            for i in range(self.media_profile_combo.count()):
                if self.media_profile_combo.itemData(i) == default_gamepath:
                    self.media_profile_combo.setCurrentIndex(i)
                    break

    def _set_default_livelink_preset(self):
        '''Sets the livelink preset combo box to the default value from nDisplay device settings'''
        # Get default livelink preset from nDisplay class settings
        default_gamepath = self.plugin_ndisplay.csettings['livelink_preset'].get_value()

        if default_gamepath:
            # Find the item in combo box that matches this gamepath
            for i in range(self.livelink_preset_combo.count()):
                if self.livelink_preset_combo.itemData(i) == default_gamepath:
                    self.livelink_preset_combo.setCurrentIndex(i)
                    break

    def create_media_profile_section(self) -> QGroupBox:
        '''Creates the Media Profile section of the UI'''
        group_box = QGroupBox("Media Profile")
        layout = QVBoxLayout()

        # Combo box and buttons layout
        controls_layout = QHBoxLayout()

        # Media Profile combo box
        self.media_profile_combo = sb_widgets.SearchableComboBox(self)
        self.media_profile_combo.setEditable(False)
        self.media_profile_combo.setSizePolicy(
            QSizePolicy.Expanding, QSizePolicy.Preferred)
        self.media_profile_combo.setMinimumWidth(300)
        self.media_profile_combo.setToolTip(
            "Select a Media Profile or paste a path from the Content Browser")
        controls_layout.addWidget(self.media_profile_combo)

        # Refresh button
        self.media_profile_refresh_btn = QPushButton()
        self.media_profile_refresh_btn.setIcon(
            QtGui.QIcon(':/icons/images/icon_refresh.png'))
        self.media_profile_refresh_btn.setToolTip("Scan project for Media Profiles")
        self.media_profile_refresh_btn.clicked.connect(
            self.scan_project_assets)
        self.media_profile_refresh_btn.setMaximumWidth(30)
        controls_layout.addWidget(self.media_profile_refresh_btn)

        # Set button
        self.media_profile_set_btn = QPushButton("Set")
        self.media_profile_set_btn.setToolTip(
            "Apply the selected Media Profile to the nDisplay cluster, or clear if (None) is selected")
        self.media_profile_set_btn.clicked.connect(
            self.on_set_media_profile)
        controls_layout.addWidget(self.media_profile_set_btn)

        layout.addLayout(controls_layout)
        group_box.setLayout(layout)

        return group_box

    def create_livelink_preset_section(self) -> QGroupBox:
        '''Creates the Live Link Preset section of the UI'''
        group_box = QGroupBox("Live Link Preset")
        layout = QVBoxLayout()

        # Combo box and buttons layout
        controls_layout = QHBoxLayout()

        # Live Link Preset combo box
        self.livelink_preset_combo = sb_widgets.SearchableComboBox(self)
        self.livelink_preset_combo.setEditable(False)
        self.livelink_preset_combo.setSizePolicy(
            QSizePolicy.Expanding, QSizePolicy.Preferred)
        self.livelink_preset_combo.setMinimumWidth(300)
        self.livelink_preset_combo.setToolTip(
            "Select a Live Link Preset or paste a path from the Content Browser")
        controls_layout.addWidget(self.livelink_preset_combo)

        # Refresh button
        self.livelink_preset_refresh_btn = QPushButton()
        self.livelink_preset_refresh_btn.setIcon(
            QtGui.QIcon(':/icons/images/icon_refresh.png'))
        self.livelink_preset_refresh_btn.setToolTip("Scan project for Live Link Presets")
        self.livelink_preset_refresh_btn.clicked.connect(
            self.scan_project_assets)
        self.livelink_preset_refresh_btn.setMaximumWidth(30)
        controls_layout.addWidget(self.livelink_preset_refresh_btn)

        # Set button
        self.livelink_preset_set_btn = QPushButton("Set")
        self.livelink_preset_set_btn.setToolTip(
            "Apply the selected Live Link Preset to the nDisplay cluster, or remove all sources if (None) is selected")
        self.livelink_preset_set_btn.clicked.connect(
            self.on_set_livelink_preset)
        controls_layout.addWidget(self.livelink_preset_set_btn)

        layout.addLayout(controls_layout)
        group_box.setLayout(layout)

        return group_box

    def create_custom_commands_section(self) -> QGroupBox:
        '''Creates the Custom Commands section of the UI'''

        group_box = QGroupBox("Custom Commands")
        main_layout = QVBoxLayout()

        # Container widget for command buttons
        self.commands_container = QWidget()
        self.commands_grid = QGridLayout(self.commands_container)
        self.commands_grid.setSpacing(5)

        # Set uniform column widths
        for col in range(self.CUSTOM_COMMANDS_NUM_COLUMNS):
            self.commands_grid.setColumnStretch(col, 1)

        main_layout.addWidget(self.commands_container)

        # Empty state label
        self.empty_commands_label = QLabel("No custom commands added yet")
        self.empty_commands_label.setStyleSheet("QLabel { color: gray; }")
        self.empty_commands_label.setAlignment(Qt.AlignCenter)
        main_layout.addWidget(self.empty_commands_label)

        # Add command button at bottom left
        bottom_layout = QHBoxLayout()
        self.add_command_btn = QPushButton()
        self.add_command_btn.setIcon(
            QtGui.QIcon(':/icons/images/PlusSymbol_12x.png'))
        self.add_command_btn.setProperty(u"frameless", True)
        self.add_command_btn.setMaximumWidth(30)
        self.add_command_btn.setToolTip("Add a new custom command")
        self.add_command_btn.clicked.connect(self.add_custom_command)
        bottom_layout.addWidget(self.add_command_btn)
        bottom_layout.addStretch()

        main_layout.addLayout(bottom_layout)

        group_box.setLayout(main_layout)

        # Load existing commands
        self.refresh_custom_commands()

        return group_box

    def refresh_media_profiles(self):
        '''Refreshes the Media Profile combo box with assets from the cache'''
        self.media_profile_combo.clear()

        # Add empty option first
        self.media_profile_combo.addItem("(None)", "")

        # Query media profiles from cache
        project = SBCache().query_project(CONFIG.UPROJECT_PATH.get_value())
        if project:
            assets = SBCache().query_assets_by_classname(
                project=project,
                classnames=DeviceUnreal.MEDIAPROFILE_CLASS_NAMES)

            # Sort assets by name
            assets = sorted(assets, key=lambda x: x.name)

            # Add to combo box
            for asset in assets:
                self.media_profile_combo.addItem(asset.name, asset.gamepath)

        # Set default value from nDisplay device settings
        self._set_default_media_profile()

    def refresh_livelink_presets(self):
        '''Refreshes the Live Link Preset combo box with assets from the cache'''
        self.livelink_preset_combo.clear()

        # Add empty option first
        self.livelink_preset_combo.addItem("(None)", "")

        # Query livelink presets from cache
        project = SBCache().query_project(CONFIG.UPROJECT_PATH.get_value())
        if project:
            assets = SBCache().query_assets_by_classname(
                project=project,
                classnames=DeviceUnreal.LIVELINKPRESET_CLASS_NAMES)

            # Sort assets by name
            assets = sorted(assets, key=lambda x: x.name)

            # Add to combo box
            for asset in assets:
                self.livelink_preset_combo.addItem(asset.name, asset.gamepath)

        # Set default value from nDisplay device settings
        self._set_default_livelink_preset()

    def on_clipboard_changed(self):
        '''Called when clipboard content changes - auto-refresh if relevant assets were pasted'''
        clipboard = QtWidgets.QApplication.clipboard()
        mime_data = clipboard.mimeData()

        if mime_data.hasText():
            text = mime_data.text()

            # Check if it looks like an Unreal asset reference
            if '/Game/' in text or '/Script/' in text:
                # Refresh both combo boxes since we don't know what was pasted
                self.refresh_media_profiles()
                self.refresh_livelink_presets()

    def on_assets_added_via_clipboard(self):
        '''Called when assets are confirmed to be added via clipboard operations'''
        # Refresh both combo boxes since new assets were added to the database
        self.refresh_media_profiles()
        self.refresh_livelink_presets()

    def get_connected_devices(self):
        '''Returns a list of connected nDisplay devices'''
        devices = []
        if hasattr(self.plugin_ndisplay, 'active_unreal_devices'):
            for device in self.plugin_ndisplay.active_unreal_devices:
                if device.device_type == "nDisplay" and device.is_connected_and_authenticated():
                    devices.append(device)
        return devices

    def on_set_media_profile(self):
        '''Sets the selected Media Profile on the cluster or clears it if (None) is selected'''
        devices = self.get_connected_devices()
        if not devices:
            LOGGER.warning("No connected nDisplay devices found")
            return

        # Get selected media profile gamepath
        current_index = self.media_profile_combo.currentIndex()
        if current_index > 0:  # (None) is 0
            gamepath = self.media_profile_combo.currentData()
            if gamepath:
                # Ensure proper formatting of the game path
                if not gamepath.endswith('.uasset'):
                    # Extract the asset name and construct the full path
                    parts = gamepath.split('/')
                    asset_name = parts[-1] if parts else ''
                    gamepath = f"{gamepath}.{asset_name}"

                exec_str = f"MediaProfile.Set {gamepath}"

                try:
                    devices[0].console_exec_cluster(devices, exec_str)
                    LOGGER.info(f"Set Media Profile: {gamepath}")
                except Exception as e:
                    LOGGER.error(f"Failed to set Media Profile: {e}")
        else:
            # "(None)" is selected - clear the media profile
            exec_str = "MediaProfile.Clear"
            try:
                devices[0].console_exec_cluster(devices, exec_str)
                LOGGER.info("Cleared Media Profile")
            except Exception as e:
                LOGGER.error(f"Failed to clear Media Profile: {e}")

    def on_set_livelink_preset(self):
        '''Sets the selected Live Link Preset on the cluster or removes all sources if (None) is selected'''
        devices = self.get_connected_devices()
        if not devices:
            LOGGER.warning("No connected nDisplay devices found")
            return

        # Get selected livelink preset gamepath
        current_index = self.livelink_preset_combo.currentIndex()
        if current_index > 0:  # Non-None selection
            gamepath = self.livelink_preset_combo.currentData()
            if gamepath:
                # Use the helper method from DeviceUnreal to format the command
                try:
                    # Create a temporary device instance to use the helper method
                    temp_device = devices[0]
                    exec_str = temp_device.exec_command_for_livelink_preset(gamepath)

                    devices[0].console_exec_cluster(devices, exec_str)
                    LOGGER.info(f"Set Live Link Preset: {gamepath}")
                except Exception as e:
                    LOGGER.error(f"Failed to set Live Link Preset: {e}")
        else:
            # "(None)" is selected - clear all LiveLink sources
            exec_str = "LiveLink.Preset.Clear"
            try:
                devices[0].console_exec_cluster(devices, exec_str)
                LOGGER.info("Cleared all Live Link sources")
            except Exception as e:
                LOGGER.error(f"Failed to clear Live Link sources: {e}")

    def add_custom_command(self):
        '''Shows dialog to add a new custom command'''

        dialog = CustomCommandDialog(self)
        if dialog.exec_() == QDialog.Accepted:
            name, command = dialog.get_values()

            # Get existing commands
            commands = self._get_custom_commands()

            # Add new command
            commands.append({
                'name': name,
                'command': command
            })

            # Save to settings
            self._save_custom_commands(commands)

            # Refresh UI
            self.refresh_custom_commands()

    def edit_custom_command(self, index):
        '''Shows dialog to edit an existing custom command'''

        commands = self._get_custom_commands()

        if 0 <= index < len(commands):
            cmd = commands[index]
            dialog = CustomCommandDialog(self, cmd['name'], cmd['command'])
            if dialog.exec_() == QDialog.Accepted:
                name, command = dialog.get_values()
                commands[index] = {'name': name, 'command': command}
                self._save_custom_commands(commands)
                self.refresh_custom_commands()

    def remove_custom_command(self, index):
        '''Removes a custom command after confirmation'''

        commands = self._get_custom_commands()

        if 0 <= index < len(commands):
            cmd = commands[index]
            reply = QMessageBox.question(
                self,
                'Remove Custom Command',
                f"Remove command '{cmd['name']}'?",
                QMessageBox.Yes | QMessageBox.No,
                QMessageBox.No
            )
            if reply == QMessageBox.Yes:
                commands.pop(index)
                self._save_custom_commands(commands)
                self.refresh_custom_commands()

    def move_custom_command(self, from_index, to_index):
        '''Moves a custom command from one position to another'''

        commands = self._get_custom_commands()

        if (0 <= from_index < len(commands) and
            0 <= to_index < len(commands) and
            from_index != to_index):

            # Move the command
            command = commands.pop(from_index)
            commands.insert(to_index, command)

            # Save and refresh
            self._save_custom_commands(commands)
            self.refresh_custom_commands()

    def execute_custom_command(self, command):
        '''Executes a custom command on the nDisplay cluster'''

        devices = self.get_connected_devices()

        if not devices:
            LOGGER.warning("No connected nDisplay devices found")
            return

        try:
            devices[0].console_exec_cluster(devices, command)
            LOGGER.info(f"Executed custom command: {command}")
        except Exception as e:
            LOGGER.error(f"Failed to execute custom command: {e}")

    def refresh_custom_commands(self):
        '''Refreshes the custom command buttons from settings'''

        # Clear existing buttons
        while self.commands_grid.count():
            item = self.commands_grid.takeAt(0)
            if item.widget():
                item.widget().deleteLater()

        # Get commands from settings
        commands = self._get_custom_commands()

        # Show/hide empty state
        self.empty_commands_label.setVisible(len(commands) == 0)
        self.commands_container.setVisible(len(commands) > 0)

        # Create buttons in grid
        columns = self.CUSTOM_COMMANDS_NUM_COLUMNS
        for i, cmd in enumerate(commands):
            row = i // columns
            col = i % columns

            # Create down arrow button.
            button_widget = QWidget()
            button_widget.setSizePolicy(QSizePolicy.Expanding, QSizePolicy.Fixed)
            button_layout = QHBoxLayout(button_widget)
            button_layout.setContentsMargins(0, 0, 0, 0)
            button_layout.setSpacing(2)

            # Main command button
            cmd_btn = QPushButton()
            cmd_btn.setMinimumHeight(50) # Intended to look taller than usual, about the width of device rows.
            cmd_btn.setToolTip(f"Execute: {cmd['command']}")
            cmd_btn.setSizePolicy(QSizePolicy.Expanding, QSizePolicy.Fixed)
            cmd_btn.clicked.connect(lambda checked=False, c=cmd['command']: self.execute_custom_command(c))

            # Create a label with word wrap inside the button (also helps keep the button width distribution even)
            btn_label = QLabel(cmd['name'])
            btn_label.setWordWrap(True)
            btn_label.setAlignment(Qt.AlignCenter)
            btn_label.setStyleSheet("QLabel { background: transparent; }")

            # Set the label as the button's layout
            btn_layout = QVBoxLayout(cmd_btn)
            btn_layout.setContentsMargins(5, 5, 5, 5)
            btn_layout.addWidget(btn_label)

            button_layout.addWidget(cmd_btn)

            # Options button
            options_btn = QToolButton()
            options_btn.setIcon(QtGui.QIcon(':/icons/images/down_arrow.png'))
            options_btn.setProperty(u"frameless", True)
            options_btn.setFixedWidth(30)
            options_btn.setSizePolicy(QSizePolicy.Fixed, QSizePolicy.Fixed)
            options_btn.setToolTip("Edit or remove this command")
            options_btn.clicked.connect(lambda checked=False, idx=i, btn=options_btn: self._show_command_menu(idx, btn))

            button_layout.addWidget(options_btn, 0, Qt.AlignTop)

            self.commands_grid.addWidget(button_widget, row, col)

    def _show_command_menu(self, index, button):
        '''Shows context menu for command editing/removal'''

        menu = QtWidgets.QMenu(self)
        commands = self._get_custom_commands()

        edit_action = menu.addAction(QtGui.QIcon(':/icons/images/icon_edit.png'), "Edit")
        edit_action.triggered.connect(lambda: self.edit_custom_command(index))

        menu.addSeparator()

        # Move back option (disabled if already at beginning)
        move_back_action = menu.addAction("Move Back")
        move_back_action.setEnabled(index > 0)
        move_back_action.triggered.connect(lambda: self.move_custom_command(index, index - 1))

        # Move forward option (disabled if already at end)
        move_forward_action = menu.addAction("Move Forward")
        move_forward_action.setEnabled(index < len(commands) - 1)
        move_forward_action.triggered.connect(lambda: self.move_custom_command(index, index + 1))

        menu.addSeparator()

        remove_action = menu.addAction(QtGui.QIcon(':/icons/images/icon_remove.png'), "Remove")
        remove_action.triggered.connect(lambda: self.remove_custom_command(index))

        menu.exec_(button.mapToGlobal(button.rect().bottomLeft()))

    def _get_custom_commands(self):
        '''Gets custom commands from settings'''

        commands_json = self.plugin_ndisplay.csettings['custom_exec_commands'].get_value()
        commands = []
        for cmd_str in commands_json:
            try:
                commands.append(json.loads(cmd_str))
            except json.JSONDecodeError:
                LOGGER.warning(f"Invalid custom command JSON: {cmd_str}")
        return commands

    def _save_custom_commands(self, commands):
        '''Saves custom commands to settings'''

        commands_json = [json.dumps(cmd) for cmd in commands]
        self.plugin_ndisplay.csettings['custom_exec_commands'].update_value(commands_json)
