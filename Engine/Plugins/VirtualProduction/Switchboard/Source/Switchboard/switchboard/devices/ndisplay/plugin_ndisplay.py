# Copyright Epic Games, Inc. All Rights Reserved.

from __future__ import annotations

import json
import os
import socket
import struct
import sys
import traceback
import uuid

from enum import Enum
from ipaddress import ip_address
from pathlib import Path, PurePosixPath
from typing import Optional, List

from PySide6 import QtCore
from PySide6 import QtWidgets

from switchboard import message_protocol, switchboard_application
from switchboard import switchboard_utils as sb_utils
from switchboard import switchboard_widgets as sb_widgets
from switchboard.config import CONFIG, BoolSetting, IntSetting, FilePathSetting, \
    LoggingSetting, OptionSetting, Setting, StringSetting, SETTINGS, \
    StringListSetting, AddressSetting, migrate_comma_separated_string_to_list, \
    map_name_is_valid, get_game_launch_level_path
from switchboard.devices.device_widget_base import AddDeviceDialog
from switchboard.devices.unreal.plugin_unreal import DeviceUnreal, \
    DeviceWidgetUnreal, LiveLinkPresetSetting, MediaProfileSetting
from switchboard.devices.unreal.uassetparser import UassetParser
from switchboard.devices.device_base import DeviceStatus
from switchboard.switchboard_logging import LOGGER
from switchboard.sbcache import SBCache, Asset
from switchboard.devices.unreal.plugin_unreal import ProgramStartQueueItem, UnrealJobs

from .ndisplay_monitor_ui import nDisplayMonitorUI
from .ndisplay_monitor import nDisplayMonitor
from .ndisplay_tools_ui import nDisplayToolsUI


class PackagingClientConfig(Enum):
    ''' Used to specify the type of client packaging'''

    Default = "Default"
    NoClient = "None"
    Shipping = "Shipping"
    Development = "Development"


class LaunchMode(Enum):
    ''' Used to specify how to launch the Unreal instances. '''

    Standalone = "Standalone (ICVFX)"
    Packaged = "Packaged Game"


class AddnDisplayDialog(AddDeviceDialog):
    def __init__(self, existing_devices, parent=None):
        super().__init__(
            device_type="nDisplay", existing_devices=existing_devices,
            parent=parent)

        # Set enough width for a decent file path length
        self.setMinimumWidth(640)

        # remove unneeded base dialog layout rows
        self.form_layout.removeRow(self.name_field)
        self.form_layout.removeRow(self.address_field)
        self.name_field = None
        self.address_field = None

        # Initialize a grid layout for the form
        grid_layout = QtWidgets.QGridLayout()

        # Config File row

        lblConfigFile = QtWidgets.QLabel(self, text="Config File")

        # Combobox with config files
        self.cbConfigs = sb_widgets.SearchableComboBox(self)
        self.cbConfigs.setSizePolicy(QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Preferred)
        self.cbConfigs.setEditable(True)  # to allow the user to type the value

        # Button to browse for supported config files
        self.btnBrowse = QtWidgets.QPushButton(self, text="Browse")
        self.btnBrowse.clicked.connect(self.on_clicked_btnBrowse)

        # Button to find and populate config files in the UE project
        self.btnFindConfigs = QtWidgets.QPushButton(self, text="Populate")
        self.btnFindConfigs.clicked.connect(self.on_clicked_btnFindConfigs)

        config_file_layout = QtWidgets.QHBoxLayout()
        config_file_layout.addWidget(self.cbConfigs)
        config_file_layout.addWidget(self.btnBrowse)
        config_file_layout.addWidget(self.btnFindConfigs)

        grid_layout.addWidget(lblConfigFile, 0, 0)
        grid_layout.addLayout(config_file_layout, 0, 1)

        # Launch As row

        lblLaunchAs = QtWidgets.QLabel(self, text="Launch As")
        self.cmbLaunchAs = QtWidgets.QComboBox(self)
        self.cmbLaunchAs.addItems([LaunchMode.Standalone.value, LaunchMode.Packaged.value])
        self.cmbLaunchAs.currentIndexChanged.connect(self.on_cmbLaunchAs_currentIndexChanged)

        launch_as_layout = QtWidgets.QHBoxLayout()
        launch_as_layout.addWidget(self.cmbLaunchAs)
        launch_as_layout.setAlignment(QtCore.Qt.AlignLeft)

        grid_layout.addWidget(lblLaunchAs, 1, 0)
        grid_layout.addLayout(launch_as_layout, 1, 1)

        # Path row (initially not editable)
        self.lblPath = QtWidgets.QLabel(self, text=DevicenDisplay.csettings['packaged_game_path'].nice_name)
        self.lblPath.setEnabled(False)
        self.pathField = QtWidgets.QLineEdit(self)
        self.pathField.setEnabled(False)
        self.btnPathBrowse = QtWidgets.QPushButton(self, text="...")
        self.btnPathBrowse.setEnabled(False)
        self.btnPathBrowse.clicked.connect(self.on_clicked_btnPathBrowse)

        path_layout = QtWidgets.QHBoxLayout()
        path_layout.addWidget(self.pathField)
        path_layout.addWidget(self.btnPathBrowse)

        grid_layout.addWidget(self.lblPath, 2, 0)
        grid_layout.addLayout(path_layout, 2, 1)

        # Add the grid layout to the form layout
        self.form_layout.addRow(grid_layout)

        # Add a spacer right before the ok/cancel buttons
        spacer_layout = QtWidgets.QHBoxLayout()
        spacer_layout.addItem(
            QtWidgets.QSpacerItem(
                20, 20, QtWidgets.QSizePolicy.Minimum,
                QtWidgets.QSizePolicy.Expanding))
        self.form_layout.addRow("", spacer_layout)

        # Find existing nDisplay devices in order to issue a warning about replacing them.
        self.existing_ndisplay_devices = []

        for device in existing_devices:
            if device.device_type == "nDisplay" or str(device.device_type).startswith("nDisplay"):
                self.existing_ndisplay_devices.append(device)

        if self.existing_ndisplay_devices:
            self.layout().addWidget(
                QtWidgets.QLabel(
                    "Warning! All existing nDisplay devices will be "
                    "replaced."))

        # populate the config combobox with the items last populated.
        self.recall_config_itemDatas()

    #@override
    def refresh_validity(self):
        # Bypass base class validation for fields we don't use
        pass

    def on_cmbLaunchAs_currentIndexChanged(self, state):
        ''' Called when the user changes the state of the LaunchAs dropdown
        It updates the editability of the packaged game path selection accordingly.
        '''
        is_packaged = self.is_packaged_game()

        self.lblPath.setEnabled(is_packaged)
        self.pathField.setEnabled(is_packaged)
        self.pathField.setEnabled(is_packaged)
        self.btnPathBrowse.setEnabled(is_packaged)

    def on_clicked_btnPathBrowse(self):
        ''' Called to browse for the packaged game executable path.'''

        file_dialog = QtWidgets.QFileDialog(self)
        file_path, _ = file_dialog.getOpenFileName(
            self,
            "Select Executable",
            "",
            "All Files (*);;Executables (*.exe);;Scripts (*.bat *.cmd *.sh)"
        )

        if file_path:
            self.pathField.setText(os.path.normpath(file_path))

    def recall_config_itemDatas(self):
        '''
        Populate the config combobox with the already found configs, avoiding
        having to re-find every time.
        '''

        # cache the current value of the combo box (currently selected preset for this device)
        cur_item = self.cbConfigs.currentData()

        self.cbConfigs.clear()

        project = SBCache().query_project(CONFIG.UPROJECT_PATH.get_value())
        if project is None:
            return

        assets = SBCache().query_assets_by_classname(project=project, classnames=DeviceUnreal.NDISPLAY_CLASS_NAMES)

        for asset in assets:
            self.cbConfigs.addItem(asset.name, asset)

        # restore selected item
        for item_idx in range(self.cbConfigs.count()):
            if cur_item and (cur_item.name == self.cbConfigs.itemData(item_idx).name):
                self.cbConfigs.setCurrentIndex(item_idx)
                break

    def current_config_path(self):
        ''' Get currently selected config path in the combobox '''

        # Detect if this is a manual entry using the itemData

        itemText = self.cbConfigs.currentText()
        asset: Asset = self.cbConfigs.currentData()

        if (self.cbConfigs.currentData() is None or
                asset.name != itemText):
            config_path = itemText.replace('"', '').strip()
        else:
            config_path = asset.localpath.replace('"', '').strip()

        # normalize the path
        config_path = os.path.normpath(config_path)
        return config_path

    def is_packaged_game(self):
        ''' Returns true if this is a packaged game'''
        return self.cmbLaunchAs.currentText() == LaunchMode.Packaged.value

    def packaged_game_path(self):
        ''' Returns the path to the packaged game'''
        return self.pathField.text()

    def result(self):
        res = super().result()
        if res == QtWidgets.QDialog.Accepted:
            config_path = self.current_config_path()
            DevicenDisplay.csettings['ndisplay_config_file'].update_value(config_path)
            launch_mode = LaunchMode.Packaged.value if self.is_packaged_game() else LaunchMode.Standalone.value
            DevicenDisplay.csettings['launch_mode'].update_value(launch_mode)
            DevicenDisplay.csettings['packaged_game_path'].update_value(self.packaged_game_path())

        return res

    def on_clicked_btnBrowse(self):
        ''' Opens a file dialog to browse for the config file
        '''

        start_path = ''

        # prefer starting at the folder of the last ndisplay config
        last_config_path = DevicenDisplay.csettings['ndisplay_config_file'].get_value()

        if os.path.exists(last_config_path):
            start_path = os.path.split(last_config_path)[0]

        # otherwise, use the project's Content folder
        if not start_path:
            content_path = os.path.join(os.path.split(CONFIG.UPROJECT_PATH.get_value())[0],'Content')
            if os.path.exists(content_path):
                start_path = content_path

        # our fallback is the home folder
        if not start_path:
            start_path = str(Path.home())

        # show the open file dialog
        cfg_path, _ = QtWidgets.QFileDialog.getOpenFileName(
            self, "Select nDisplay config file", start_path,
            "nDisplay Config (*.ndisplay *.uasset)")

        # update the field with the selected path
        if len(cfg_path) > 0 and os.path.exists(cfg_path):
            self.cbConfigs.setCurrentText(cfg_path)

    def on_clicked_btnFindConfigs(self):
        ''' Finds and populates config combobox '''

        DeviceUnreal.analyze_project_assets()

        # update the combo box with the items.
        self.recall_config_itemDatas()

    def devices_to_add(self):
        cfg_file = self.current_config_path()
        try:
            devices = DevicenDisplay.parse_config(cfg_file).nodes

            if len(devices) == 0:
                LOGGER.error(
                    "Could not read any devices from nDisplay config file "
                    f"{cfg_file}")

            # Initialize from the config our setting that identifies which
            # device is the primary.
            for node in devices:
                if node['primary']:
                    DevicenDisplay.csettings['primary_device_name'].update_value(node['name'])
                    break

            return devices

        except (IndexError, KeyError, FileNotFoundError):
            LOGGER.error(f"Error parsing nDisplay config file {cfg_file}")
            return []

    def devices_to_remove(self):
        return self.existing_ndisplay_devices


class DeviceWidgetnDisplay(DeviceWidgetUnreal):

    signal_device_widget_primary = QtCore.Signal(object)

    def on_name_changed(self, text):
        super().on_name_changed(text)

        if self.name_line_edit.is_valid:
            parent = self.name_line_edit.parent()
            if parent:
                point = parent.mapToGlobal(self.name_line_edit.geometry().topRight())
                self.help_tool_tip.showText(
                    point,
                    "WARNING: Device names must match the nDisplay configuration")

    def _add_control_buttons(self):
        self._autojoin_visible = False
        super()._add_control_buttons()

    def add_widget_to_layout(self, widget):
        ''' DeviceWidget base class method override. '''

        if widget == self.name_line_edit:
            self.add_primary_button()

            # shorten the widget to account for the inserted primary button
            btn_added_width = (
                max(
                    self.primary_button.iconSize().width(),
                    self.primary_button.minimumSize().width())
                + 2 * self.layout.spacing()
                + self.primary_button.contentsMargins().left()
                + self.primary_button.contentsMargins().right())

            le_maxwidth = self.name_line_edit.maximumWidth() - btn_added_width
            self.name_line_edit.setMaximumWidth(le_maxwidth)

        super().add_widget_to_layout(widget)

    def add_primary_button(self):
        '''
        Adds to the layout a button to select which device should be the
        primary device in the cluster.
        '''

        self.primary_button = sb_widgets.ControlQPushButton.create(
            ':/icons/images/star_yellow_off.png',
            icon_disabled=':/icons/images/star_yellow_off.png',
            icon_hover=':/icons/images/star_yellow_off.png',
            icon_disabled_on=':/icons/images/star_yellow_off.png',
            icon_on=':/icons/images/star_yellow.png',
            icon_hover_on=':/icons/images/star_yellow.png',
            icon_size=QtCore.QSize(16, 16),
            tool_tip='Select as primary node'
        )

        self.add_widget_to_layout(self.primary_button)

        self.primary_button.clicked.connect(self.on_primary_button_clicked)

    def on_primary_button_clicked(self):
        ''' Called when primary_button is clicked '''

        self.signal_device_widget_primary.emit(self)


class DisplayConfig(object):
    ''' Encapsulates nDisplay config'''

    def __init__(self):
        self.nodes = []
        self.uasset_path = ''


class DevicenDisplay(DeviceUnreal):

    add_device_dialog = AddnDisplayDialog

    csettings = {
        'ndisplay_config_file': StringSetting(
            attr_name="ndisplay_cfg_file",
            nice_name="nDisplay Config File",
            value="",
            tool_tip="Path to nDisplay config file",
            allow_reset=False,
            is_read_only=True,
            category="General Settings",
        ),
        'launch_mode': OptionSetting(
            attr_name="launch_mode",
            nice_name="Launch As",
            value=LaunchMode.Standalone.value,
            possible_values=[
                LaunchMode.Standalone.value,
                LaunchMode.Packaged.value,
            ],
            tool_tip=f"Select the mode in which to launch the cluster. '{LaunchMode.Packaged.value}' will use the Packaged Game Path,"
            f" while '{LaunchMode.Standalone.value}' will launch the project using the UnrealEditor executable with -game in the command line.",
            category="General Settings",
        ),
        'packaged_game_path': FilePathSetting(
            attr_name="packaged_game_path",
            nice_name="Packaged Executable",
            value="",
            tool_tip=f"Path to the nDisplay packaged game executable. Only used when '{LaunchMode.Packaged.value}' launch mode is selected.",
            show_ui=True,
            file_path_filter="Programs and Scripts (*.exe;*.bat;*.sh);;All Files (*)",
            category="General Settings",
        ),
        'overwrite_remote_ndc_asset': BoolSetting(
            attr_name="overwrite_remote_ndc_asset",
            nice_name="Overwrite nDisplay UAsset",
            tool_tip="Overwrite the remote nDisplay .uasset from the local one when launching, if it exists",
            value=False,
            category="Network Settings",
        ),
        'use_all_available_cores': BoolSetting(
            attr_name="use_all_available_cores",
            nice_name="Use All Available Cores",
            value=False,
            category="UE Settings",
        ),
        'texture_streaming': BoolSetting(
            attr_name="texture_streaming",
            nice_name="Texture Streaming",
            value=True,
            category="Render Settings",
        ),
        'sound': BoolSetting(
            attr_name="sound",
            nice_name="Sound",
            value=False,
            category="UE Settings",
        ),
        'mcp': BoolSetting(
            attr_name="mcp",
            nice_name="MCP Subsystem",
            value=False,
            category="UE Settings",
        ),
        'loading_screen': BoolSetting(
            attr_name="loading_screen",
            nice_name="Loading Screen",
            value=False,
            tool_tip='When unchecked, will add -NoLoadingScreen to the command line',
            category="UE Settings",
        ),
        'allow_python': BoolSetting(
            attr_name="allow_python",
            nice_name="Python",
            value=False,
            tool_tip='When unchecked, will add -DisablePython to the command line',
            category="UE Settings",
        ),
        'incremental_gc_reachability': BoolSetting(
            attr_name="incremental_gc_reachability",
            nice_name="Incremental GC reachability",
            value=True,
            tool_tip='When checked, will set the CVar gc.AllowIncrementalReachability to 1',
            category="UE Settings",
        ),
        'render_api': OptionSetting(
            attr_name="render_api",
            nice_name="Render API",
            value="dx12",
            possible_values=[
                "dx11",
                "dx11 -sm5",
                "dx11 -sm6",
                "dx12",
                "dx12 -sm5",
                "dx12 -sm6",
                "vulkan",
                "vulkan -sm5",
                "vulkan -sm6"
            ],
            category="Render Settings",
        ),
        'multiplayer_mode': OptionSetting(
            attr_name="multiplayer_mode",
            nice_name="Multiplayer Server Mode",
            value='None',
            possible_values=['None', 'Listen server', 'Dedicated server'],
            category="Multiplayer Settings",
        ),
        'dedicated_server_address': AddressSetting(
            attr_name="dedicated_server_address",
            nice_name="Dedicated Server Address",
            value='127.0.0.1',
            tool_tip='Server address to connect to. Not used if Multiplayer Server Mode '
                     'is set to "Listen Server" or auto start of dedicated server is enabled',
            category="Multiplayer Settings",
        ),
        'dedicated_server_port': IntSetting(
            attr_name="dedicated_server_port",
            nice_name="Dedicated Server Port",
            value='7777',
            tool_tip='Server port to connect to. Not used if Multiplayer Server Mode is not Dedicated Server',
            category="Multiplayer Settings",
        ),
        'render_mode': OptionSetting(
            attr_name="render_mode",
            nice_name="Render Mode",
            value="Mono",
            possible_values=[
                "Mono", "Frame sequential", "Side-by-Side", "Top-bottom"],
            category="Render Settings",
        ),
        'render_sync_policy': OptionSetting(
            attr_name="render_sync_policy",
            nice_name="Render Sync Policy",
            value='Config',
            possible_values=['Config', 'None', 'Ethernet', 'Nvidia'],
            tool_tip=(
                "Select which cluster synchronization policy to use. \n"
                "- 'Config': Use the setting in the nDisplay config file \n"
                "- 'None': Freerun. Formerly known as 'sync policy 0'\n"
                "- 'Ethernet': Ethernet-based sync. Formerly known as 'sync policy 1'\n"
                "- 'Nvidia': Nvidia's Quadro Sync Framelock. Formerly known as 'sync policy 2'\n"
            ),
            category="Render Settings",
        ),
        'executable_filename': FilePathSetting(
            attr_name="executable_filename",
            nice_name="Unreal Editor Filename",
            value="UnrealEditor.exe",
            file_path_filter="Programs (*.exe;*.bat);;All Files (*)",
            category="UE Settings",
        ),
        'ndisplay_cmd_args': StringSetting(
            attr_name="ndisplay_cmd_args",
            nice_name="Extra Cmd Line Args",
            value="",
            category="Command Line Args",
        ),
        'ndisplay_exec_cmds': StringListSetting(
            attr_name="ndisplay_exec_cmds",
            nice_name='ExecCmds',
            value=[],
            tool_tip='ExecCmds to be passed. No need for outer double quotes.',
            allow_reset=False,
            migrate_data=migrate_comma_separated_string_to_list,
            category="Command Line Args",
        ),
        'ndisplay_dp_cvars': StringListSetting(
            attr_name='ndisplay_dp_cvars',
            nice_name="DPCVars",
            value=[],
            tool_tip="Device profile console variables.",
            migrate_data=migrate_comma_separated_string_to_list,
            category="Command Line Args",
        ),
        'ndisplay_unattended': BoolSetting(
            attr_name='ndisplay_unattended',
            nice_name='Unattended',
            value=True,
            tool_tip=(
                'Include the "-unattended" command line argument, which is '
                'documented to "Disable anything requiring feedback from the '
                'user."'),
            category="UE Settings",
        ),
        'max_gpu_count': OptionSetting(
            attr_name="max_gpu_count",
            nice_name="Number of GPUs",
            value=2,
            possible_values=list(range(1, 17)),
            tool_tip=(
                "If you have multiple GPUs in the PC, you can specify how "
                "many to use."),
            category="GPU/CPU Settings",
       ),
        'priority_modifier': OptionSetting(
            attr_name='priority_modifier',
            nice_name="Process Priority",
            value=sb_utils.PriorityModifier.Normal.name,
            possible_values=[p.name for p in sb_utils.PriorityModifier],
            tool_tip="Used to override the priority of the process.",
            category="GPU/CPU Settings",
        ),
        'populated_config_itemDatas': Setting(
            attr_name='populated_config_itemDatas',
            nice_name="Populated Config Files",
            value=[],  # populated by AddnDisplayDialog
            tool_tip="Remember the last populated list of config files",
            show_ui=False,
        ),
        'minimize_before_launch': BoolSetting(
            attr_name='minimize_before_launch',
            nice_name="Minimize Before Launch",
            value=True,
            tool_tip="Minimizes windows before launch",
            category="General Settings",
        ),
        'primary_device_name': StringSetting(
            attr_name='primary_device_name',
            nice_name="Primary Device",
            value='',
            tool_tip=(
                "Identifies which nDisplay device should be the primary node in the cluster"),
            show_ui=False,
        ),
        'logging': LoggingSetting(
            attr_name='logging',
            nice_name='Logging',
            value=None,
            # Extracted from:
            #   Engine\Plugins\Runtime\nDisplay\Source\DisplayCluster\
            #   Private\Misc\DisplayClusterLog.cpp
            categories=[
                'LogConcert',
                'LogDisplayClusterCluster',
                'LogDisplayClusterConfig',
                'LogDisplayClusterEngine',
                'LogDisplayClusterGame',
                'LogDisplayClusterNetwork',
                'LogDisplayClusterRender',
                'LogDisplayClusterRenderSync',
                'LogDisplayClusterViewport',
                'LogDisplayClusterMedia',
                'LogDisplayClusterStageMonitoringDWM',
                'LogLiveLink',
                'LogRemoteControl',
            ],
            tool_tip='Logging categories and verbosity levels',
            category="UE Settings",
        ),
        'udpmessaging_unicast_endpoint': StringSetting(
            attr_name='udpmessaging_unicast_endpoint',
            nice_name='Unicast Endpoint',
            value=':0',
            tool_tip=(
                'Local interface binding (-UDPMESSAGING_TRANSPORT_UNICAST) of '
                'the form {address}:{port}. If {address} is omitted, the device '
                'address is used.'),
            category="Network Settings",
        ),
        'udpmessaging_extra_static_endpoints': StringSetting(
            attr_name='udpmessaging_extra_static_endpoints',
            nice_name='Extra Static Endpoints',
            value='',
            tool_tip=(
                'Comma separated. Used to add static endpoints '
                '(-UDPMESSAGING_TRANSPORT_STATIC) in addition to those '
                'managed by Switchboard.'),
            category="Network Settings",
        ),
        'disable_ensures': BoolSetting(
            attr_name='disable_ensures',
            nice_name="Disable Ensures",
            value=True,
            tool_tip="When checked, disables the handling of ensure errors - which are non-fatal and may cause hitches.",
            category="UE Settings",
        ),
        'disable_all_screen_messages': BoolSetting(
            attr_name='disable_all_screen_messages',
            nice_name="Disable All Screen Messages",
            value=True,
            tool_tip="When checked, adds DisableAllScreenMessages to ExecCmds",
            category="UE Settings",
        ),
        'livelink_preset': LiveLinkPresetSetting(
            attr_name='livelink_preset',
            nice_name='LiveLink Preset',
            value='',
            tool_tip=(
                'Adds the selected LiveLink preset to the command line \n'),
            category="Tools Settings",
        ),
        'graphics_adapter': OptionSetting(
            attr_name="graphics_adapter",
            nice_name="Graphics Adapter",
            value='Config',
            possible_values=['Config'] + list(range(8)),
            tool_tip=(
                "Select which graphics adapter to use (will set r.GraphicsAdapter early CVar) \n"
                "- 'Config' : Use the setting in the nDisplay config file \n"
                "- 0, 1, .. : The specified gpu index \n"
            ),
            category="GPU/CPU Settings",
        ),
        'mediaprofile': MediaProfileSetting(
            attr_name='mediaprofile',
            nice_name='Media Profile',
            value='',
            tool_tip=('Adds the selected Media Profile to the command line'),
            category="Tools Settings",
        ),
        'lock_gpu_clock': BoolSetting(
            attr_name="lock_gpu_clock",
            nice_name="Lock GPU Clock",
            value=True,  # nDisplay hitches less whe GPU clocks are locked.
            tool_tip=(
                "Hint to lock the GPU clock to its allowed maximum. Requires SwitchboardListenerHelper \n"
                "to be running on the client machine as administrator, otherwise this option will be ignored."
            ),
            show_ui=True if sys.platform in ('win32', 'linux') else False,  # Gpu Clocker is available in select platforms
            category="GPU/CPU Settings",
        ),
        'cluster_exec_history': StringListSetting(
            attr_name='cluster_exec_history',
            nice_name='Cluster exec console command history',
            value=[],
            show_ui=False,
        ),
        'custom_exec_commands': StringListSetting(
            attr_name='custom_exec_commands',
            nice_name='Custom Exec Commands',
            value=[],
            show_ui=False,
            tool_tip='List of custom exec commands for the nDisplay Tools UI'
        )
    }

    ndisplay_monitor_ui = None
    ndisplay_monitor = None
    ndisplay_tools_ui = None

    def __init__(self, name, address, **kwargs):
        super().__init__(name, address, **kwargs)

        self.settings = {
            'ue_command_line': StringSetting(
                attr_name="ue_command_line",
                nice_name="UE Command Line",
                value=kwargs.get("ue_command_line", ''),
                allow_reset=False,
                is_read_only=True,
                category="Command Line Args",
            ),
            'window_position': Setting(
                attr_name="window_position",
                nice_name="Window Position",
                value=tuple(kwargs.get("window_position", (0, 0))),
                show_ui=False
            ),
            'window_resolution': Setting(
                attr_name="window_resolution",
                nice_name="Window Resolution",
                value=tuple(kwargs.get("window_resolution", (100, 100))),
                show_ui=False
            ),
            'fullscreen': BoolSetting(
                attr_name="fullscreen",
                nice_name="fullscreen",
                value=kwargs.get("fullscreen", False),
                show_ui=False
            ),
            'headless' : BoolSetting(
                attr_name="headless",
                nice_name="headless",
                value=kwargs.get("renderHeadless", False),
                show_ui=False
            ),
            'config_graphics_adapter' : IntSetting(
                attr_name="config_graphics_adapter",
                nice_name="Config Graphics Adapter",
                value=kwargs.get("graphics_adapter", -1),
                show_ui=False
            ),
        }

        self.render_mode_cmdline_opts = {
            "Mono": "-dc_dev_mono",
            "Frame sequential": "-quad_buffer_stereo",
            "Side-by-Side": "-dc_dev_side_by_side",
            "Top-bottom": "-dc_dev_top_bottom"
        }

        self.pending_transfer_cfg = False
        self.pending_transfer_uasset = False
        self.bp_object_path: Optional[str] = None
        self.path_to_config_on_host = DevicenDisplay.csettings[
            'ndisplay_config_file'].get_value()

        CONFIG.ENGINE_DIR.signal_setting_overridden.connect(
            self.on_cmdline_affecting_override)
        CONFIG.ENGINE_DIR.signal_setting_changed.connect(
            self.on_change_setting_affecting_command_line)
        CONFIG.UPROJECT_PATH.signal_setting_overridden.connect(
            self.on_cmdline_affecting_override)
        CONFIG.UPROJECT_PATH.signal_setting_changed.connect(
            self.on_change_setting_affecting_command_line)

        # common settings affect instance command line
        for csetting in self.__class__.plugin_settings():
            csetting.signal_setting_changed.connect(
                self.on_change_setting_affecting_command_line)

        self.unreal_client.delegates[
            'send file complete'] = self.on_send_file_complete
        self.unreal_client.delegates[
            'get sync status'] = self.on_get_sync_status
        self.unreal_client.delegates[
            'refresh mosaics'] = self.on_refresh_mosaics

        # create monitor if it doesn't exist
        self.__class__.create_monitor_if_necessary()

        # node configuration (updated from config file)
        self.nodeconfig = {}

        if len(self.settings['ue_command_line'].get_value()) == 0:
            self.generate_unreal_command_line()

        try:
            cfg_file = DevicenDisplay.csettings[
                'ndisplay_config_file'].get_value(self.name)
            self.update_settings_controlled_by_config(cfg_file)
        except Exception:
            LOGGER.error(
                f"{self.name}: Could not update from '{cfg_file}' during "
                "initialization. \n\n=== Traceback BEGIN ===\n"
                f"{traceback.format_exc()}=== Traceback END ===\n")

    @classmethod
    def create_monitor_if_necessary(cls):
        ''' Creates the nDisplay Monitor if it doesn't exist yet.
        '''
        if not DevicenDisplay.ndisplay_monitor:
            DevicenDisplay.ndisplay_monitor = nDisplayMonitor(None)

        return DevicenDisplay.ndisplay_monitor

    def on_change_setting_affecting_command_line(self, oldval, newval):
        self.generate_unreal_command_line()

    @classmethod
    def plugin_settings(cls):
        ''' Returns common settings that belong to all devices of this class.
        '''
        return list(cls.csettings.values()) + [
            DeviceUnreal.csettings['port'],
            DeviceUnreal.csettings['roles_filename'],
            DeviceUnreal.csettings['stage_session_id'],
        ]

    def device_settings(self):
        '''
        This is given to the config, so that it knows to save them when they
        change. settings_dialog.py will try to create a UI for each setting if
        setting.show_ui is True.
        '''
        return super().device_settings() + list(self.settings.values())

    def setting_overrides(self):
        return [
            DevicenDisplay.csettings['ndisplay_cmd_args'],
            DevicenDisplay.csettings['ndisplay_exec_cmds'],
            DevicenDisplay.csettings['ndisplay_dp_cvars'],
            DevicenDisplay.csettings['max_gpu_count'],
            DevicenDisplay.csettings['priority_modifier'],
            DevicenDisplay.csettings['udpmessaging_unicast_endpoint'],
            DevicenDisplay.csettings['udpmessaging_extra_static_endpoints'],
            DevicenDisplay.csettings['livelink_preset'],
            DevicenDisplay.csettings['mediaprofile'],
            DevicenDisplay.csettings['graphics_adapter'],
            DevicenDisplay.csettings['packaged_game_path'],
            DevicenDisplay.csettings['render_mode'],
            CONFIG.ENGINE_DIR,
            CONFIG.SOURCE_CONTROL_WORKSPACE,
            CONFIG.PROJECT_WORKSPACE,
            CONFIG.UPROJECT_PATH,
        ]

    def on_cmdline_affecting_override(self, device_name):
        if self.name == device_name:
            self.generate_unreal_command_line()

    @property
    def is_recording_device(self):
        return False

    # Override these properties from the DeviceUnreal base class to use the
    # nDisplay-specific settings.
    @property
    def executable_filename(self):
        return DevicenDisplay.csettings["executable_filename"].get_value()

    @property
    def extra_cmdline_args_setting(self) -> str:
        return DevicenDisplay.csettings['ndisplay_cmd_args'].get_value(
            self.name)

    @property
    def udpmessaging_unicast_endpoint_setting(self) -> str:
        return DevicenDisplay.csettings[
            'udpmessaging_unicast_endpoint'].get_value(self.name)

    @property
    def udpmessaging_extra_static_endpoints_setting(self) -> str:
        return DevicenDisplay.csettings[
            'udpmessaging_extra_static_endpoints'].get_value(self.name)

    def generate_multiplayer_map_name_args(self, map_name):

        primary_device = self.get_primary_node_device()
        primary_device_name = 'Node_0' if primary_device is None else primary_device.name
        primary_device_address = '127.0.0.1' if primary_device is None else primary_device.address

        primary_node_binary_event_port = 41004

        # set primary node binary event port
        if self.nodeconfig and 'port_ce_binary' in self.nodeconfig:
            primary_node_binary_event_port = self.nodeconfig['port_ce_binary']

        cluster_nodes_count = 0
        for device in self.active_unreal_devices:
            is_device_connected = (device.status == DeviceStatus.CLOSED) or (device.status == DeviceStatus.OPEN)
            if (device.device_type == "nDisplay") and is_device_connected:
                cluster_nodes_count = cluster_nodes_count + 1

        def get_common_args():
            """ 
            get common args for client and server 
            """
            return '?'.join([
                    f'ClusterId={os.getpid()}',
                    f'PrimaryNodeId={primary_device_name}',
                    f'PrimaryNodePort={primary_node_binary_event_port}',
                    f'ClusterNodesCount={cluster_nodes_count}'
                ])

        def get_client_args(server_address):
            return '?'.join([
                    server_address,
                    f'node={self.name}',
                    get_common_args()
                ])

        multiplayer_mode_name = DevicenDisplay.csettings["multiplayer_mode"].get_value()

        # single player
        if multiplayer_mode_name == 'None':
            return map_name

        def is_local_address(address):
            return address == SETTINGS.ADDRESS.get_value() or address == '127.0.0.1'

        # dedicated server
        if multiplayer_mode_name == 'Dedicated server':
            dedicated_server_address = DevicenDisplay.csettings["dedicated_server_address"].get_value()

            return get_client_args(
                f'{dedicated_server_address}:{DevicenDisplay.csettings["dedicated_server_port"].get_value()}')

        # listen server
        if not self.is_primary():
            return get_client_args(primary_device_address)

        if map_name_is_valid(map_name):
            return '?'.join([map_name, 'Listen', get_common_args()])

        current_map_name = get_game_launch_level_path()
        if current_map_name.strip() == '':
            return map_name

        return '?'.join([current_map_name, 'Listen', get_common_args()])

    def should_use_project_path_in_command_line(self) -> bool:
        ''' Returns true if the project path should be added to the command line.
        This is normally true when launching the editor, but not when launching a cooked game executable.
        '''

        # Don't use project path if this is a packaged game
        if self.is_packaged_game():
            return False

        # The default exe of an Unreal device is the Editor.
        default_exe = Path(DeviceUnreal.csettings['ue_exe']._original_value)

        # This is the current executable
        exe = Path(self.generate_exe_path())

        # We don't use direct comparison to include editor build variants, such as -Debug builds.
        return exe.stem.lower().startswith(default_exe.stem.lower())

    def is_packaged_game(self) -> bool:
        ''' Returns True if this node is launching as a package game'''
        return DevicenDisplay.csettings['launch_mode'].get_value() == LaunchMode.Packaged.value

    def get_packaged_game_path(self) -> str:
        ''' Returns the packaged game executable path '''
        return DevicenDisplay.csettings['packaged_game_path'].get_value(self.name)

    def get_remote_log_path(self):
        ''' Override from base class '''

        # If packaged, assume exe name also the name of the folder where Saved is located in.

        if self.is_packaged_game():
            exe_path = Path(self.get_packaged_game_path())
            return exe_path.with_name(exe_path.stem) / 'Saved' / 'Logs'

        return super().get_remote_log_path()

    def generate_exe_path(self) -> str:
        ''' Uses the packaged executable path when this is a packaged game launch
        Note: We didn't override generate_unreal_exe_path because Fill DDCs would grab the wrong executable.
        '''

        if self.is_packaged_game():
            return self.get_packaged_game_path()

        return super().generate_unreal_exe_path()

    def generate_unreal_command_line(self, map_name=""):

        uproject = ''

        if self.should_use_project_path_in_command_line():
            uproject = CONFIG.UPROJECT_PATH.get_value(self.name)

        # normalize path if not empty
        if uproject != "":
            # Skip normpath when the remote isn't Windows: on a Windows
            # host it would flip the Linux/Mac remote's forward slashes
            # to backslashes and break the launch.
            # (It's not obvious why we do it on Windows, either.)
            if self._remote_path_uses_windows_separators():
                uproject = os.path.normpath(uproject)
            uproject = f'"{uproject}"'

        # Extra arguments specified in settings
        additional_args = self.extra_cmdline_args_setting

        # Path to config file
        cfg_file = self.path_to_config_on_host

        # nDisplay window info
        win_pos = self.settings['window_position'].get_value()
        win_res = self.settings['window_resolution'].get_value()
        fullscreen = self.settings['fullscreen'].get_value()
        headless = self.settings['headless'].get_value()

        # Misc settings
        render_mode = self.render_mode_cmdline_opts[
            DevicenDisplay.csettings['render_mode'].get_value(self.name)]
        render_api = DevicenDisplay.csettings['render_api'].get_value(
            self.name)
        render_api = f"-{render_api}"
        use_all_cores = (
            "-useallavailablecores"
            if DevicenDisplay.csettings['use_all_available_cores'].get_value(
                self.name)
            else "")
        no_texture_streaming = (
            "-notexturestreaming"
            if not DevicenDisplay.csettings['texture_streaming'].get_value(
                self.name)
            else "")

        # Sound
        no_sound = (
            "-nosound"
            if not DevicenDisplay.csettings['sound'].get_value(self.name)
            else "")

        # MCP Subsystem
        no_mcp = (
            "-nomcp"
            if not DevicenDisplay.csettings['mcp'].get_value(self.name)
            else "")

        # Loading Screen
        no_loading_screen = (
            "-NoLoadingScreen"
            if not DevicenDisplay.csettings['loading_screen'].get_value(self.name)
            else "")

        # Allow Python
        no_python = (
            "-DisablePython"
            if not DevicenDisplay.csettings['allow_python'].get_value(self.name)
            else "")

        # MaxGPUCount (mGPU)
        max_gpu_count = DevicenDisplay.csettings["max_gpu_count"].get_value(
            self.name)
        try:
            max_gpu_count = (
                f"-MaxGPUCount={max_gpu_count}"
                if int(max_gpu_count) > 1 else '')
        except ValueError:
            LOGGER.warning(f"Invalid Number of GPUs '{max_gpu_count}'")
            max_gpu_count = ''

        # Overridden classes at runtime
        ini_engine = (
            "-ini:Engine:"
            "[/Script/Engine.Engine]:"
            "GameEngine=/Script/DisplayCluster.DisplayClusterGameEngine"
            ",[/Script/Engine.Engine]:"
            "GameViewportClientClassName="
            "/Script/DisplayCluster.DisplayClusterViewportClient"
            ",[/Script/Engine.UserInterfaceSettings]:"
            "bAllowHighDPIInGameMode=True"
            ",[/Script/Engine.UserInterfaceSettings]:"
            "bAllowHighDpiWhenUnattended=True")

        ini_game = (
            "-ini:Game:"
            "[/Script/EngineSettings.GeneralProjectSettings]:"
            "bUseBorderlessWindow=True")

        ini_input = (
            "-ini:Input:"
            "[/Script/Engine.InputSettings]:"
            "DefaultPlayerInputClass=/Script/DisplayCluster.DisplayClusterPlayerInput")

        # VP roles
        vproles, missing_roles = self.get_vproles()

        if missing_roles:
            LOGGER.error(
                f"{self.name}: Omitted roles not in the remote roles ini "
                "file which would cause UE to fail at launch: "
                f"{'|'.join(missing_roles)}")

        vproles = '-VPRole=' + '|'.join(vproles) if vproles else ''

        # Session ID
        session_id = DeviceUnreal.csettings["stage_session_id"].get_value(
            self.name)
        session_id = f"-StageSessionId={session_id}" if session_id > 0 else ''

        # Friendly name. Avoid spaces to avoid parsing issues.
        friendly_name = f'-StageFriendlyName="{self.name.replace(" ", "_")}"'

        # Unattended mode
        unattended = (
            '-unattended'
            if DevicenDisplay.csettings['ndisplay_unattended'].get_value()
            else '')

        # UdpMessaging endpoints
        udpm_transport_multi = ''
        if self.udpmessaging_multicast_endpoint:
            udpm_transport_multi = (
                '-UDPMESSAGING_TRANSPORT_MULTICAST='
                f'"{self.udpmessaging_multicast_endpoint}"'
            )

        udpm_transport_unicast = ''
        if self.udpmessaging_unicast_endpoint:
            udpm_transport_unicast = (
                '-UDPMESSAGING_TRANSPORT_UNICAST='
                f'\"{self.udpmessaging_unicast_endpoint}\"')

        udpm_transport_static = ''
        static_endpoints = self.build_udpmessaging_static_endpoint_list()
        if len(static_endpoints) > 0:
            udpm_transport_static = (
                '-UDPMESSAGING_TRANSPORT_STATIC='
                f'\"{",".join(static_endpoints)}\"')

        # Disable ensures
        disable_ensures = (
            '-handleensurepercent=0'
            if DevicenDisplay.csettings['disable_ensures'].get_value()
            else '')

        no_screen_messages = (
            '-NoScreenMessages'
            if DevicenDisplay.csettings['disable_all_screen_messages'].get_value()
            else '')

        # Modify map name arg for multiplayer mode
        map_name = self.generate_multiplayer_map_name_args(map_name)

        dash_game = "-game" if not self.is_packaged_game() else ""

        # fill in fixed arguments
        args = [
            f'{uproject}',
            dash_game,                    # render nodes run in -game
            f'{map_name}',                # map to open
            "-messaging",                 # enables messaging, needed for MultiUser
            "-dc_cluster",                # this is a cluster node
            "-nosplash",                  # avoids splash screen
            "-fixedseed",                 # for determinism
            "-NoVerifyGC",                # improves performance
            "-noxrstereo",                # avoids a conflict with steam/oculus
            "-xrtrackingonly",            # allows multi-UE SteamVR for trackers (but disallows sending frames)
            "-RemoteControlIsHeadless",   # avoids notification window when using RemoteControlWebUI
            f'{additional_args}',         # specified in settings
            f'{vproles}',                 # VP roles for this instance
            f'{friendly_name}',           # Stage Friendly Name
            f'{session_id}',              # Session ID.
            f'{max_gpu_count}',           # Max GPU count (mGPU)
            f'-dc_cfg="{cfg_file}"',      # nDisplay config file
            f'{render_api}',              # RHI (dx11/dx12/vulkan)
            f'{render_mode}',             # mono/...
            f'{use_all_cores}',           # -useallavailablecores
            f'{no_texture_streaming}',    # -notexturestreaming
            f'{no_sound}',                # -nosound
            f'{no_mcp}',                  # -nomcp
            f'{no_loading_screen}',       # -NoLoadingScreen
            f'{no_python}',               # -DisablePython
            f'-dc_node={self.name}',      # name of this node in the nDisplay cluster
            f'Log={self.log_filename}',   # log file
            f'{ini_engine}',              # Engine ini injections
            f'{ini_game}',                # Game ini injections
            f'{ini_input}',               # Input ini injections
            f'{unattended}',              # -unattended
            f'{no_screen_messages}',      # -NoScreenMessages
            f'{disable_ensures}',         # -handleensurepercent=0
            f'{udpm_transport_multi}',    # -UDPMESSAGING_TRANSPORT_MULTICAST=
            f'{udpm_transport_unicast}',  # -UDPMESSAGING_TRANSPORT_UNICAST=
            f'{udpm_transport_static}',   # -UDPMESSAGING_TRANSPORT_STATIC=
        ]

        # Add -multiprocess to all except the first co-hosted node, sorted by name.
        # Used to avoid race conditions when writing or deleting certain files on disk, such as
        # shader caches.

        shared_ip_connected_nodes = [
            node for node in self.get_connected_devices()
            if ip_address(node.address) == ip_address(self.address)
        ]

        # Launch command is also generated when devices are not yet connected, so empty shared_ip_connected_nodes
        # is a valid situation when the cluster is not being actually launched.
        if len(shared_ip_connected_nodes):
            first_cohosted_node = sorted(shared_ip_connected_nodes, key=lambda node: node.name)[0]

            if self != first_cohosted_node:
                args.append('-multiprocess')

        # Specify multiplayer with listen server mode by arg for clients if enabled
        if DevicenDisplay.csettings["multiplayer_mode"].get_value() == 'Listen server' and not self.is_primary():
            args.extend([
                '-dc_replicationserver_type listen'
            ])

        # fill in ExecCmds
        exec_cmds = self.csettings["ndisplay_exec_cmds"].get_value(self.name).copy()

        if DevicenDisplay.csettings['disable_all_screen_messages'].get_value() and 'DisableAllScreenMessages' not in exec_cmds:
            exec_cmds.append('DisableAllScreenMessages')

        # LiveLink preset
        livelink_preset_gamepath = DevicenDisplay.csettings["livelink_preset"].get_value(self.name)
        if livelink_preset_gamepath:
            exec_cmds.append(self.exec_command_for_livelink_preset(livelink_preset_gamepath))

        # Format exec cmds
        exec_cmds = [cmd for cmd in exec_cmds if len(cmd.strip())]

        if len(exec_cmds):
            exec_cmds_expanded = ','.join(exec_cmds)
            args.append(f'-ExecCmds="{exec_cmds_expanded}"')

        # when in fullscreen, the window parameters should not be passed
        if fullscreen:
            args.extend([
                '-fullscreen',
            ])
        else:
            args.extend([
                '-windowed',
                '-forceres',
                f'WinX={win_pos[0]}',
                f'WinY={win_pos[1]}',
                f'ResX={win_res[0]}',
                f'ResY={win_res[1]}',
            ])

        if headless:
            args.extend([
                '-RenderOffscreen',
            ])

        # MultiUser parameters
        if CONFIG.MUSERVER_AUTO_JOIN.get_value() and self.autojoin_mu_server.get_value():
            args.extend([
                '-CONCERTRETRYAUTOCONNECTONERROR',
                '-CONCERTAUTOCONNECT'])

        if CONFIG.MUSERVER_FILESHARING_ENABLED.get_value() and CONFIG.MUSERVER_FILESHARING_SRV.get_value():
            filesharing = CONFIG.MUSERVER_FILESHARING_SRV.get_value()
            args.extend([
                '-CONCERTFILESHARINGENABLED=1',
                f'-CONCERTFILESHAREPATH={filesharing}'])

        mu_server = switchboard_application.get_multi_user_server_instance()
        args.extend([
            f'-CONCERTSERVER="{mu_server.configured_server_name()}"',
            f'-CONCERTSESSION="{CONFIG.MUSERVER_SESSION_NAME.get_value()}"',
            f'-CONCERTDISPLAYNAME="{self.name}"',
            '-CONCERTISHEADLESS',
        ])

        # Device profile CVars.
        dp_cvars = []

        # If choosing unattended, complement with disabling toasts
        if DevicenDisplay.csettings['ndisplay_unattended'].get_value():
            dp_cvars.append('Slate.bAllowNotifications=0')

        # Insights traces parameters
        if CONFIG.INSIGHTS_TRACE_ENABLE.get_value() and not self.exclude_from_insights.get_value():
            LOGGER.warning(f"Unreal Insight Tracing is enabled for '{self.name}'. This may affect Unreal Engine performance.")

            remote_utrace_path = self.get_utrace_filepath()

            traces = CONFIG.INSIGHTS_TRACE_ARGS.get_value()

            args.extend([
                f'-tracefile="{remote_utrace_path}"',
                f'-trace="{traces}"'
            ])

            if CONFIG.INSIGHTS_STAT_EVENTS.get_value():
                args.append("-statnamedevents")

            # if bookmarks are enabled, also enable the vblank monitoring thread
            if any(bmtrace in traces.split(',') for bmtrace in ('bookmark','default')) :
                dp_cvars.append('nDisplay.sync.diag.VBlankMonitoring=1')

        # Graphics Adapter
        graphics_adapter = DevicenDisplay.csettings["graphics_adapter"].get_value(self.name)
        if graphics_adapter == 'Config':
            graphics_adapter = self.settings["config_graphics_adapter"].get_value(self.name)

        if graphics_adapter >= 0:
            dp_cvars.append(f'r.GraphicsAdapter={graphics_adapter}')

        # Always tell Chaos to be deterministic
        dp_cvars.append('p.Chaos.Solver.Deterministic=1')
        # Always enable LevelInstance World Editor Mode.
        dp_cvars.append('LevelInstance.ForceEditorWorldMode=1')

        # mediaprofile
        mediaprofile_gamepath = DevicenDisplay.csettings["mediaprofile"].get_value(self.name)
        if mediaprofile_gamepath:
            dp_cvars.append(self.dpcvar_for_mediaprofile(mediaprofile_gamepath))

        # GC
        if DevicenDisplay.csettings['incremental_gc_reachability'].get_value(self.name):
            dp_cvars.append('gc.AllowIncrementalReachability=1')

        # PSO Eviction time.
        #
        # PSO eviction may cause PSO rebuilding and may induce hitches.
        # Currently, only non-editor builds support pre-caching, so for now a way to avoid
        # these hitches is by lengthening the eviction time from the default 60 seconds.
        dp_cvars.append(f'r.pso.evictiontime={60*60}')

        # Add user set dp cvars, overriding any of the forced ones.
        user_dp_cvars = self.conform_dpcvars(self.csettings['ndisplay_dp_cvars'].get_value(self.name))
        dp_cvars = self.add_or_override_cvars(dp_cvars, user_dp_cvars)

        # add accumulated dpcvars to args
        if len(dp_cvars):
            args.append(f"-DPCVars=\"{','.join(dp_cvars)}\"")

        # Logging
        args.append(self.csettings['logging'].get_command_line_arg(
            override_device_name=self.name))

        path_to_exe = self.generate_exe_path()
        args_expanded = ' '.join(args)

        self.settings['ue_command_line'].update_value(
            f"{path_to_exe} {args_expanded}")

        return path_to_exe, args_expanded

    def on_send_file_complete(self, message):
        try:
            destination = message['destination']
            succeeded = message['bAck']
            error = message.get('error')
        except KeyError:
            LOGGER.error(
                f'Error parsing "send file complete" response ({message})')
            return

        ext = os.path.splitext(destination)[1].lower()

        if (ext == '.uasset') and self.pending_transfer_uasset:
            self.pending_transfer_uasset = False
            if succeeded:
                LOGGER.info(
                    f"{self.name}: nDisplay uasset successfully transferred "
                    f"to {destination} on host")
            else:
                LOGGER.error(
                    f"{self.name}: nDisplay uasset transfer failed: {error}")
        elif (ext == '.ndisplay') and self.pending_transfer_cfg:
            self.pending_transfer_cfg = False
            self.path_to_config_on_host = destination
            if succeeded:
                LOGGER.info(
                    f"{self.name}: nDisplay config file successfully "
                    f"transferred to {destination} on host")
            else:
                LOGGER.error(
                    f"{self.name}: nDisplay config file transfer "
                    f"failed: {error}")
        else:
            LOGGER.error(
                f"{self.name}: Unexpected send file completion "
                f"for {destination}")
            return

        if not (self.pending_transfer_cfg or self.pending_transfer_uasset):

            if DevicenDisplay.csettings['minimize_before_launch'].get_value(
                    True):
                self.minimize_windows()

            super().launch(map_name=self.map_name_to_launch)

    def on_get_sync_status(self, message):
        ''' Called when 'get sync status' is received. '''
        self.__class__.ndisplay_monitor.on_get_sync_status(
            device=self, message=message)

    def refresh_mosaics(self):
        ''' Request that the listener update its cached mosaic topologies. '''
        if self.unreal_client.is_connected:
            _, msg = message_protocol.create_refresh_mosaics_message()
            self.unreal_client.send_message(msg)

    def on_refresh_mosaics(self, message):
        try:
            if message['bAck'] is True:
                return
            elif message['error'] == 'Duplicate':
                LOGGER.warning('Duplicate "refresh mosaics" command ignored')
            else:
                LOGGER.error(f'"refresh mosaics" command rejected ({message})')
        except KeyError:
            LOGGER.error(
                f'Error parsing "refresh mosaics" response ({message})')

    def get_primary_node_device(self):
        for device in self.active_unreal_devices:
            if device.device_type == 'nDisplay' and device.is_primary():
                return device

    def is_primary(self):
        '''Check if the device is the primary or not'''
        return self.name == DevicenDisplay.csettings['primary_device_name'].get_value()

    def select_as_primary(self):
        ''' Selects this node as the primary node in the nDisplay cluster '''
        self.__class__.select_device_as_primary(self)

    def package_game(
            self,
            clientconfig: PackagingClientConfig = PackagingClientConfig.Default,
            dryrun: bool = False) -> uuid.UUID:
        ''' Packages the game using default settings '''

        # Determine the folder where to package the game.

        outdir = Path(self.get_packaged_game_path())

        # Note: is_file() can't work reliably if the file doesn't exist yet.
        if outdir.exists():
            if outdir.is_file():
                outdir = outdir.parent
        else:
            # if it ends with a separator, we know it is a directory
            is_surely_dir = str(outdir).endswith(Path().anchor)
            could_be_file = not is_surely_dir
            if could_be_file:
                outdir = outdir.parent

        if outdir == Path(""):
            raise FileNotFoundError("Invalid Packaged Game Path")

        program_name = UnrealJobs.PackageGame.value

        # Make sure there isn't any other program already running.
        # @todo Should include un-started programs
        if self.program_start_queue.running_programs_count():
            raise PermissionError("Another task is already running. Please try again later.")

        # @todo using the remote platform would be more correct.

        if sys.platform.startswith('linux'):
            platform = 'Linux'
        elif sys.platform.startswith('darwin'):
            platform = 'Mac'
        else:
            platform = 'Win64'

        if clientconfig == PackagingClientConfig.Default:
            clientconfigarg = ''
        else:
            clientconfigarg = f'-clientconfig={clientconfig.value}'

        args = [
            'BuildCookRun',
            f'-project="{CONFIG.UPROJECT_PATH.get_value(self.name)}"',
            '-noP4',
            f'-platform={platform}',
            clientconfigarg,
            '-cook',
            '-allmaps',
            '-build',
            '-stage',
            '-pack',
            '-archive',
            f'-archivedirectory="{str(outdir)}"'
        ]

        prog_exe = "RunUAT"

        if sys.platform.startswith('win'):
            prog_exe += ".bat"
        else:
            prog_exe += ".sh"

        prog_path = str(Path(CONFIG.ENGINE_DIR.get_value(self.name)).parent / prog_exe)
        prog_args = ' '.join(args)

        if dryrun:
            return uuid.uuid4()

        LOGGER.info(f'Packaging "{self.name}" with command: {prog_path} {prog_args}')

        puuid, msg = message_protocol.create_start_process_message(
            prog_path=prog_path,
            prog_args=prog_args,
            prog_name=program_name,
            caller=self.name,
            update_clients_with_stdout=True,
        )

        self.program_start_queue.add(
            ProgramStartQueueItem(
                name=program_name,
                puuid_dependency=None,
                puuid=puuid,
                msg_to_unreal_client=msg,
            ),
            unreal_client=self.unreal_client,
        )

        return puuid

    @classmethod
    def extract_configexport_from_uasset(cls, cfg_file) -> str:
        ''' Extract the configexport from the config uasset '''

        # Initialize uasset parser
        with open(cfg_file, 'rb') as file:
            aparser = UassetParser(file)

            # Check that the asset is of the right class, then find its
            # ConfigExport tag and parse it.

            for assetdata in aparser.aregdata:
                if assetdata.ObjectClassName in DeviceUnreal.NDISPLAY_CLASS_NAMES:
                    return assetdata.tags['ConfigExport']

        raise ValueError('Invalid nDisplay config .uasset')

    @classmethod
    def select_device_as_primary(cls, device):
        ''' Selects the given devices as the primary of the nDisplay cluster'''

        DevicenDisplay.csettings['primary_device_name'].update_value(
            device.name)

        devices = [
            dev for dev in cls.active_unreal_devices
            if dev.device_type == 'nDisplay']

        for dev in devices:
            if dev.widget:
                checked = (dev.name == device.name)
                dev.widget.primary_button.setChecked(checked)

    @classmethod
    def apply_local_overrides_to_config(cls, cfg_content, cfg_ext):
        ''' Applies supported local settings overrides to configuration '''

        if cfg_ext.lower() != '.ndisplay':
            return cfg_content

        data = json.loads(cfg_content)

        configversion = float(data['nDisplay']['version'])
        primaryNodeKey = 'masterNode' if configversion < 5.0 else 'primaryNode'

        # override primary node
        #

        nodes = data['nDisplay']['cluster']['nodes']

        primaryNodeId = data['nDisplay']['cluster'][primaryNodeKey]['id']

        activenodes = {}

        devices = [
            dev for dev in cls.active_unreal_devices
            if dev.device_type == 'nDisplay' or str(dev.device_type).startswith('nDisplay')]

        shown_unknown_node_warning = False

        for device in devices:
            # We exclude devices by disconnecting from them
            if device.is_disconnected:
                continue

            # Match local node with config by name
            node = nodes.get(device.name, None)

            # No name match, no node. Renaming nodes in Switchboard is not
            # currently supported.
            if node is None:
                LOGGER.error(
                    f'Skipped "{device.name}" because it did not match any '
                    'node in the nDisplay configuration')

                if not shown_unknown_node_warning:
                    shown_unknown_node_warning = True
                    QtWidgets.QMessageBox.warning(
                        None, 'Unknown nDisplay Node',
                        'One or more nDisplay devices have names that do not '
                        'correspond to nodes in the configuration. This may '
                        'prevent the cluster from launching correctly. Refer '
                        'to the log for more details.')

                continue

            # override address
            node['host'] = device.address

            # add to active nodes
            activenodes[device.name] = node

            # override primary node by name
            if (device.name == DevicenDisplay.csettings['primary_device_name'].get_value()):
                primaryNodeId = device.name

            # override GraphicsAdapter
            graphics_adapter = DevicenDisplay.csettings['graphics_adapter'].get_value(device.name)
            if graphics_adapter != 'Config':
                node['graphicsAdapter'] = int(graphics_adapter)

        data['nDisplay']['cluster']['nodes'] = activenodes
        data['nDisplay']['cluster'][primaryNodeKey]['id'] = primaryNodeId

        # override sync policy
        #

        render_sync_policy = DevicenDisplay.csettings['render_sync_policy'].get_value()

        new_rsp = data['nDisplay']['cluster']['sync']['renderSyncPolicy']
        
        if render_sync_policy == 'Nvidia':

            new_rsp['type'] = 'Nvidia'
            new_rsp['parameters'] = {
                "SwapGroup": "1",
                "SwapBarrier": "1"
            }

        elif render_sync_policy == 'Ethernet':

            new_rsp['type'] = 'Ethernet'
            new_rsp['parameters'] = {}

        elif render_sync_policy == 'None':

            new_rsp['type'] = 'None'
            new_rsp['parameters'] = {}

        data['nDisplay']['cluster']['sync']['renderSyncPolicy'] = new_rsp

        return json.dumps(data)

    @classmethod
    def parse_config_uasset(cls, cfg_file):
        ''' Parses nDisplay config file of type uasset '''
        jsstr = cls.extract_configexport_from_uasset(cfg_file)
        return cls.parse_config_json_string(jsstr)

    @classmethod
    def parse_config_json_string(cls, jsstr):
        '''
        Parses nDisplay config JSON string, returning (nodes, uasset_path).
        '''

        js = json.loads(jsstr)

        config = DisplayConfig()

        configversion = float(js['nDisplay']['version'])
        primaryNodeKey = 'masterNode' if configversion < 5.0 else 'primaryNode'

        cnodes = js['nDisplay']['cluster']['nodes']
        primaryNode = js['nDisplay']['cluster'][primaryNodeKey]
        config.uasset_path = js['nDisplay'].get('assetPath')

        for name, cnode in cnodes.items():
            kwargs = {"ue_command_line": ""}

            winx = int(cnode['window'].get('x', 0))
            winy = int(cnode['window'].get('y', 0))
            resx = int(cnode['window'].get('w', 0))
            resy = int(cnode['window'].get('h', 0))

            kwargs["window_position"] = (winx, winy)
            kwargs["window_resolution"] = (resx, resy)
            # Note the capital 'S', 'H'...
            kwargs["fullscreen"] = bool(cnode.get('fullScreen', False))
            kwargs["headless"] = bool(cnode.get('renderHeadless', False))
            kwargs["config_graphics_adapter"] = int(cnode.get('graphicsAdapter', -1))

            primary = True if primaryNode['id'] == name else False

            config.nodes.append({
                "name": name,
                "address": cnode['host'],
                "primary": primary,
                "port_ce": int(primaryNode['ports']['ClusterEventsJson']),
                "port_ce_binary": int(primaryNode['ports']['ClusterEventsBinary']),
                "kwargs": kwargs,
            })

        return config

    @classmethod
    def parse_config_json(cls, cfg_file):
        ''' Parses nDisplay config file of type json '''
        jsstr = open(cfg_file, 'r').read()
        return cls.parse_config_json_string(jsstr)

    @classmethod
    def parse_config(cls, cfg_file) -> DisplayConfig:
        '''
        Parses an nDisplay file and returns the nodes with the relevant
        information.
        '''
        ext = os.path.splitext(cfg_file)[1].lower()

        if ext == '.ndisplay':
            return cls.parse_config_json(cfg_file)
        elif ext == '.uasset':
            return cls.parse_config_uasset(cfg_file)

        raise Exception(f'Unknown config extension "{ext}"')

    def update_settings_controlled_by_config(self, cfg_file):
        '''
        Updates settings that are exclusively controlled by the config file.
        '''
        try:
            config = self.__class__.parse_config(cfg_file)
        except Exception as e:
            LOGGER.error(f"Error parsing config file {cfg_file}: {e}")
            return

        self.bp_object_path = config.uasset_path
        nodes = config.nodes

        # find which node is self:
        try:
            menode = next(node for node in nodes if node['name'] == self.name)
        except StopIteration:
            LOGGER.error(f"{self.name} not found in config file {cfg_file}")
            return

        self.nodeconfig = menode

        self.settings['window_position'].update_value(
            menode['kwargs'].get("window_position", (0, 0)))
        self.settings['window_resolution'].update_value(
            menode['kwargs'].get("window_resolution", (100, 100)))
        self.settings['fullscreen'].update_value(
            menode['kwargs'].get("fullscreen", False))
        self.settings['headless'].update_value(
            menode['kwargs'].get("headless", False))
        self.settings['config_graphics_adapter'].update_value(
            menode['kwargs'].get("config_graphics_adapter", -1))

    def get_connected_devices(self):
        ''' Returns a list with the connected devices/nodes
        '''
        nodes = []
        for device in self.active_unreal_devices:
            is_device_connected = (device.status == DeviceStatus.CLOSED) or (device.status == DeviceStatus.OPEN)
            if (device.device_type == "nDisplay") and is_device_connected:
                nodes.append(device)
        return nodes

    def launch(self, map_name):

        if not self.check_settings_valid():
            LOGGER.error(f"{self.name}: Not launching due to invalid settings")
            self.widget._close()
            return

        # Ensure that one of the devices is set as the primary node.
        nodes = self.get_connected_devices()

        assert len(nodes) > 0  # we wouldn't be launching otherwise

        primary_name = DevicenDisplay.csettings['primary_device_name'].get_value()
        primary = None

        for node in nodes:
            if node.name == primary_name:
                primary = node
                break

        # if there is no primary, assign one
        if primary is None:
            nodes[0].select_as_primary()

        # cache the map to launch
        self.map_name_to_launch = map_name

        # Update settings controlled exclusively by the nDisplay config file.
        cfg_file = DevicenDisplay.csettings['ndisplay_config_file'].get_value(
            self.name)
        try:
            self.update_settings_controlled_by_config(cfg_file)
        except Exception:
            LOGGER.error(
                f"{self.name}: Could not update from '{cfg_file}' before "
                "launch. \n\n=== Traceback BEGIN ===\n"
                f"{traceback.format_exc()}=== Traceback END ===\n")
            self.widget._close()
            return

        # Transfer config file
        if not os.path.exists(cfg_file):
            LOGGER.error(
                f"{self.name}: Could not find nDisplay config file "
                f"at {cfg_file}")
            self.widget._close()
            return

        cfg_ext = os.path.splitext(cfg_file)[1].lower()

        # If the config file is a uasset, we send it as well as the
        # extracted JSON.
        if cfg_ext == '.uasset':
            cfg_ext = '.ndisplay'
            cfg_content = self.__class__.extract_configexport_from_uasset(
                cfg_file).encode('utf-8')
        else:
            with open(cfg_file, 'rb') as f:
                cfg_content = f.read()

        # Apply local overrides to configuration (e.g. addresses)
        try:
            cfg_content = self.__class__.apply_local_overrides_to_config(
                cfg_content, cfg_ext).encode('utf-8')
        except Exception:
            LOGGER.error(
                'Could not parse config data when trying to override with '
                'local settings')
            self.widget._close()
            return

        cfg_destination = f"%TEMP%/ndisplay/%RANDOM%{cfg_ext}"

        self.pending_transfer_cfg = True
        _, cfg_msg = message_protocol.create_send_filecontent_message(
            cfg_content, cfg_destination)
        self.unreal_client.send_message(cfg_msg)

        overwrite_ndc = DevicenDisplay.csettings['overwrite_remote_ndc_asset'].get_value(self.name)

        # When desired, send .uasset but only if not launching packaged
        if overwrite_ndc and self.bp_object_path and not self.is_packaged_game():
            local_project_path = os.path.dirname(
                CONFIG.UPROJECT_PATH.get_value())
            dest_project_path = os.path.dirname(
                CONFIG.UPROJECT_PATH.get_value(self.name))

            local_uasset_path = CONFIG.ue_plugin_mgr.content_to_file_path(
                PurePosixPath(self.bp_object_path)).with_suffix('.uasset')

            uasset_project_relative = os.path.relpath(local_uasset_path,
                                                      local_project_path)

            dest_uasset_path = os.path.join(dest_project_path,
                                            uasset_project_relative)

            if os.path.isfile(local_uasset_path):
                self.pending_transfer_uasset = True
                _, uasset_msg = message_protocol.create_send_file_message(
                    str(local_uasset_path), dest_uasset_path,
                    force_overwrite=True)
                self.unreal_client.send_message(uasset_msg)
            else:
                LOGGER.warning(
                    f'{self.name}: Could not find nDisplay '
                    f'uasset at {local_uasset_path}')

    @classmethod
    def plug_into_ui(cls, menubar, tabs):
        '''
        Implementation of base class function that allows plugin to inject
        UI elements.
        '''
        cls.create_monitor_if_necessary()

        # Create Monitor UI if it doesn't exist
        if not cls.ndisplay_monitor_ui:
            cls.ndisplay_monitor_ui = nDisplayMonitorUI(parent=tabs, monitor=cls.ndisplay_monitor)
            cls.ndisplay_monitor_ui.replace_exec_history(DevicenDisplay.csettings['cluster_exec_history'].get_value())
            cls.ndisplay_monitor_ui.signal_exec_history_changed.connect(lambda: cls._handle_exec_history_changed())

        # Create Tools UI if it doesn't exist
        if not cls.ndisplay_tools_ui:
            cls.ndisplay_tools_ui = nDisplayToolsUI(
                parent=tabs, plugin_ndisplay=cls)

        # Add our UIs to the main tabs in the UI
        tabs.addTab(cls.ndisplay_monitor_ui, 'nDisplay &Monitor')
        tabs.addTab(cls.ndisplay_tools_ui, 'nDisplay &Tools')

    @classmethod
    def _handle_exec_history_changed(cls):
        if cls.ndisplay_monitor_ui:
            DevicenDisplay.csettings['cluster_exec_history'].update_value(
                list(cls.ndisplay_monitor_ui.exec_history.values()))

    @classmethod
    def added_device(cls, device):
        '''
        Implementation of base class function. Called when one of our plugin
        devices has been added.
        '''
        super().added_device(device)

        if not cls.ndisplay_monitor:
            return

        cls.ndisplay_monitor.added_device(device)

    @classmethod
    def removed_device(cls, device):
        '''
        Implementation of base class function. Called when one of our plugin
        devices has been removed.
        '''
        super().removed_device(device)

        if not cls.ndisplay_monitor:
            return

        cls.ndisplay_monitor.removed_device(device)

    @classmethod
    def send_cluster_event(cls, devices: List[DeviceUnreal], cluster_event):
        '''
        Sends a cluster event (to the primary node, which will replicate to the rest
        of the cluster).
        '''

        # find the primary node

        primary = None

        for device in devices:
            if (device.name ==
                    DevicenDisplay.csettings[
                        'primary_device_name'].get_value()):
                primary = device
                break

        if primary is None:
            LOGGER.warning('Could not find primary device when trying to send '
                           'cluster event. Please make sure the primary '
                           'device is marked as such.')
            raise ValueError

        if not primary.is_connected_and_authenticated():
            LOGGER.warning('Primary node must be connected and running to send a cluster event')
            raise ValueError

        if not primary.program_start_queue.running_programs_named(UnrealJobs.Unreal.value):
            LOGGER.warning('Primary node must be connected and running to send a cluster event')
            raise ValueError

        msg = bytes(json.dumps(cluster_event), 'utf-8')
        msg = struct.pack('I', len(msg)) + msg

        # We copy the original primary's cluster event port (port_ce) to the
        # config of all nodes, so we can use the port from the overridden
        # primary.
        port = primary.nodeconfig['port_ce']

        # Use the overridden address of this node
        address = primary.address

        sock = socket.socket(socket.AF_INET, socket.SOCK_STREAM)
        sock.connect((address, port))
        sock.send(msg)

    @classmethod
    def soft_kill_cluster(cls, devices):
        ''' Kills the cluster by sending a message to the primary. '''
        LOGGER.info('Issuing nDisplay cluster soft kill')

        quit_event = {
            "bIsSystemEvent": "true",
            "Category": "nDisplay",
            "Type": "control",
            "Name": "quit",
            "Parameters": {},
        }

        cls.send_cluster_event(devices, quit_event)

    @classmethod
    def console_exec_cluster(cls, devices, exec_str, executor=''):
        ''' Executes a console command on the cluster. '''
        LOGGER.info(f"Issuing nDisplay cluster console exec: '{exec_str}'")

        exec_event = {
            "bIsSystemEvent": "true",
            "Category": "nDisplay",
            "Type": "control",
            "Name": "console exec",
            "Parameters": {
                "ExecString": exec_str,
                "Executor": executor,
            },
        }

        cls.send_cluster_event(devices, exec_event)

    @classmethod
    def close_all(cls, devices):
        ''' Closes all devices in the plugin '''
        try:
            cls.soft_kill_cluster(devices)

            for device in devices:
                device.device_qt_handler.signal_device_closing.emit(device)
        except Exception:
            LOGGER.warning("Could not soft kill the cluster")

    # Class variables for managing non-blocking cluster shutdown
    _cluster_shutdown_timer = None
    _devices_pending_shutdown = []

    @classmethod
    def close_all_devices(cls, devices: list[DevicenDisplay]):
        '''
        Override to use nDisplay's cluster-aware shutdown with timeout fallback.
        First tries cluster soft kill, then falls back to individual closes after timeout.
        Uses half the configured timeout for each phase to keep total time consistent.
        '''
        if not devices:
            return

        full_timeout = DeviceUnreal.csettings['soft_kill_timeout'].get_value()
        cluster_timeout = full_timeout / 2.0  # Half timeout for cluster phase

        try:
            # Issue cluster command, which is preferred because it issues the quit command at the same time on all
            # nodes which helps normal shutdown without thread barrier related deadlocks.
            cls.soft_kill_cluster(devices)

            # Since the cluster command may not succeed, after a timeout we issue the regular kill command via 
            # SwitchboardListener on the nodes that are still running.

            cls._devices_pending_shutdown = devices.copy()

            # Cancel any existing timeout timer
            if cls._cluster_shutdown_timer:
                cls._cluster_shutdown_timer.stop()
                cls._cluster_shutdown_timer = None

            # Create new timer for the fallback kill command - use half the timeout
            cls._cluster_shutdown_timer = QtCore.QTimer()
            cls._cluster_shutdown_timer.timeout.connect(cls._on_cluster_shutdown_timeout)
            cls._cluster_shutdown_timer.setSingleShot(True)
            cls._cluster_shutdown_timer.start(int(cluster_timeout * 1000))  # Convert to ms

            # Set devices to CLOSING state (disables buttons and shows proper state)
            for device in devices:
                try:
                    if device.status in (DeviceStatus.OPEN, DeviceStatus.READY):
                        device.status = DeviceStatus.CLOSING
                except Exception:
                    pass

        except Exception:
            LOGGER.warning("Cluster event for shutdown could not be issued, falling back to immediate device close")

            # Issue an immediate regular device shutdown if the cluster command fails to execute.
            cls._close_nodes_individually(devices)

    @classmethod
    def _on_cluster_shutdown_timeout(cls):
        '''Called after timeout - check if devices are still running and force close if needed'''
        if not cls._devices_pending_shutdown:
            return

        # Check which devices are still running and force close them
        devices_still_running = []
        for device in cls._devices_pending_shutdown:
            # Check if device is still in CLOSING state (meaning cluster shutdown didn't complete)
            if device.status == DeviceStatus.CLOSING:
                devices_still_running.append(device)

        if devices_still_running:
            cls._close_nodes_individually(devices_still_running)

        cls._devices_pending_shutdown.clear()
        cls._cluster_shutdown_timer = None

    @classmethod
    def _close_nodes_individually(cls, devices):
        '''Force close individual devices using the standard close mechanism with remaining timeout'''
        full_timeout = DeviceUnreal.csettings['soft_kill_timeout'].get_value()
        remaining_timeout = full_timeout / 2.0  # Use remaining half of timeout

        for device in devices:
            try:
                # Call device.close() directly with the remaining timeout
                if device.status in (DeviceStatus.OPEN, DeviceStatus.READY, DeviceStatus.CLOSING):
                    device.close(soft_kill_timeout=remaining_timeout)
            except Exception:
                pass

    # ~ Begin Device Interface
    @classmethod
    def all_devices_added(cls):
        ''' Device interface implementation '''

        super().all_devices_added()

        devices = [dev for dev in cls.active_unreal_devices
                   if dev.device_type == 'nDisplay']

        # If no devices were added, there is nothing to do.
        if len(devices) == 0:
            return

        # Verify that there is a device that matches our primary selection
        primary_device_name = DevicenDisplay.csettings['primary_device_name'].get_value()
        primary_found = False

        for device in devices:
            if device.name == primary_device_name:
                primary_found = True
                break

        # If there isn't a primary selection (e.g. old config), try to assign
        # one based on the nDisplay config.
        if not primary_found:

            cfg_file = DevicenDisplay.csettings['ndisplay_config_file'].get_value()

            try:
                config = DevicenDisplay.parse_config(cfg_file)
            except (IndexError, KeyError):
                LOGGER.error(f"Error parsing nDisplay config file {cfg_file}")
                return

            for node in config.nodes:
                if node['primary']:
                    for device in devices:
                        if node['name'] == device.name:
                            cls.select_device_as_primary(device)
                            primary_found = True
                            break
                    break

            # If we still don't have a primary device, log the error.
            if not primary_found:
                LOGGER.error('Could not find or assign primary device in the '
                             f'current devices and config file "{cfg_file}"')

    def device_widget_registered(self, device_widget):
        ''' Device interface method '''

        super().device_widget_registered(device_widget)

        # hook to its primary_button clicked event
        device_widget.signal_device_widget_primary.connect(
            self.select_as_primary)

        # Update the state of the button depending on whether this device is
        # the primary or not.
        device_widget.primary_button.setChecked(self.is_primary())

    def on_device_name_changed(self, new_name):
        super().on_device_name_changed(new_name)
        if self.__class__.ndisplay_monitor:
            self.__class__.ndisplay_monitor.update_device_name_and_address(self)

    def on_address_changed(self, new_address):
        super().on_address_changed(new_address)

        if self.__class__.ndisplay_monitor:
            self.__class__.ndisplay_monitor.update_device_name_and_address(self)
    # ~ End Device Interface

    # ~ Begin DeviceUnreal Interface
    @QtCore.Slot()
    def _on_listener_connected(self) -> None:
        super()._on_listener_connected()
        if self.__class__.ndisplay_monitor:
            self.__class__.ndisplay_monitor.on_device_connected(device=self)

    def on_listener_disconnect(self, unexpected=False, exception=None) -> None:
        super().on_listener_disconnect(unexpected, exception)
        if self.__class__.ndisplay_monitor:
            self.__class__.ndisplay_monitor.on_device_disconnected(device=self)

    def on_unreal_started(self) -> None:
        super().on_unreal_started()

        # Always try to disable Full Screen Optimizations. If this is removed, you can add back
        # the "Disable FSO" button using ndisplay_monitor.show_disable_fso_btn
        self.disable_fso()
    # ~ End DeviceUnrealInterface
