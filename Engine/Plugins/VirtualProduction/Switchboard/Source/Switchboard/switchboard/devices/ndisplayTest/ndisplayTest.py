# Copyright Epic Games, Inc. All Rights Reserved.

import copy
import os
import re
import xml.etree.ElementTree as ET

from pathlib import Path
from switchboard.config import CONFIG
from switchboard.switchboard_logging import LOGGER


####################################################
# A wrapper class for XML based nDisplay tests
####################################################
class nDisplayTest:

    def __init__(self, xml_path):
        self._file_path = xml_path
        self._tree = None
        self._root = None
        self._ns = {"bg": "http://www.epicgames.com/BuildGraph"}
        self._properties = { "RootDir": str(Path(CONFIG.ENGINE_DIR.get_value()).parent) }
        self._load_xml()

    def _load_xml(self):
        ''' Attempts to load and parse the an XML file '''

        try:
            self._tree = ET.parse(self._file_path)
            self._root = self._tree.getroot()
            # Extract all properties into a dictionary
            self._properties |= {
                prop.attrib["Name"]: prop.attrib["Value"]
                for prop in self._root.findall(".//bg:Property", self._ns)}
        except:
            self._tree = None
            self._root = None
            LOGGER.error(f"Could not read test file: '{self._file_path}'")

    @classmethod
    def parse_test(cls, file_path) -> "nDisplayTest":
        ''' Parses an nDisplay test file into a test wrapper '''

        try:
            return cls(file_path) if os.path.exists(file_path) else None
        except Exception:
            LOGGER.error(f"Couldn't parse nDisplay test file: {file_path}")
            return None

    def duplicate(self) -> "nDisplayTest":
        ''' Duplicates this instance, returns the new one '''

        root_clone = copy.deepcopy(self._root)
        cls = self.__class__
        instance = cls.__new__(cls)  # Bypass __init__
        instance._tree = ET.ElementTree(root_clone)
        instance._root = root_clone
        instance._file_path = self._file_path
        instance._ns = self._ns
        instance._properties = self._properties
        return instance

    def update_properties(self, updates: dict[str, str]):
        ''' Updates properties with the new values (in memory) '''

        # Build property map
        prop_map = {
            prop.attrib.get("Name"): prop
            for prop in self._root.findall('.//bg:Property', self._ns)
        }

        # Iterate & update
        for name, new_val in updates.items():
            prop = prop_map.get(name)
            if prop is not None:
                prop.set("Value", new_val)

    def to_string(self):
        ''' Returns current XML data as string '''
        ET.register_namespace('', "http://www.epicgames.com/BuildGraph")
        return ET.tostring(self._root, encoding='utf-8', xml_declaration=True).decode('utf-8') if self.is_valid() else None

    def to_bytes(self):
        ''' Returns current XML data as bytes '''
        ET.register_namespace('', "http://www.epicgames.com/BuildGraph")
        return ET.tostring(self._root, encoding='utf-8', xml_declaration=True) if self.is_valid() else None

    def save(self, path):
        ''' Saves current XML data to file '''
        if self.is_valid():
            self._tree.write(path, encoding="utf-8", xml_declaration=True)
        
    def is_valid(self):
        ''' Checks if XML was successfully loaded, and contains valid data '''
        return self._tree is not None and self._root is not None

    def _resolve_path(self, source):
        ''' Resolves placeholders in paths from XML '''
        output = re.sub(r"\$\((.*?)\)", lambda m: self._properties.get(m.group(1), m.group(0)), source)
        return os.path.normpath(output)

    def _get_property(self, property_name):
        ''' Returns value of a specified XML property '''
        return self._properties.get(property_name, "") if self.is_valid() else None

    def get_test_path(self):
        ''' Returns original test file path '''
        return self._file_path

    def get_ndisplay_config_path(self):
        ''' Returns nDisplay config paths specified in an XML test '''

        # Search for the config path property
        config_path = self._get_property("DisplayConfigPath")

        # Resolve placeholders if there are any
        if config_path:
            config_path = self._resolve_path(config_path)

        return config_path

    def get_test_map(self):
        ''' Returns test map if specified '''
        return self._get_property("MapOverride")

    def get_script_dir(self):
        ''' Returns script dir if specified '''
        return self._get_property("ICVFXScriptDir")

