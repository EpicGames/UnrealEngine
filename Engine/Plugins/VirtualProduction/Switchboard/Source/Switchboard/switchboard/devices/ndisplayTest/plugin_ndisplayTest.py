# Copyright Epic Games, Inc. All Rights Reserved.

import glob
import os
import threading
import time
import uuid

from datetime import datetime
from pathlib import Path
from PySide6 import QtCore, QtWidgets
from typing import Callable

from switchboard import message_protocol
from switchboard import switchboard_utils as sb_utils
from switchboard import switchboard_widgets as sb_widgets
from switchboard.config import CONFIG, BoolSetting, IntSetting, StringSetting, map_name_is_valid
from switchboard.devices.device_base import DeviceStatus
from switchboard.devices.device_widget_base import AddDeviceDialog
from switchboard.devices.ndisplay.plugin_ndisplay import DevicenDisplay, DeviceWidgetnDisplay, LaunchMode
from switchboard.devices.unreal.plugin_unreal import ProgramStartQueueItem, UnrealJobs
from switchboard.switchboard_logging import LOGGER

from .ndisplayTest import nDisplayTest


####################################################
# Add nDisplay Test device dialog
####################################################
class AddnDisplayTestDialog(AddDeviceDialog):
    
    def __init__(self, existing_devices, parent=None):

        # Initialization flag to ignore any virtual calls during initialization
        self.baseInitializing = True

        super().__init__(
            device_type="nDisplayTest",
            existing_devices=existing_devices,
            parent=parent)

        # Now clear the guard flag
        self.baseInitializing = False

        # Initialize members
        self.existing_devices = existing_devices
        self.test_dir = self.get_default_test_dir()
        LOGGER.info(f"Test location: '{self.test_dir}'")

        # Initialize GUI
        self.initialize_widgets()

        # Update "Ok" button state
        self.refresh_validity()


    def initialize_widgets(self):
        ''' Performs widgets initialization '''

        # Set enough width for a decent file path length
        self.setMinimumWidth(640)

        # Remove unneeded base dialog layout rows
        self.form_layout.removeRow(self.name_field)
        self.form_layout.removeRow(self.address_field)
        self.name_field = None
        self.address_field = None

        # Initialize grid layout
        grid_layout = QtWidgets.QGridLayout()

        # Test Dir: Label
        lblTestDir = QtWidgets.QLabel(self, text="Tests Location")

        # Test Dir: Text field
        self.txtTestDir = QtWidgets.QLineEdit()
        self.txtTestDir.setReadOnly(False)
        self.txtTestDir.setText(self.test_dir)
        self.txtTestDir.textChanged.connect(lambda text: self.set_test_location(str(text).strip(' "'), False))

        # Test Dir: Button to browse for a test directory
        self.btnBrowseTestDir = QtWidgets.QPushButton(self, text="Browse")
        self.btnBrowseTestDir.clicked.connect(self.on_clicked_btnBrowseTestDir)

        # Test Dir: Row widget
        test_dir_layout = QtWidgets.QHBoxLayout()
        test_dir_layout.addWidget(self.txtTestDir)
        test_dir_layout.addWidget(self.btnBrowseTestDir)

        # Test File: Label
        lblTestFile = QtWidgets.QLabel(self, text="Test File")

        # Test File: Combobox with the files
        self.cbTests = sb_widgets.SearchableComboBox(self)
        self.cbTests.setSizePolicy(QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Preferred)
        self.cbTests.setEditable(False)

        # Test File: Row widget
        test_file_layout = QtWidgets.QHBoxLayout()
        test_file_layout.addWidget(self.cbTests)

        # Initialize grid layout
        grid_layout.addWidget(lblTestDir, 0, 0)
        grid_layout.addLayout(test_dir_layout, 0, 1)
        grid_layout.addWidget(lblTestFile, 1, 0)
        grid_layout.addLayout(test_file_layout, 1, 1)

        # Add the grid layout to the form layout
        self.form_layout.addRow(grid_layout)

        # Add a spacer right before the ok/cancel buttons
        spacer_layout = QtWidgets.QHBoxLayout()
        spacer_layout.addItem(
            QtWidgets.QSpacerItem(20, 20, QtWidgets.QSizePolicy.Minimum, QtWidgets.QSizePolicy.Expanding))
        self.form_layout.addRow("", spacer_layout)

        # Add warning if there are any devices exist
        if self.existing_devices:
            self.layout().addWidget(
                QtWidgets.QLabel("Warning! All existing nDisplay devices will be replaced."))

        # Update GUI contents
        self.refresh_test_list()

    def set_test_location(self, test_location, update_editbox = True):
        ''' Sets new test location and re-scans for available tests '''
        LOGGER.info(f"Set new test location: {test_location}")
        self.test_dir = test_location
        if update_editbox:
            self.txtTestDir.setText(self.test_dir)
        self.refresh_test_list()

    def refresh_test_list(self):
        ''' Parses the currently selected test directory for all available test files (xml)'''

        # Clear the current test list
        self.cbTests.clear()

        # Find all .xml files, and extract names only
        test_files = glob.glob(f"{self.test_dir}/*.xml")
        test_file_names = [os.path.splitext(os.path.basename(file))[0] for file in test_files]

        # Update the combo data
        self.cbTests.addItems(test_file_names)

    def get_default_test_dir(self):
        ''' Returns default test location '''

        # Project root & potential test locations
        project_dir = CONFIG.get_project_dir()
        build_dir = os.path.join(project_dir, "Build")
        automation_dir = os.path.join(build_dir, "Automation")

        # Search for the existing test location
        result_dir = next((p for p in [automation_dir, build_dir] if os.path.exists(p)), project_dir)

        return result_dir

    def on_clicked_btnBrowseTestDir(self):
        ''' Opens a folder dialog to browse for the test directory '''

        # Chose new location
        dir_old = self.test_dir
        dir_new = QtWidgets.QFileDialog.getExistingDirectory(self, "Choose tests location", self.test_dir)

        # Update & refresh if new location is chosen
        if os.path.exists(dir_new):
            dir_new = os.path.normpath(dir_new)
            if dir_old != dir_new:
                self.set_test_location(dir_new)
        
        # Update "Ok" button state
        self.refresh_validity()

    def get_selected_test_file(self):
        ''' Returns the test chosen '''
        test_name = self.cbTests.currentText()
        return os.path.join(self.test_dir, test_name + ".xml")

    def devices_to_add(self):
        ''' [override] Provides a list of devices to add '''

        # Get selected test file path
        test_path = self.get_selected_test_file()
        if not test_path:
            LOGGER.error(f"Wrong test path: {test_path}")
            return []

        # Parse the test file
        test_chosen = nDisplayTest.parse_test(test_path)
        if not test_chosen:
            LOGGER.error(f"Couldn't parse test file: {test_path}")
            return []

        # Extract nDisplay config from the test file
        config_path = test_chosen.get_ndisplay_config_path()
        if not config_path:
            LOGGER.error(f"No nDisplay configuration file found in test: {test_path}")
            return []

        # Pass data to the device class
        DevicenDisplay.csettings['ndisplay_config_file'].update_value(config_path)
        DevicenDisplay.csettings['launch_mode'].update_value(LaunchMode.Standalone.value)
        DevicenDisplayTest.tsettings['ndisplay_test_file'].update_value(test_path)
        DevicenDisplayTest.current_test = test_chosen

        # Parse nDisplay config in order to get the list of devices
        try:
            devices = DevicenDisplay.parse_config(os.path.normpath(config_path)).nodes
            if len(devices) == 0:
                LOGGER.error(f"Could not read any devices in nDisplay config: {config_path}")

            # Pass P-node ID
            for node in devices:
                if node['primary']:
                    DevicenDisplay.csettings['primary_device_name'].update_value(node['name'])
                    break
                
            return devices

        except (IndexError, KeyError, FileNotFoundError):
            LOGGER.error(f"Error parsing nDisplay config: {config_path}")
            return []

    def devices_to_remove(self):
        ''' [override] Provides a list of devices to remove '''
        # Remove all existing devices
        return self.existing_devices

    def refresh_validity(self):
        ''' [override] Updates the Ok button state '''

        # Ignore calls from the base constructor
        if self.baseInitializing:
            return

        # Activate the Ok button if any test selected, otherwise deactivate
        self.ok_btn.setEnabled(bool(self.cbTests.currentText().strip()))


####################################################
# nDisplay Test device widget
####################################################
class DeviceWidgetnDisplayTest(DeviceWidgetnDisplay):

    signal_open_test_artifacts = QtCore.Signal(object)

    def populate_context_menu(self, cmenu: QtWidgets.QMenu):
        ''' [override] Called to populate the given context menu with any desired actions '''
        cmenu.addAction("Open fetched log", lambda: self.signal_open_last_log.emit(self))
        cmenu.addAction("Open fetched trace", lambda: self.signal_open_last_trace.emit(self))
        cmenu.addAction("Show artifacts", lambda: self.signal_open_test_artifacts.emit(self))
        cmenu.addAction("Copy last launch command", lambda: self.signal_copy_last_launch_command.emit(self))


####################################################
# nDisplay Test device
####################################################
class DevicenDisplayTest(DevicenDisplay):

    add_device_dialog = AddnDisplayTestDialog
    current_test: nDisplayTest = None
    last_launch_time: datetime = None
    artifacts_dir = None
    addr_run_tracking = { }
    addr_run_tracking_lock = threading.Lock()

    tsettings = {
        'ndisplay_test_file': StringSetting(
            attr_name="ndisplay_test_file",
            nice_name="nDisplay Test File",
            value="",
            tool_tip="Path to nDisplay test file",
            allow_reset=False,
            is_read_only=True,
            category="General Settings",
        ),
        'game_target_override': StringSetting(
            attr_name="game_target_override",
            nice_name="Override Target",
            value="",
            tool_tip="Game target name. By default %ProjectName% is used.",
            allow_reset=True,
            is_read_only=False,
            category="Test Configuration",
        ),
        'editor_target_override': StringSetting(
            attr_name="editor_target_override",
            nice_name="Override Editor Target",
            value="",
            tool_tip="Editor target name. By default %ProjectName%Editor is used.",
            allow_reset=True,
            is_read_only=False,
            category="Test Configuration",
        ),
        'test_build_info': StringSetting(
            attr_name="test_build_info",
            nice_name="Test Build Info",
            value="",
            tool_tip="Custom test information. If not set, engine&project CL will be specified.",
            allow_reset=True,
            is_read_only=False,
            category="Test Configuration",
        ),
        'test_multi_node_delay': IntSetting(
            attr_name="test_multi_node_delay",
            nice_name="Multi-nodes Start Delay",
            value=5,
            tool_tip=("Delay between starting multiple nodes on the same address"),
            category="Test Configuration",
        ),
        'test_run_boot_editor': BoolSetting(
            attr_name='test_run_boot_editor',
            nice_name='Run Editor Boot Test',
            value=False,
            tool_tip=('Whether a short editor session should run before actual test'),
            category="Test Configuration",
        ),
        'report_24fps_format': BoolSetting(
            attr_name='report_24fps_format',
            nice_name='Generate 24 FPS Report ',
            value=False,
            tool_tip=('Should generate 24-fps based report (experimental)'),
            category="Test Configuration",
        ),
    }


    def __init__(self, name, address, **kwargs):
        super().__init__(name, address, **kwargs)

        # "Send file" is an intermediate step on the way to launch a test
        self.unreal_client.delegates['send file complete'] = self.on_send_file_complete

        # Clean up temporaries
        self.reset_launch_temporary_data()

        # Load&parse test data if not done yet
        if(self.current_test is None):
            test_file = self.tsettings['ndisplay_test_file'].get_value()
            self.current_test = nDisplayTest.parse_test(test_file)
            if self.current_test is None:
                LOGGER.error(f'Could not initialzie test data. Something is wrong with test file: "{test_file}"')

    @classmethod
    def plugin_settings(cls):
        ''' Returns common settings that belong to all devices of this class. '''
        return [
            cls.tsettings['ndisplay_test_file'],
            cls.tsettings['game_target_override'],
            cls.tsettings['editor_target_override'],
            cls.tsettings['test_build_info'],
            cls.tsettings['test_multi_node_delay'],
            cls.tsettings['test_run_boot_editor'],
            cls.tsettings['report_24fps_format'],
        ] + DevicenDisplay.plugin_settings()

    @property
    def test_name(self):
        ''' Returns current test name '''
        file_path = self.tsettings['ndisplay_test_file'].get_value()
        file_name = os.path.splitext(os.path.basename(file_path))[0]
        return file_name
    
    def reset_launch_temporary_data(self):
        ''' Cleans up the temporary data used to launch a test '''
        self.is_sending_cfg_file = False
        self.remote_path_cfg = None
        self.is_sending_test_file = False
        self.remote_path_test = None

    def device_widget_registered(self, device_widget):
        '''[override]  Device interface method '''
        # Initialize base first
        super().device_widget_registered(device_widget)
        # Initialize local signals
        device_widget.signal_open_test_artifacts.connect(self.on_open_test_artifacts)

    def on_open_test_artifacts(self):
        ''' Opens test artifacts location in a browser '''
        if self.artifacts_dir and self.artifacts_dir.exists():
            os.startfile(self.artifacts_dir)
        else:
            LOGGER.warning('Output directory not found.')
            
    def get_connected_devices(self):
        ''' Returns a list with the connected devices/nodes '''

        # Find all nDisplayTest devices currently connected
        nodes = []
        for device in self.active_unreal_devices:
            is_device_connected = (device.status == DeviceStatus.CLOSED) or (device.status == DeviceStatus.OPEN)
            if (device.device_type == "nDisplayTest") and is_device_connected:
                nodes.append(device)
        return nodes

    def on_send_file_complete(self, message):
        ''' Handles file transition finish callback '''

        # Get transfer result
        try:
            destination = message['destination']
            succeeded = message['bAck']
            error = message.get('error')
        except KeyError:
            LOGGER.error(f'Error parsing "send file complete" response ({message})')
            return

        # Based on the file extension, find what exactly we just transferred
        ext = os.path.splitext(destination)[1].lower()

        if (ext == '.ndisplay') and self.is_sending_cfg_file:
            # Config file transfer finished
            self.is_sending_cfg_file = False
            self.remote_path_cfg = destination

            if succeeded:
                LOGGER.info(f"{self.name}: nDisplay config successfully transferred to {destination}")
            else:
                LOGGER.error(f"{self.name}: nDisplay config transfer failed: {error}")

            # Now prepare and transfer the test file
            self.generate_and_deploy_test_script()

        elif (ext == '.xml') and self.is_sending_test_file:
            # Test file transfer finished
            self.is_sending_test_file = False
            self.remote_path_test = destination

            if succeeded:
                LOGGER.info(f"{self.name}: nDisplay test file successfully transferred to {destination}")
            else:
                LOGGER.error(f"{self.name}: nDisplay test file transfer failed: {error}")
            
            # Finally, we can launch the test
            self.launch_test()

        else:
            LOGGER.error(f"{self.name}: Unexpected send file completion for {destination}")
            return

    def generate_and_deploy_config_file(self):
        ''' Generates temp config with local overrides '''

        # Get original config file path
        cfg_file = self.current_test.get_ndisplay_config_path()
        # Config file extension
        cfg_ext = os.path.splitext(cfg_file)[1].lower()

        # Read config data
        if cfg_ext == '.uasset':
            cfg_content = self.__class__.extract_configexport_from_uasset(cfg_file).encode('utf-8')
        else:
            with open(cfg_file, 'rb') as f:
                cfg_content = f.read()

        # Apply local changes
        cfg_ext = ".ndisplay" # only json configs for now
        cfg_content = self.__class__.apply_local_overrides_to_config(cfg_content, cfg_ext).encode('utf-8')

        # Generate target config file path
        remote_path_cfg_requested = f'%TEMP%/ndisplay/{self.name}_{uuid.uuid4().hex[:12]}{cfg_ext}'

        # Send content to the destination
        self.is_sending_cfg_file = True
        _, cfg_msg = message_protocol.create_send_filecontent_message(cfg_content, remote_path_cfg_requested)
        self.unreal_client.send_message(cfg_msg)

    def generate_and_deploy_test_script(self):
        ''' Generate new test files with updated config path, and deploys it to a test machine '''

        # Make sure we have received the remote config path
        if not self.remote_path_cfg:
            LOGGER.error(f"No remote config path available for {self.name}")
            return

        # Generate target test file
        test_ext = ".xml"
        remote_path_test_requested = f'%TEMP%/ndisplay/{self.name}_{uuid.uuid4().hex[:12]}{test_ext}'

        try:
            # Duplicate our test data
            test_cloned = self.current_test.duplicate()

            # Update config path in the test XML data
            test_cloned.update_properties({
                "DisplayConfigPath": f"{self.remote_path_cfg}"
            })

            # Get test content
            test_content = test_cloned.to_bytes()

            # Send content to the destination
            self.is_sending_test_file = True
            _, test_msg = message_protocol.create_send_filecontent_message(test_content, remote_path_test_requested)
            self.unreal_client.send_message(test_msg)

        except Exception as e:
            LOGGER.error(f"Could not deploy test file for {self.name}: {e}")
            return

    def prepare_location_for_artifacts(self):
        ''' Generates the name of output directory, and creates it '''

        try:
            # Create directory with the generated name
            formatted_test_time = datetime.now().strftime("%d%m%y_%I%M%S")
            download_dir = self.get_log_download_dir()
            download_dir = Path(download_dir) / self.test_name / formatted_test_time
            Path(download_dir).mkdir(parents=True, exist_ok=True)
            DevicenDisplayTest.artifacts_dir = Path(download_dir)
        except:
            DevicenDisplayTest.artifacts_dir = self.get_log_download_dir()

        LOGGER.info(f'Test output dir: {str(DevicenDisplayTest.artifacts_dir)}')

    def pre_launch_delay(self):
        ''' Performs conditional delay before launch '''

        # Exclusive access
        with DevicenDisplayTest.addr_run_tracking_lock:
            # If any node has already started on this host
            if self.address in DevicenDisplayTest.addr_run_tracking:
                # Check how long ago
                time_delta = datetime.now() - DevicenDisplayTest.addr_run_tracking[self.address]
                launch_delay = self.tsettings['test_multi_node_delay'].get_value()
                # Sleep if required
                if(time_delta.seconds < launch_delay):
                    sleep_time = launch_delay - time_delta.seconds
                    LOGGER.info(f'Pre-launch delay: {sleep_time} seconds')
                    time.sleep(sleep_time)

            # Finally, update the most recent launch time for this address
            DevicenDisplayTest.addr_run_tracking[self.address] = datetime.now()

    def pre_launch_check(self) -> bool:
        ''' Checks if everything is Ok before launch '''

        # Check: test data
        if self.current_test is None:
            LOGGER.error("No test data loaded")
            return False

        # Check: test file
        test_file = self.current_test.get_test_path()
        if not os.path.exists(test_file):
            LOGGER.error("Test file not found")
            return False

        # Check: at least one node available
        nodes = self.get_connected_devices()
        nodes_num = len(nodes)
        if nodes_num < 1:
            LOGGER.error(f"Invalid node count {nodes_num}")
            return False

        # Check: primary node is there
        primary_name = DevicenDisplay.csettings['primary_device_name'].get_value()
        if not any(node.name == primary_name for node in nodes):
            LOGGER.error("Primary node not found")
            return False

        # Check Ok
        return True

    def launch(self, map_name):
        ''' [override] Starts an instance of device '''

        # Delay launch if neccessary
        self.pre_launch_delay()

        # Update launch time
        self.last_launch_time = datetime.now()

        # Check if we're ready to go
        if not self.pre_launch_check():
            LOGGER.error("Pre-launch check failed")
            self.widget._close()
            return

        LOGGER.info(f"Starting: {self.current_test.get_test_path()}")

        # Primary device is responsible for preparing the output directory
        if self.is_primary():
            self.prepare_location_for_artifacts()
            self.rsync_server.set_incoming_logs_path(DevicenDisplayTest.artifacts_dir)

        try:
            # Test launch is a multi-step procedure that starts from config deployment
            self.generate_and_deploy_config_file()
        except Exception as e:
            print(f"Couldn't start test: {e}")
            self.widget._close()
            return

    def launch_test(self):
        ''' Starts the test on the destination machine '''

        # Prepare some intermediate data to generate the launch parameters
        runuat = os.path.join(str(Path(CONFIG.ENGINE_DIR.get_value()).parent), "RunUAT.bat")
        project_name = CONFIG.PROJECT_NAME.get_value()
        program_name = UnrealJobs.Unreal.value
        script_dir="Engine/Plugins/VirtualProduction/ICVFXTesting/Build"

        # Prepare the launch parameters
        param_executable = 'cmd'
        param_extra = self.generate_extra_cmdline_params()
        param_args = '/C call ' + \
            f'{runuat} ' + \
            'BuildGraph ' + \
            f'-Script=\"{self.remote_path_test}\" ' + \
            f'-Target=\"BuildAndTest {project_name}\" ' + \
            f'-ScriptDir=\"{script_dir}\" ' + \
            f'{param_extra}'

        # Update last launch command
        self.last_launch_command.update_value(f'{param_executable} {param_args}')

        # Reset any temporaries of this session
        self.reset_launch_temporary_data()

        # Create process start request
        puuid, msg = message_protocol.create_start_process_message(
            prog_path = param_executable,
            prog_args = param_args,
            prog_name = program_name,
            caller = self.name,
            update_clients_with_stdout = False,
            priority_modifier = sb_utils.PriorityModifier.Normal.value,
            lock_gpu_clock = False,
        )

        # Schedule process start request to send
        self.program_start_queue.add(
            ProgramStartQueueItem(
                name=program_name,
                puuid_dependency=None,
                puuid=puuid,
                msg_to_unreal_client=msg,
            ),
            unreal_client=self.unreal_client,
        )

        # Optionally, minimize the window after launch
        if DevicenDisplay.csettings['minimize_before_launch'].get_value(True):
            self.minimize_windows()

    def generate_extra_cmdline_params(self):
        ''' Generates additional cmdline parameters for the test session '''

        # Aux lambda to convert boolean to lowercase text
        bool_str = lambda b: "true" if b else "false"
        # Aux lambda to convert changlist to string
        cl_str = lambda cl: str(cl) if cl else "na"

        # Temporary & params
        project_name = CONFIG.PROJECT_NAME.get_value()
        project_path = CONFIG.get_project_dir()
        with_trace = True if CONFIG.INSIGHTS_TRACE_ENABLE.get_value() else False
        trace_file = self.generate_trace_file_name()
        wnd_pos = self.settings['window_position'].get_value()
        wnd_res = self.settings['window_resolution'].get_value()
        fullscreen = self.settings['fullscreen'].get_value()
        headless = self.settings['headless'].get_value()
        run_editor_boot_test = self.tsettings['test_run_boot_editor'].get_value()
        max_gpu_count = DevicenDisplay.csettings["max_gpu_count"].get_value(self.name)

        # Generate extra args
        extra_args = \
            f'-set:ProjectName={project_name} ' + \
            f'-set:ProjectPath="{project_path}" ' + \
            f'-set:IsOnStagePerfTest={bool_str(True)} ' + \
            f'-set:SkipPerfReportServer={bool_str(True)} ' + \
            f'-set:EditorBootTest={bool_str(run_editor_boot_test)} ' + \
            f'-set:TestNameExt={self.test_name} ' + \
            f'-set:WithTrace={bool_str(with_trace)} ' + \
            f'-set:TraceFileName={trace_file} ' + \
            f'-set:DisplayClusterNodeName={self.name} ' + \
            f'-set:Fullscreen={bool_str(fullscreen)} ' + \
            f'-set:Offscreen={bool_str(headless)} ' + \
            f'-set:WndLocX={wnd_pos[0]} ' + \
            f'-set:WndLocY={wnd_pos[1]} ' + \
            f'-set:WndResX={wnd_res[0]} ' + \
            f'-set:WndResY={wnd_res[1]} ' + \
            f'-set:MaxGPUCount={max_gpu_count} '

        # [Optional] game target
        target_game = self.tsettings['game_target_override'].get_value()
        if target_game:
            extra_args += f'-set:TargetName={target_game} '

        # [Optional] editor target
        target_edtr = self.tsettings['editor_target_override'].get_value()
        if target_edtr:
            extra_args += f'-set:EditorTargetName={target_edtr} '

        # [Optional] explicit level
        level = CONFIG.CURRENT_LEVEL
        if map_name_is_valid(level):
            extra_args += f'-set:MapOverride={level} '

        # [Optional] 24fps report
        use_24fps_report = self.tsettings['report_24fps_format'].get_value()
        if use_24fps_report:
            extra_args += f'-set:SummaryReportType=VP24fps '

        # [OverrideDefault] Test build info
        build_info = self.tsettings['test_build_info'].get_value()
        if build_info:
            # If specified, use explicit info
            extra_args += f'-set:TestBuildName={build_info} '
        else:
            # Otherwise, use the default one with engine & project CL numbers
            extra_args += f'-set:TestBuildName=E:{cl_str(self.engine_changelist)}-P:{cl_str(self.project_changelist)} '

        return extra_args

    def generate_trace_file_name(self):
        ''' Generates the name of a trace file for this device. '''
        return f'{self.name}_{self.last_launch_time.strftime("%d%m%y_%I%M%S")}.utrace'

    def generate_remote_test_root_directory(self):
        ''' Generates remote test root directory path. '''

        prj_name = CONFIG.PROJECT_NAME.get_value(self.name)

        root_dir = Path(CONFIG.ENGINE_DIR.get_value(self.name)).parent
        root_dir = root_dir / 'LocalBuilds/Reports'
        root_dir = root_dir / prj_name
        root_dir = root_dir / self.test_name
        root_dir = root_dir / self.name
        root_dir = root_dir / f'{prj_name}Win64{prj_name}Perf'

        return root_dir

    def generate_remote_test_log_filepath(self):
        ''' Generates remote test log file path. '''
        remote_file = self.generate_remote_test_root_directory()
        remote_file = remote_file / 'ICVFXTest.PerformanceReport_(Win64_Development_EditorGame)'
        remote_file = remote_file / 'EditorGame/EditorGameOutput.log'
        return remote_file

    def generate_remote_test_utrace_filepath(self):
        ''' Generates remote test utrace file path. '''
        remote_file = self.generate_remote_test_root_directory()
        remote_file = remote_file / self.generate_trace_file_name()
        return remote_file

    def generate_remote_test_report_dir(self):
        ''' Generates remote test report directory path. '''
        report_dir = self.generate_remote_test_root_directory()
        report_dir = report_dir / 'ICVFXTest.PerformanceReport_(Win64_Development_EditorGame)/Reports/Performance'
        return report_dir

    def do_program_ended_update(self, *,
        program_name: str,
        returncode: int,
        get_stdout_str: Callable[[], str],
        get_stderr_str: Callable[[], str],
    ):
        ''' [override] Handles test & transfer finish '''

        # Reset address tracking data before next launch
        with DevicenDisplayTest.addr_run_tracking_lock:
            DevicenDisplayTest.addr_run_tracking.clear()

        # Let the base class do most of handling
        super().do_program_ended_update(
            program_name=program_name,
            returncode=returncode,
            get_stdout_str=get_stdout_str,
            get_stderr_str=get_stderr_str)

        # Handle test finish
        if program_name == UnrealJobs.Unreal.value:
            self.start_retrieve_report(unreal_exit_code=returncode)

    def start_retrieve_log(self, unreal_exit_code: int):
        ''' [override] Retrieve the log file if logging is enabled. '''

        # Local & remote log file
        local_log_path = DevicenDisplayTest.artifacts_dir / self.log_filename
        remote_log_path = self.generate_remote_test_log_filepath()
        # Update the most recent log reference
        self.last_log_path.update_value(str(local_log_path))
        # Download log file
        return self.fetch_file(remote_log_path, self.log_filename)
    
    def start_retrieve_utrace(self, unreal_exit_code:int ):
        ''' [override] Retrieve the utrace file if tracing is enabled. '''

        if not CONFIG.INSIGHTS_TRACE_ENABLE.get_value():
            return False

        # Local & remote utrace file
        local_utrace_filename = f'{self.name}.utrace'
        local_utrace_path = DevicenDisplayTest.artifacts_dir / local_utrace_filename
        remote_utrace_path = self.generate_remote_test_utrace_filepath()
        # Update the most recent utrace reference
        self.last_trace_path.update_value(str(local_utrace_path))
        # Download utrace file
        return self.fetch_file(remote_utrace_path, local_utrace_filename)

    def start_retrieve_report(self, unreal_exit_code:int ):
        ''' Retrieve report files. '''

        # Reports to download
        reports = [
            'HistoricReport_14DaysBase.html',
            'HistoricReport_AllTimeBase.html',
            'SummaryTable.html'
        ]

        # Download all the reports
        remote_report_dir = self.generate_remote_test_report_dir()
        for report in reports:
            self.fetch_file(Path(remote_report_dir) / report, f'{self.name}_{report}')

