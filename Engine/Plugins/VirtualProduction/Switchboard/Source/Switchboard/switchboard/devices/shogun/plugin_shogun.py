# Copyright Epic Games, Inc. All Rights Reserved.
import datetime
from functools import wraps

from switchboard.config import StringSetting
from switchboard.devices.device_base import Device, DeviceStatus
from switchboard.devices.device_widget_base import DeviceWidget
from switchboard.switchboard_logging import LOGGER
import switchboard.switchboard_utils as utils

from .thirdparty.vicon_core_api import vicon_core_api
from .thirdparty.shogun_live_api import shogun_live_api


def unresponsive_shogun(f):
    """
    Decorator to gracefully disconnect if a Shogun command comes back with
    vicon_core_api.client.RPCError.
    """
    @wraps(f)
    def wrapped(self, *args, **kwargs):
        try:
            return f(self, *args, **kwargs)
        except vicon_core_api.client.RPCError:
            self.device_qt_handler.signal_device_connect_failed.emit(self)
            return None
        except ModuleNotFoundError:
            LOGGER.error(
                'Could not connect to Shogun because the module was not '
                'installed')
            return None

    return wrapped


class DeviceShogun(Device):
    def __init__(self, name, address, **kwargs):
        self.setting_save_path = StringSetting("save_path", "Save Path", "")
        super().__init__(name, address, **kwargs)

        self.trigger_start = True
        self.trigger_stop = True

        self.client = None

        self._slate = 'slate'
        self._take = 1

    def device_settings(self):
        return super().device_settings() + [self.setting_save_path]

    @property
    def save_path(self):
        return self.setting_save_path.get_value()

    @save_path.setter
    @unresponsive_shogun
    def save_path(self, value):
        if self.setting_save_path.get_value() == value:
            return

        self.setting_save_path.update_value(value)

        self.set_capture_folder()

    @unresponsive_shogun
    def connect_listener(self):
        super().connect_listener()

        self.client = vicon_core_api.Client(self.address)
        self.capture_service = shogun_live_api.CaptureServices(self.client)

        if self.client.connected:
            self.status = DeviceStatus.READY
            self.set_capture_folder()
        else:
            self.device_qt_handler.signal_device_connect_failed.emit(self)

    @unresponsive_shogun
    def set_slate(self, value):
        self._slate = value
        self.capture_service.set_capture_name(
            utils.capture_name(self._slate, self._take))

    @unresponsive_shogun
    def set_take(self, value):
        self._take = value
        self.capture_service.set_capture_name(
            utils.capture_name(self._slate, self._take))

    @unresponsive_shogun
    def set_capture_folder(self):
        d = datetime.date.today()

        save_path = d.strftime(self.save_path)

        # HOW TO MAKE DIR ON OTHER MACHINE
        # os.makedirs(save_path, exist_ok=True)

        result = self.capture_service.set_capture_folder(save_path)

        if result != vicon_core_api.result.Result.Ok:
            LOGGER.error(
                f'{self.name}: "{save_path}" is an invalid path. '
                'Capture Folder not set')

    @unresponsive_shogun
    def record_start(self, slate, take, description):
        if self.is_disconnected or not self.trigger_start:
            return

        self.set_slate(slate)
        self.set_take(take)

        result, _ = self.capture_service.start_capture()

        if result == vicon_core_api.result.Result.Ok:
            self.record_start_confirm(self.timecode())

    @unresponsive_shogun
    def record_stop(self):
        if self.is_disconnected or not self.trigger_stop:
            return

        result = self.capture_service.stop_capture(0)

        import time
        time.sleep(3)

        if result == vicon_core_api.result.Result.Ok:
            # TODO: THIS BLOCKS THE MAIN THREAD ON STOP. FIX THIS
            result, _, _ = self.capture_service.latest_capture_file_paths()
            # START HERE: GET PATHS OF FILES WRITTEN
            # LOGGER.debug(f'{result} {paths}')
            self.record_stop_confirm(self.timecode(), paths=None)

    def timecode(self):
        return '00:00:00:00'


class DeviceWidgetShogun(DeviceWidget):
    def __init__(self, name, device_hash, address, icons, parent=None):
        super().__init__(name, device_hash, address, icons, parent=parent)

    def _add_control_buttons(self):
        super()._add_control_buttons()
        self.trigger_start_button = self.add_control_button(
            ':/icons/images/icon_trigger_start_disabled.png',
            icon_hover=':/icons/images/icon_trigger_start_hover.png',
            icon_disabled=':/icons/images/icon_trigger_start_disabled.png',
            icon_on=':/icons/images/icon_trigger_start.png',
            icon_hover_on=':/icons/images/icon_trigger_start_hover.png',
            icon_disabled_on=':/icons/images/icon_trigger_start_disabled.png',
            tool_tip='Trigger when recording starts',
            checkable=True, checked=True)

        self.trigger_stop_button = self.add_control_button(
            ':/icons/images/icon_trigger_stop_disabled.png',
            icon_hover=':/icons/images/icon_trigger_stop_hover.png',
            icon_disabled=':/icons/images/icon_trigger_stop_disabled.png',
            icon_on=':/icons/images/icon_trigger_stop.png',
            icon_hover_on=':/icons/images/icon_trigger_stop_hover.png',
            icon_disabled_on=':/icons/images/icon_trigger_stop_disabled.png',
            tool_tip='Trigger when recording stops',
            checkable=True, checked=True)

        self.connect_button = self.add_control_button(
            ':/icons/images/icon_connect.png',
            icon_hover=':/icons/images/icon_connect_hover.png',
            icon_disabled=':/icons/images/icon_connect_disabled.png',
            icon_on=':/icons/images/icon_connected.png',
            icon_hover_on=':/icons/images/icon_connected_hover.png',
            icon_disabled_on=':/icons/images/icon_connected_disabled.png',
            tool_tip='Connect/Disconnect from listener')

        self.trigger_start_button.clicked.connect(self.trigger_start_clicked)
        self.trigger_stop_button.clicked.connect(self.trigger_stop_clicked)
        self.connect_button.clicked.connect(self.connect_button_clicked)

        # Disable the buttons
        self.trigger_start_button.setDisabled(True)
        self.trigger_stop_button.setDisabled(True)

    def trigger_start_clicked(self):
        if self.trigger_start_button.isChecked():
            self.signal_device_widget_trigger_start_toggled.emit(self, True)
        else:
            self.signal_device_widget_trigger_start_toggled.emit(self, False)

    def trigger_stop_clicked(self):
        if self.trigger_stop_button.isChecked():
            self.signal_device_widget_trigger_stop_toggled.emit(self, True)
        else:
            self.signal_device_widget_trigger_stop_toggled.emit(self, False)

    def connect_button_clicked(self):
        if self.connect_button.isChecked():
            self._connect()
        else:
            self._disconnect()

    def _connect(self):
        # Make sure the button is in the correct state
        self.connect_button.setChecked(True)

        # Enable the buttons
        self.trigger_start_button.setDisabled(False)
        self.trigger_stop_button.setDisabled(False)

        # Emit Signal to Switchboard
        self.signal_device_widget_connect.emit(self)

    def _disconnect(self):
        # Make sure the button is in the correct state
        self.connect_button.setChecked(False)

        # Disable the buttons
        self.trigger_start_button.setDisabled(True)
        self.trigger_stop_button.setDisabled(True)

        # Emit Signal to Switchboard
        self.signal_device_widget_disconnect.emit(self)
