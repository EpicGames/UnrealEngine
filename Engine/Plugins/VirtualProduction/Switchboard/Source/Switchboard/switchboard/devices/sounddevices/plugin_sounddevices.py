# Copyright Epic Games, Inc. All Rights Reserved.
from switchboard import recording
from switchboard.devices.device_base import Device, DeviceStatus
from switchboard.devices.device_widget_base import DeviceWidget
from switchboard.switchboard_logging import LOGGER
import switchboard.switchboard_utils as utils

from PySide6 import QtWidgets

import datetime
import json


class DeviceSoundDevices(Device):
    def __init__(self, name, address, **kwargs):
        super().__init__(name, address, **kwargs)

        self.user_name = 'guest'
        self.password = 'guest'

        self._interval = 0.1
        self._timeout = 4

        self.repeat_function = None

    def connect(self):
        if not self._send_command('tmcode'):
            self.device_qt_handler.signal_device_connect_failed.emit(self)
            return

        self.status = DeviceStatus.READY

        # Make sure it's recording as Scene-Take
        self._send_command('setsetting/FileNameFormat=Scene-Take')
        # Change the reel to today's date
        self.set_reel(utils.date_to_string(datetime.date.today()))
        # Make sure we are in record mode
        self.set_record_mode()

    def disconnect(self):
        self.status = DeviceStatus.DISCONNECTED

    def set_reel(self, value):
        self._send_command(f'setsetting/ReelName={value}')

    def set_slate(self, value):
        self._send_command(f'setsetting/SceneName={value}')

    def set_take(self, value):
        self._send_command(f'setsetting/TakeNumber={value}')

    def _send_command(self, command):
        try:
            import requests
            from requests.auth import HTTPDigestAuth
            command = f'http://{self.address}/sounddevices/{command}'
            response = requests.get(command, auth=HTTPDigestAuth(self.user_name, self.password), timeout=3)
            return response.content.decode('utf-8')
        except ModuleNotFoundError:
            LOGGER.error(e)
            #LOGGER.error('Could not connect to SoundDevice because requests module was not installed')
        except requests.exceptions.ConnectionError:
            self.device_qt_handler.signal_device_connect_failed.emit(self)
        except requests.exceptions.Timeout as e:
            LOGGER.error(e)

    def record_start(self, slate, take, description):
        if self.is_disconnected or not self.is_recording_device:
            return

        # Set Slate/Take   
        self.set_reel(utils.date_to_string(datetime.date.today()))      # Change the reel to today's date
        self.set_slate(slate)
        self.set_take(take)

        # Start a new recording
        self.device_recording = recording.DeviceRecording()
        self.device_recording.device_name = self.name
        self.device_recording.device_type = self.device_type

        if self.repeat_function:
            self.repeat_function.stop()
            self.repeat_function = None

        # Start device Recording
        self.repeat_function = utils.RepeatFunction(self._interval, self._timeout, self.record)
        self.repeat_function.add_finish_callback(self.record_start_confirm, self.timecode()) # TODO: Don't bake
        self.repeat_function.start(results_function=lambda results: results == 'rec')

    def record_stop(self):
        # If there is already a repeating function trying to record, stop it
        if self.repeat_function:
            self.repeat_function.stop()
            self.repeat_function = None

        self.repeat_function = utils.RepeatFunction(self._interval, self._timeout, self.stop)
        self.repeat_function.add_finish_callback(self.record_stop_confirm, self.timecode()) # TODO: Don't bake
        self.repeat_function.start(results_function=lambda results: results == 'stop')

    def record_stop_confirm(self, timecode, paths=None):
        try:
            results = self._send_command('invoke/RemoteApi/currentRecordTake()')
            path = json.loads(results)['String']

            corrected_path = path
            for i in range(1,5):
                corrected_path = corrected_path.replace(f'/HD{i}/', f'/Drive_{i}/')

            results = json.loads(self._send_command(f'filedetails{path}'))
            self.device_recording.timecode_in = results['FileDetails']['timecodeStart']
            self.device_recording.timecode_out = timecode
            self.device_recording.duration = results['FileDetails']['duration']

            self.device_recording.paths = [corrected_path]
            self.device_recording.status = recording.RecordingStatus.ON_DEVICE
        except Exception as e:
            LOGGER.error(f'{self.name}: record_stop_confirm error - {e}')

        if self.repeat_function:
            self.repeat_function.stop()
            self.repeat_function = None

        self.status = DeviceStatus.READY

    def timecode(self):
        return self._send_command('tmcode')

    def stop(self):
        self._send_command('settransport/stop') #TODO: handle failed command
        return self.__transport_state()

    def record(self):
        self._send_command('settransport/rec')
        return self.__transport_state()

    def __transport_state(self):
        result = self._send_command('transport')

        try:
            return json.loads(result)['Transport']
        except:
            return None

    def set_record_mode(self):
        for i in range(1,5):
            self._send_command(f'setsetting/RecordToDrive{i}=Record')

    def set_transport_mode(self):
        pass

    def transport_file(self, device_path, output_dir):
        # Put all drives into file transfer mode
        for i in range(1,5):
            self._send_command(f'setsetting/RecordToDrive{i}=Ethernet%20File%20Transfer')
        r'''
        f'net use * \\\\{self.address} /user:{self.user_name} {self.password}'
        robocopy.exe "W:\wwwroot\MyProject" "\\192.168.0.1\Share\wwwroot\MyProject" *.* /E /XO /XD "App_Data/Search" "*.svn" /XF "sitefinity.log" "Thumbs.db" /NDL /NC /NP
        net use * /delete /yes

        import os
        os.system('ls -l')
        '''


class DeviceWidgetSoundDevices(DeviceWidget):
    def __init__(self, name, device_type, device_hash, address, parent=None):
        super().__init__(name, device_type, device_hash, address, parent=parent)

    def _add_control_buttons(self):
        super()._add_control_buttons()
        self.connect_button = self.add_control_button(':/icons/images/icon_connect.png',
                                           icon_hover=':/icons/images/icon_connect_hover.png',
                                        icon_disabled=':/icons/images/icon_connect_disabled.png',
                                              icon_on=':/icons/images/icon_connected.png',
                                        icon_hover_on=':/icons/images/icon_connected_hover.png',
                                     icon_disabled_on=':/icons/images/icon_connected_disabled.png',
                                             tool_tip='Connect/Disconnect from listener')

        self.connect_button.clicked.connect(self.connect_button_clicked)

    def connect_button_clicked(self):
        if self.connect_button.isChecked():
            self._connect()
        else:
            self._disconnect()

    def _connect(self):
        # Make sure the button is in the correct state
        self.connect_button.setChecked(True)

        # Emit Signal to Switchboard
        self.signal_device_widget_connect.emit(self)

    def _disconnect(self):
        # Make sure the button is in the correct state
        self.connect_button.setChecked(False)

        # Emit Signal to Switchboard
        self.signal_device_widget_disconnect.emit(self)
