# Copyright Epic Games, Inc. All Rights Reserved.

from switchboard import switchboard_utils as sb_utils
from switchboard.switchboard_logging import LOGGER

from . import version_helpers

from PySide6 import QtCore

import pathlib, re, subprocess, threading, time, typing


class ListenerWatcher(QtCore.QObject):
    ''' Wraps a QFileSystemWatcher to update cached version info and signal when the local listener EXE changes. '''

    signal_listener_changed = QtCore.Signal()

    def __init__(self):
        super().__init__()

        self._listener_path: typing.Optional[pathlib.PurePath] = None
        self.listener_mtime = 0
        self.listener_ver: typing.Optional[typing.Tuple[int, int, int]] = None
        self.watcher = QtCore.QFileSystemWatcher(self)
        self.watcher.directoryChanged.connect(self.on_dir_changed)
        self.watcher.fileChanged.connect(self.on_file_changed)
        self.thread: typing.Optional[threading.Thread] = None

    @property
    def listener_dir(self) -> str:
        return str(self._listener_path.parent) if self._listener_path else ''

    @property
    def listener_path(self) -> str:
        return str(self._listener_path) if self._listener_path else ''

    def update_listener_path(self, path: str):
        if path == str(self._listener_path):
            return

        if self._listener_path:
            self.watcher.removePaths([self.listener_dir, self.listener_path])

        self._listener_path = pathlib.PurePath(path)
        self._update_listener_info(self._listener_path)

        self.watcher.addPaths([self.listener_dir, self.listener_path])

    @QtCore.Slot(str)
    def on_dir_changed(self, changed_path):
        assert self._listener_path

        # Try to restore the file watch if the file was previously deleted.
        if self.listener_path not in self.watcher.files():
            self.watcher.addPath(self.listener_path)
            self.on_file_changed(self.listener_path)

    @QtCore.Slot(str)
    def on_file_changed(self, changed_path):
        assert self._listener_path and self.listener_path == changed_path

        # SwitchboardListener.exe takes 500+ms to init, presumably due to engine static/global ctors.
        if not self.thread or not self.thread.is_alive():
            self.thread = threading.Thread(target=self._update_listener_info, args=[self._listener_path])
            self.thread.start()

    def _update_listener_info(self, purepath: pathlib.PurePath):
        updated = False
        retries = 5
        retry_interval_sec = 1.0
        while retries > 0:
            retries -= 1
            try:
                path = pathlib.Path(purepath).resolve(strict=True)
                mtime = path.stat().st_mtime

                # Early out if the modtime hasn't changed.
                if mtime == self.listener_mtime:
                    return

                ver_out = subprocess.check_output([str(path), '-version'], startupinfo=sb_utils.get_hidden_sp_startupinfo())
                match = re.fullmatch(r'SwitchboardListener (\d+).(\d+).(\d+)', ver_out.decode())
                if match:
                    parsed_ver = tuple(int(_) for _ in match.group(1,2,3))
                    LOGGER.debug(f"Discovered listener v{version_helpers.version_str(parsed_ver)} at {path}")
                    self.listener_mtime = mtime
                    self.listener_ver = parsed_ver
                    updated = True
                    break
                else:
                    LOGGER.error(f"Couldn't parse version output from invoking `{path} -version`: {ver_out.decode(errors='replace')}")
                    time.sleep(retry_interval_sec)
            except FileNotFoundError as e:
                break
            except PermissionError:
                # This happens if the file is in use by another process.
                time.sleep(retry_interval_sec)
            except subprocess.CalledProcessError as e:
                LOGGER.error(f"Non-zero exit code ({e.returncode}) when invoking `{e.cmd}`", exc_info=e)
                time.sleep(retry_interval_sec)
            except OSError as e:
                # This can happen if the drive is locked (e.g. by Bitlocker)
                LOGGER.error(e)
                break

        if not updated:
            self.listener_mtime = 0
            self.listener_ver = None

        self.signal_listener_changed.emit()