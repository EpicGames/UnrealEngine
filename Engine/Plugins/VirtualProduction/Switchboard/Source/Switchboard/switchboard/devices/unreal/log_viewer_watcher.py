# Copyright Epic Games, Inc. All Rights Reserved.

import subprocess
import threading
import time
import weakref
from typing import Dict, Optional

from PySide6 import QtCore

from switchboard.switchboard_logging import LOGGER


class LogViewerProcessInfo:
    '''Holds information about a log viewer process and its associated device.
    This information is needed to know what to do when the process closes. e.g. End live stream. '''

    def __init__(self, process: subprocess.Popen, device_name: str, device_ref: weakref.ReferenceType, log_file_path: str):
        self.process = process
        self.device_name = device_name
        self.device_ref = device_ref  # Weak reference to device
        self.log_file_path = log_file_path
        self.launch_time = time.time()


class LogViewerWatcher(QtCore.QObject):
    ''' Monitors log viewer processes and notifies when they close.
    Currently expected to be a singleton, declared in this very module.
    '''

    # Signal emitted when a process terminates. str argument is the device name.
    process_terminated = QtCore.Signal(str)

    def __init__(self):
        super().__init__()
        self._processes: Dict[int, LogViewerProcessInfo] = {}  # key is PID.
        self._monitor_timer = QtCore.QTimer()
        self._monitor_timer.timeout.connect(self._check_processes)
        self._monitor_timer.setInterval(2000)  # Check every 2 seconds
        self._lock = threading.Lock()

    def start_monitoring(self):
        ''' Start the process monitoring timer '''

        if not self._monitor_timer.isActive():
            self._monitor_timer.start()

    def stop_monitoring(self):
        ''' Stop the process monitoring timer '''
        if self._monitor_timer.isActive():
            self._monitor_timer.stop()

    def register_process(self, process: subprocess.Popen, device_name: str, device, log_file_path: str):
        ''' Register a log viewer process for monitoring '''
        with self._lock:
            try:
                device_ref = weakref.ref(device)
                process_info = LogViewerProcessInfo(process, device_name, device_ref, log_file_path)
                self._processes[process.pid] = process_info

                # Start monitoring if this is the first process
                if len(self._processes) == 1:
                    self.start_monitoring()

            except Exception as e:
                LOGGER.error(f"LogViewerWatcher: Failed to register process: {e}")

    def unregister_process(self, pid: int):
        ''' Manually unregister a process (e.g., when stopping live log) '''

        with self._lock:
            if pid in self._processes:

                del self._processes[pid]

                # Stop monitoring if no processes left
                if not self._processes:
                    self.stop_monitoring()

    def terminate_process(self, pid: int) -> bool:
        ''' Terminate a specific process by PID '''

        with self._lock:

            if pid in self._processes:

                process_info = self._processes[pid]

                try:
                    if process_info.process.poll() is None:  # Process still running
                        process_info.process.terminate()
                        return True

                except Exception as e:
                    LOGGER.error(f"LogViewerWatcher: Failed to terminate process PID {pid}: {e}")

            return False

    def terminate_log_viewers_for_device(self, device_name: str):
        ''' Terminate all processes associated with a specific device '''

        pids_to_terminate = []

        with self._lock:

            for pid, process_info in self._processes.items():
                if process_info.device_name == device_name:
                    pids_to_terminate.append(pid)

        for pid in pids_to_terminate:
            self.terminate_process(pid)

    def get_process_for_device(self, device_name: str) -> Optional[subprocess.Popen]:
        ''' Get the process object for a device, if any '''

        with self._lock:
            for process_info in self._processes.values():
                if process_info.device_name == device_name:
                    return process_info.process
        return None

    @QtCore.Slot()
    def _check_processes(self):
        ''' Check all registered processes and clean up terminated ones '''

        terminated_pids = []

        with self._lock:
            for pid, process_info in self._processes.items():
                try:
                    # Check if process has terminated
                    if process_info.process.poll() is not None:
                        terminated_pids.append(pid)
                        
                except Exception as e:
                    # Process might be invalid, mark for removal
                    LOGGER.warning(f"LogViewerWatcher: Error checking process PID {pid}: {e}")
                    terminated_pids.append(pid)

        # Clean up terminated processes and notify

        for pid in terminated_pids:
            with self._lock:
                if pid in self._processes:
                    process_info = self._processes[pid]
                    device_name = process_info.device_name
                    device_ref = process_info.device_ref

                    # Remove from tracking
                    del self._processes[pid]

                    # Try to get the device object to clean up live log streaming
                    device = device_ref() if device_ref else None

                    if device:
                        # Force main thread execution. We're not in a hurry.
                        QtCore.QMetaObject.invokeMethod(
                            device, '_on_log_viewer_closed',
                            QtCore.Qt.QueuedConnection
                        )
                    else:
                        LOGGER.info(f"LogViewerWatcher: Device '{device_name}' no longer exists")

                    # Emit signal for external listeners
                    self.process_terminated.emit(device_name)

        # Stop monitoring if no processes left
        with self._lock:
            if not self._processes:
                self.stop_monitoring()


# Global log viewer watcher instance
log_viewer_watcher = LogViewerWatcher()