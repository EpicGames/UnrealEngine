# Copyright Epic Games, Inc. All Rights Reserved.

from __future__ import annotations

import base64
import concurrent.futures
import hashlib
import json
import os
import pathlib
import posixpath
import re
import socket
import subprocess
import sys
import threading
import time
import uuid
import weakref
import zlib

from abc import abstractmethod
from collections import OrderedDict
from datetime import datetime
from enum import Enum
from functools import wraps
from ipaddress import IPv4Address
from itertools import count
from pathlib import PurePosixPath
from typing import Callable, Dict, Generator, Iterable, Optional, Union

from PySide6 import QtCore, QtGui, QtWidgets

import switchboard.config_osc as osc
import switchboard.switchboard_dialog as sb_dialog
import switchboard.switchboard_utils as sb_utils
import switchboard.switchboard_widgets as sb_widgets

from switchboard import message_protocol, switchboard_application, ugs_utils
from switchboard.config import (
    CONFIG, DEFAULT_MAP_TEXT, ENABLE_UGS_SUPPORT, SETTINGS, BoolSetting, DirectoryPathSetting,
    EngineSyncMethod, FilePathSetting, FloatSetting, IntSetting, LoggingSetting, MultiOptionSetting, OptionSetting, Setting,
    StringListSetting, StringSetting, migrate_comma_separated_string_to_list)
from switchboard.devices.device_base import Device, DeviceStatus, PluginHeaderWidgets
from switchboard.devices.device_widget_base import DeviceAutoJoinMUServerUI, DeviceWidget
from switchboard.devices.unreal.uassetparser import UassetParser
from switchboard.listener_client import ListenerClient
from switchboard.switchboard_logging import LOGGER
from switchboard.tools.insights_launcher import InsightsLauncher
from . import version_helpers
from .listener_watcher import ListenerWatcher
from .log_viewer_watcher import log_viewer_watcher
from .redeploy_dialog import RedeployListenerDialog
from switchboard.sbcache import SBCache, Asset


class UnrealJobs(Enum):
    ''' Used to specify the type of job sent to the unreal client '''

    BuildPrefix = 'build_'
    BuildProject = 'build_project'

    SyncStatus = 'syncstatus'

    Sync = 'sync'
    BuildDDC = 'unreal_ddc'
    Unreal = 'unreal'
    Retrieve = 'retrieve'
    PackageGame = 'package_game'


class ProgramStartQueueItem:
    '''
    Item that holds the information for a program to start in the future.
    '''
    def __init__(
        self, name: str, puuid_dependency: Optional[uuid.UUID],
        puuid: uuid.UUID, msg_to_unreal_client: bytes, pid: int = 0,
        launch_fn: Callable[[], None] = lambda: None
    ):
        self.name = name
        # So that it can check wait_for_previous_to_end against the correct
        # program
        self.puuid_dependency = puuid_dependency
        # id of the program.
        self.puuid = puuid
        # command for listener/unreal_client
        self.msg_to_unreal_client = msg_to_unreal_client
        self.launch_fn = launch_fn
        self.pid = pid

    @classmethod
    def from_listener_process(cls, process):
        return cls(
            puuid_dependency=None,
            puuid=uuid.UUID(process['uuid']),
            msg_to_unreal_client=b'',
            name=process['name'],
            pid=process['pid'],
            launch_fn=lambda: None,
        )


def use_lock(func):
    '''
    Decorator to ensure the decorated function is executed by a single thread
    at a time.
    '''
    @wraps(func)
    def _use_lock(self, *args, **kwargs):

        self.lock.acquire()

        try:
            ret = func(self, *args, **kwargs)

            # Using the lock implies a write operation, which requires
            # updating the reader indexed caches.
            self._update_rd_caches()

            return ret

        finally:
            self.lock.release()

    return _use_lock


class ProgramStartQueue:
    ''' Queue of programs to launch that may have dependencies 
    Write operations can from from the listener thread (e.g. program events)
    or from the Main thread (e.g. add a program). Read operations can happen
    on the Main thread (e.g. see if a program is running) or from the listener
    thread.

    The locking strategy is for the writers to use a lock and update a shallow
    copy for the readers. Reading is expected to happen much more often than writing,
    especially in relation to checking if programs are running or not.
    '''

    def __init__(self):
        # Needed because these functions will be called from listener thread
        # and main thread.
        self.lock = threading.Lock()

        self.queued_programs: list[ProgramStartQueueItem] = []
        self.starting_programs: OrderedDict[uuid.UUID, ProgramStartQueueItem] = OrderedDict()

        # initialized as returned by listener
        self.running_programs: OrderedDict[uuid.UUID, ProgramStartQueueItem] = OrderedDict()

        # Indexed caches for the readers.
        self.rd_running_progs_by_name: dict[str, list[ProgramStartQueueItem]] = dict()
        self.rd_puuid_by_name: dict[str, uuid.UUID] = dict()

    # Write operations

    @use_lock
    def reset(self):
        ''' Clears the list and any internal state '''
        self.queued_programs = []
        self.starting_programs = OrderedDict()
        self.running_programs = OrderedDict()

    @use_lock
    def on_program_started(self, prog):
        ''' Returns the name of the program that started
        Moves the program from starting_programs to running_programs
        '''
        try:
            self.starting_programs.pop(prog.puuid)
        except KeyError:
            LOGGER.error(
                f"on_program_started::starting_programs.pop({prog.puuid}) "
                "KeyError")

        self.running_programs[prog.puuid] = prog

        return prog.name

    @use_lock
    def on_program_ended(self, puuid, unreal_client):
        ''' Returns the name of the program that ended
        Launches any dependent programs in the queue.
        Removes the program from starting_programs or running_programs.
        '''
        # remove from lists
        prog = None

        if puuid is not None:
            # check if it was waiting to start
            try:
                prog = self.starting_programs.pop(puuid)
            except KeyError:
                pass

            # check if it is already running
            if prog is None:
                try:
                    prog = self.running_programs.pop(puuid)
                except KeyError:
                    pass

            # check if it is queued
            if prog is None:
                programs = [
                    program for program in self.queued_programs
                    if program.puuid == puuid]

                for program in programs:
                    prog = program
                    self.queued_programs.remove(program)

            if prog is not None:
                LOGGER.debug(f"Ended {prog.name} {prog.puuid}")

        self._launch_dependents(puuid=puuid, unreal_client=unreal_client)

        return prog.name if prog else "unknown"

    @use_lock
    def add(self, prog, unreal_client) -> None:
        ''' Adds a new program to be started in the queue
        Must be of type ProgramStartQueueItem.
        It may start launch it right away if it doesn't have any dependencies.
        '''
        assert isinstance(prog, ProgramStartQueueItem)

        # Ensure that dependence is still possible, and if it isn't, replace
        # with None.
        if prog.puuid_dependency is not None:
            try:
                self._name_from_puuid(prog.puuid_dependency)
            except KeyError:
                LOGGER.debug(
                    f"{prog.name} specified non-existent dependency on puuid "
                    f"{prog.puuid_dependency}")
                prog.puuid_dependency = None

        self.queued_programs.append(prog)

        # This effectively causes a launch if it doesn't have any dependencies
        self._launch_dependents(puuid=None, unreal_client=unreal_client)

    @use_lock
    def clear_running_programs(self) -> None:
        self.running_programs.clear()

    @use_lock
    def update_running_program(self, prog) -> None:
        self.running_programs[prog.puuid] = prog

    # Helper functions

    def _launch_dependents(self, puuid, unreal_client):
        ''' Launches programs dependend on given puuid
        Do not call externally because it does not use the thread lock.
        '''
        # see if we need to launch any dependencies
        progs_launched = []

        for prog in self.queued_programs:
            if prog.puuid_dependency is None or puuid == prog.puuid_dependency:

                progs_launched.append(prog)
                self.starting_programs[prog.puuid] = prog

                unreal_client.send_message(prog.msg_to_unreal_client)
                prog.launch_fn()

        for prog in progs_launched:
            self.queued_programs.remove(prog)

    def _name_from_puuid(self, puuid) -> str:
        ''' Returns the name of the specified program id '''
        for prog in self.queued_programs:
            if prog.puuid == puuid:
                return prog.name

        for thedict in [self.starting_programs, self.running_programs]:
            try:
                return thedict[puuid].name
            except KeyError:
                pass

        raise KeyError

    def _update_rd_caches(self) -> None:
        ''' Updates the indexed dictionaries for the readers.
        This function assumes it is called by one of the writer functions
        that already hold the lock.
        '''

        # Update rd_running_progs_by_name
        #
        rd_running_progs_by_name: dict[str, list[ProgramStartQueueItem]] = dict()

        for prog in self.running_programs.values():

            if prog.name not in rd_running_progs_by_name:
                rd_running_progs_by_name[prog.name] = []

            rd_running_progs_by_name[prog.name].append(prog)

        # Update rd_puuid_by_name
        #
        rd_puuid_by_name: dict[str, uuid.UUID] = dict()

        # Combine all programs into one list, such that when iterated,
        # the last program will be the the one more likely to finish last.
        # I.e. a queued program of a given name is expected to end after
        # one that is already running.
        progs = reversed(
            list(self.queued_programs) +
            list(self.starting_programs.values()) +
            list(self.running_programs.values())
        )

        for prog in progs:
            rd_puuid_by_name[prog.name] = prog.puuid

        # Atomically replace the dicts for the readers.

        self.rd_running_progs_by_name = rd_running_progs_by_name
        self.rd_puuid_by_name = rd_puuid_by_name

    # Read operations

    def running_programs_named(self, name) -> list[ProgramStartQueueItem]:
        ''' Returns the program ids of running programs named as specified '''
        return self.rd_running_progs_by_name.get(name, [])

    def running_puuids_named(self, name) -> list[uuid.UUID]:
        ''' Returns the program ids of running programs named as specified '''
        progs = self.running_programs_named(name)
        return [prog.puuid for prog in progs]

    def puuid_from_name(self, name: str) -> uuid.UUID:
        '''
        Returns the puuid of the specified program name.

        Searches as to return the one most likely to return last - but not
        guaranteed to do so.
        '''
        return self.rd_puuid_by_name[name]

    def running_programs_count(self) -> int:
        ''' Returns number of running programs '''
        return sum(len(program_list) for program_list in self.rd_running_progs_by_name.values())


class UAssetSetting(Setting):
    '''
    A setting which points to an Unreal asset.

    Its main widget is a combo box that enumerates certain project/plugin asset types.
    This is an abstract base class; concrete subclasses override `_classnames()` to specify their uasset class names.

    Its "refresh" button triggers an asset traversal to refresh the options.
    '''

    def _create_widgets(self, override_device_name=None):

        # create combo with livelink preset options

        combo = sb_widgets.SearchableComboBox(None)
        combo.setSizePolicy(QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Preferred)
        combo.setFocusPolicy(QtCore.Qt.FocusPolicy.StrongFocus)

        self._update_combo_items(combo, override_device_name)

        combo.currentTextChanged.connect(
            lambda: self._validate_and_commit_value(combo, override_device_name)
        )

        # create refresh button

        btn_refresh = sb_widgets.ControlQPushButton()
        btn_refresh.setIcon(QtGui.QIcon(':/icons/images/icon_refresh.png'))
        btn_refresh.setProperty("frameless", True)
        btn_refresh.setToolTip("Refresh list")
        btn_refresh.setDefault(False)
        btn_refresh.setAutoDefault(False)
        btn_refresh.clicked.connect(
            lambda: self._refresh_list(combo, override_device_name)
        )

        # create layout with combo and refresh button

        layout = QtWidgets.QHBoxLayout()

        layout.addWidget(combo)
        layout.addWidget(btn_refresh)

        # set the class' widget as the combo box itself
        self.set_widget(widget=combo, override_device_name=override_device_name)

        return layout

    def _refresh_list(self, combo:QtWidgets.QComboBox, override_device_name:Optional[str] = None):
        ''' Traveses the content to refresh the list of live link presets available '''

        DeviceUnreal.analyze_project_assets()
        self._update_combo_items(combo, override_device_name)

    def _validate_and_commit_value(self, combo: QtWidgets.QComboBox, override_device_name:Optional[str] = None):

        asset: Asset = combo.currentData()

        value_str = ''

        if asset:
            value_str = asset.gamepath  # we use path and not name because it is unambiguous

        self._on_widget_value_changed(value_str, override_device_name=override_device_name)

    def _update_combo_items(self, combo: QtWidgets.QComboBox, override_device_name):
        '''
        Populate the combobox itself with known list of live link presets available.
        Makes sure that the currently selected preset is preserved, unless it is not found,
        in which case it defaults to no preset.
        '''

        # cache the current value of the combo box (currently selected preset for this device)
        cur_value = self.get_value(override_device_name)

        # We will re-populate the combo box from scratch
        combo.clear()

        # add the empty/none choice
        noneasset = Asset(id=0, project=None, assettype=None, gamepath='', name='', localpath='')
        combo.addItem(noneasset.name, noneasset)

        project = SBCache().query_project(CONFIG.UPROJECT_PATH.get_value())
        if project is None:
            return

        assets = SBCache().query_assets_by_classname(
            project=project,
            classnames=self._classnames())

        assets = sorted(assets, key=lambda x: x.name)

        # generate the combo box items
        for idx, asset in zip(count(1), assets):
            combo.addItem(asset.name.removesuffix('.uasset'), asset)
            combo.setItemData(idx, asset.gamepath.removesuffix('.uasset'),
                              QtCore.Qt.ItemDataRole.ToolTipRole)

        # set the current index to the live link preset that was already selected
        for item_idx in range(combo.count()):
            if cur_value == combo.itemData(item_idx).gamepath:
                combo.setCurrentIndex(item_idx)
                break

    @abstractmethod
    def _classnames(self) -> Iterable[str]:
        ''' Return the valid class names of this type of asset '''
        assert False, "Must be implemented by derived classes"

    def _on_setting_changed(
            self, new_value: str,
            override_device_name: Optional[str] = None):

        combo = self.get_widget(override_device_name=override_device_name)

        if not combo:
            return

        try:
            asset: Asset = combo.currentData()
            old_value = asset.gamepath
        except (AttributeError, TypeError):
            old_value = ''

        new_str_value = new_value

        # if the value changed, find the new index in the combo box based on gamepath
        if new_str_value != old_value:
            for item_idx in range(combo.count()):
                asset: Asset = combo.itemData(item_idx)
                if new_str_value == asset.gamepath:
                    combo.setCurrentIndex(item_idx)
                    break


class LiveLinkPresetSetting(UAssetSetting):
    ''' Container of the LiveLinkPreset setting.
    '''
    def _classnames(self) -> Iterable[str]:
        ''' Return the valid class names of this type of asset '''
        return DeviceUnreal.LIVELINKPRESET_CLASS_NAMES


class MediaProfileSetting(UAssetSetting):
    ''' Container of the MediaProfile setting.
    '''
    def _classnames(self) -> Iterable[str]:
        ''' Return the valid class names of this type of asset '''
        return DeviceUnreal.MEDIAPROFILE_CLASS_NAMES


class SyncCategoryOption:
    ''' Used to represent a UGS sync filter category within a Setting. '''
    def __init__(self, catid: Union[str, uuid.UUID], name: str):
        if isinstance(catid, str):
            catid = uuid.UUID(catid)

        self.catid = catid
        self.name = name

    def __str__(self) -> str:
        ''' Determines the text displayed in the `MultiSelectionComboBox`. '''
        return self.name

    def __eq__(self, other: object):
        ''' Two category options compare equal if their UUIDs are equal. '''
        if isinstance(other, SyncCategoryOption):
            return self.catid == other.catid
        else:
            return False


# =============
# Unreal Device
# =============

class DeviceUnreal(Device):

    NDISPLAY_CLASS_NAMES = (
        'DisplayClusterBlueprint',
        '/Script/DisplayCluster.DisplayClusterBlueprint',
    )

    LIVELINKPRESET_CLASS_NAMES = (
        'LiveLinkPreset',
        '/Script/LiveLink.LiveLinkPreset',
    )

    MEDIAPROFILE_CLASS_NAMES = (
        'MediaProfile',
        '/Script/MediaFrameworkUtilities.MediaProfile',
        '/Script/MediaProfile.MediaProfile',  # module renamed in UE5
    )

    csettings: dict[str, Setting] = {
        'buffer_size': IntSetting(
            attr_name="buffer_size",
            nice_name="Buffer Size",
            value=1024,
            tool_tip=(
                "Buffer size used for communication with SwitchboardListener"),
            category="UE Settings",
        ),
        'command_line_arguments': StringSetting(
            attr_name="command_line_arguments",
            nice_name='Command Line Arguments',
            value="",
            tool_tip='Additional command line arguments for the engine',
            category="Command Line Args",
        ),
        'exec_cmds': StringListSetting(
            attr_name="exec_cmds",
            nice_name='ExecCmds',
            value=[],
            tool_tip='ExecCmds to be passed. No need for outer double quotes.',
            migrate_data=migrate_comma_separated_string_to_list,
            category="Command Line Args",
        ),
        'dp_cvars': StringListSetting(
            attr_name='dp_cvars',
            nice_name="DPCVars",
            value=[],
            tool_tip="Device profile console variables.",
            migrate_data=migrate_comma_separated_string_to_list,
            category="Command Line Args",
        ),
        'port': IntSetting(
            attr_name="port",
            nice_name="Listener Port",
            value=2980,
            tool_tip="Port of SwitchboardListener",
            category="Network Settings",
        ),
        'osc_port': IntSetting(
            attr_name='osc_port',
            nice_name='OSC Port',
            value=5500,
            tool_tip=(
                'Must match the port on which the Unreal Editor OSC server '
                'is configured to listen for OSC connections.'),
            category="Network Settings",
        ),
        'roles_filename': StringSetting(
            attr_name="roles_filename",
            nice_name="Roles Filename",
            value="VPRoles.ini",
            tool_tip=(
                "File that stores VirtualProduction roles. "
                "Default: Config/Tags/VPRoles.ini"),
            category="UE Settings",
        ),
        'stage_session_id': IntSetting(
            attr_name="stage_session_id",
            nice_name="Stage Session ID",
            value=0,
            tool_tip=(
                "An ID that groups Stage Monitor providers and monitors. "
                "Instances with different Session IDs are invisible to each "
                "other in Stage Monitor."),
            category="Tools Settings",
        ),
        'ue_exe': StringSetting(
            attr_name="editor_exe",
            nice_name="Unreal Editor filename",
            value="UnrealEditor.exe",
            category="General Settings",
        ),
        'max_gpu_count': OptionSetting(
            attr_name="max_gpu_count",
            nice_name="Number of GPUs",
            value=1,
            possible_values=list(range(1, 17)),
            tool_tip=(
                "If you have multiple GPUs in the PC, you can specify how "
                "many to use."),
            category="GPU/CPU Settings",
        ),
        'priority_modifier': OptionSetting(
            attr_name='priority_modifier',
            nice_name="Process Priority",
            value=sb_utils.PriorityModifier.Normal.name,
            possible_values=[p.name for p in sb_utils.PriorityModifier],
            tool_tip="Used to override the priority of the process.",
            category="GPU/CPU Settings",
        ),
        'auto_decline_package_recovery': BoolSetting(
            attr_name='auto_decline_package_recovery',
            nice_name='Skip Package Recovery',
            value=False,
            tool_tip=(
                'Automatically DISCARDS auto-saved packages at startup, '
                'skipping the restore prompt. Useful in multi-user '
                'scenarios, where restoring from auto-save may be '
                'undesirable.'),
            category="UE Settings",
        ),
        'udpmessaging_unicast_endpoint': StringSetting(
            attr_name='udpmessaging_unicast_endpoint',
            nice_name='Unicast Endpoint',
            value=':0',
            tool_tip=(
                'Local interface binding (-UDPMESSAGING_TRANSPORT_UNICAST) of '
                'the form {address}:{port}. If {address} is omitted, the '
                'device address is used.'),
            category="Network Settings",
        ),
        'udpmessaging_extra_static_endpoints': StringSetting(
            attr_name='udpmessaging_extra_static_endpoints',
            nice_name='Extra Static Endpoints',
            value='',
            tool_tip=(
                'Comma separated. Used to add static endpoints '
                '(-UDPMESSAGING_TRANSPORT_STATIC) in addition to those '
                'managed by Switchboard.'),
            category="Network Settings",
        ),
        'udpmessaging_multicast_endpoint': StringSetting(
            attr_name='udpmessaging_multicast_endpoint',
            nice_name='Multicast Endpoint',
            value='230.0.0.1:6666',
            tool_tip=(
                'Multicast group and port (-UDPMESSAGING_TRANSPORT_MULTICAST) '
                'in the {address}:{port} endpoint format. The multicast group address '
                'must be in the range 224.0.0.0 to 239.255.255.255.'),
            category="Network Settings",
        ),
        'log_download_dir': DirectoryPathSetting(
            attr_name='log_download_dir',
            nice_name='Log Download Dir',
            value='',
            tool_tip=(
                'Directory in which to store logs transferred from devices. '
                'If unset, defaults to $(ProjectDir)/Saved/Logs/Switchboard/'),
            category="General Settings",
        ),
        'rsync_port': IntSetting(
            attr_name='rsync_port',
            nice_name='Rsync Server Port',
            value=switchboard_application.RsyncServer.DEFAULT_PORT,
            tool_tip='Port number on which the rsync server should listen.',
            category="Network Settings",
        ),
        'listener_inactive_timeout': IntSetting(
            attr_name='listener_inactive_timeout',
            nice_name='Listener Timeout',
            value=5,
            tool_tip=(
                'Tells the connected Listener to wait at least N seconds '
                'between network messages before considering the connection '
                'to Switchboard lost and closing it with a timeout error.'),
            category="Network Settings",
        ),
        'slate_allow_throttling': BoolSetting(
            attr_name='slate_allow_throttling',
            nice_name='Allow Slate Throttling',
            value=False,
            tool_tip=(
                'Sets the Slate.bAllowThrottling cvar. When unchecked, the Editor viewports do not freeze/throttle \n'
                'during certain operations. Not thottling is typically desired when using the Editor in \n'
                'a virtual production stage.\n'),
            category="UE Settings",
        ),
        'retrieve_logs': BoolSetting(
            attr_name='retrieve_logs',
            nice_name='Retrieve Logs',
            value=True,
            tool_tip='When checked, retrieves the logs and traces after Unreal Engine terminates.',
            category="UE Settings",
        ),
        'logging': LoggingSetting(
            attr_name='logging',
            nice_name='Logging',
            value=None,
            categories=[
                'LogConcert',
                'LogImgMedia',
                'LogLiveLink',
                'LogMediaIOCore',
                'LogRemoteControl',
                'LogRivermaxMedia',
                'LogSharedMemoryMedia',
                'LogUdpMessaging',
            ],
            tool_tip='Logging categories and verbosity levels',
            category="UE Settings",
        ),
        'livelink_preset': LiveLinkPresetSetting(
            attr_name='livelink_preset',
            nice_name='LiveLink Preset',
            value='',
            tool_tip='Adds the selected LiveLink preset to the command line',
            category="Tools Settings",
        ),
        'mediaprofile': MediaProfileSetting(
            attr_name='mediaprofile',
            nice_name='Media Profile',
            value='',
            tool_tip=('Adds the selected Media Profile to the command line'),
            category="Tools Settings",
        ),
        'lock_gpu_clock': BoolSetting(
            attr_name="lock_gpu_clock",
            nice_name="Lock GPU Clock",
            value=False,
            tool_tip=(
                "Hint to lock the GPU clock to its allowed maximum. Requires SwitchboardListenerHelper \n"
                "to be running on the client machine, otherwise this option will be ignored."
            ),
            show_ui=True if sys.platform in ('win32', 'linux') else False,  # Only available in select platforms
            category="GPU/CPU Settings",
        ),
        'use_fast_sync': BoolSetting(
            attr_name='use_fast_sync',
            nice_name='Use Fast Sync',
            value=False,
            tool_tip='Controls whether Perforce sync operations use the revision range optimization',
            category="Source Control Settings",
        ),
        'use_sync_filters': BoolSetting(
            attr_name='use_sync_filters',
            nice_name='Use Sync Filters',
            value=False,
            tool_tip=(
                'Controls whether UnrealGameSync filter categories or custom '
                'views specified below are used during Perforce sync'),
            category="Source Control Settings",
        ),
        'included_sync_categories': MultiOptionSetting(
            attr_name='included_sync_categories',
            nice_name='Included Sync Filter Categories',
            value=[
                SyncCategoryOption('6703e989-d912-451d-93ad-b48de748d282',
                                   'Content'),
                SyncCategoryOption('f44b2d25-cbc0-4a8f-b6b3-e4a8125533dd',
                                   'Platform Support: Linux'),
                SyncCategoryOption('5206ccee-9024-4e36-8b89-f5f5a7d288d2',
                                   'Platform Support: Win64'),
                SyncCategoryOption('cfec942a-bb90-4f0c-accf-238ecaad9430',
                                   'Source Code'),
            ],
            tool_tip='UnrealGameSync filter categories to include during sync',
            category="Source Control Settings",
        ).with_get_json_override_fn(
            lambda val_list: [(str(x.catid), x.name) for x in val_list]
        ).with_config_set_override_fn(
            lambda val_list: [
                SyncCategoryOption(uuid.UUID(x[0]), x[1])
                for x in val_list]
        ),
        'custom_sync_view': StringSetting(
            attr_name='custom_sync_view',
            nice_name='Custom Sync View',
            value='',
            placeholder_text='-/Samples/Games/...,-.../ExcludePlatform/...',
            tool_tip=(
                'Comma separated. Used to specify freeform Perforce-style '
                'wildcards to be applied during sync.'),
            category="Source Control Settings",
        ),
        'force_holdout_if_ndisplay': BoolSetting(
            attr_name='force_holdout_if_ndisplay',
            nice_name='Force nDisplay Editor Holdout',
            value=True,
            tool_tip=(
                'When checked, force-enables Primitive Alpha Holdout on Editor launches when there are nDisplay devices in '
                'this Switchboard configuration \n'),
            category="UE Settings",
        ),
        'editor_target_override': StringSetting(
            attr_name='editor_target_override',
            nice_name='Editor Target Name Override',
            value='',
            placeholder_text='$(ProjectName)Editor',
            tool_tip=(
                "The name of your project's editor target. Typically $(ProjectName)Editor (which is inferred "
                "automatically). This override only needs to be set if deviating from the standard convention."),
            category='UE Settings',
        ),
        'soft_kill_timeout': FloatSetting(
            attr_name='soft_kill_timeout',
            nice_name='Soft Kill Timeout',
            value=3.0,
            tool_tip=(
                'Timeout in seconds for graceful process shutdown before force termination. '
                'Applies to Unreal and nDisplay devices. '
                'Processes have this amount of time to save state and clean up before being forcefully killed.'
            ),
            category="General Settings",
        ),
        'disable_ensures': BoolSetting(
            attr_name='disable_ensures',
            nice_name="Disable Ensures",
            value=True,
            tool_tip="When checked, disables the handling of ensure errors - which are non-fatal and may cause hitches.",
            category="UE Settings",
        ),
    }

    if ENABLE_UGS_SUPPORT:
        csettings['unrealgamesync_lib_dir'] = DirectoryPathSetting(
            attr_name='unrealgamesync_lib_dir',
            nice_name='UnrealGameSync Dll Dir',
            value="",
            tool_tip=(
                "Expects an absolute path to the device directory containing the ugs.dll library file. \n"
                "If left blank, switchboard will attempt to find UGS by searching the system PATH and default install locations. \n"
                "On Windows, the default install location is '${LOCALAPPDATA}/UnrealGameSync/Latest/'."),
            category="UGS Settings",
        )

    # A stand-in QObject to emit signals from classmethods.
    class StaticSignals(QtCore.QObject):
        ugs_config_updated_signal = QtCore.Signal(ugs_utils.IniParser)
        clipboard_level_added_signal = QtCore.Signal(str)
        clipboard_assets_added_signal = QtCore.Signal()
        project_assets_updated_signal = QtCore.Signal()

    static_signals = StaticSignals()

    unreal_started_signal = QtCore.Signal()

    mu_server = switchboard_application.get_multi_user_server_instance()
    rsync_server = switchboard_application.RsyncServer()
    ugs_config: Optional[ugs_utils.IniParser] = None

    # Monitors the local listener executable and notifies when the file is
    # changed.
    listener_watcher = ListenerWatcher()

    # Every DeviceUnreal (and derived class, e.g. DevicenDisplay) instance;
    # used for listener updates.
    active_unreal_devices: set[DeviceUnreal] = set()

    # Global live log management - only one device can stream live logs at a time
    active_live_log_device: Optional['DeviceUnreal'] = None

    # Flag used to batch together multiple rapid calls to
    # `_queue_notify_redeploy`.
    _pending_notify_redeploy = False

    class UgsConfigRunnable(QtCore.QRunnable):
        ''' Performs a UGS config update on the thread pool. '''

        class Signals(QtCore.QObject):
            result = QtCore.Signal(ugs_utils.IniParser)

        def __init__(self, engine_dir: Optional[str], proj_dir: Optional[str]):
            super().__init__()
            self.engine_dir = engine_dir
            self.proj_dir = proj_dir
            self.signals = self.Signals()

        def run(self):
            ugs_config = ugs_utils.parse_depot_ugs_configs(
                self.engine_dir, self.proj_dir)
            self.signals.result.emit(ugs_config)

    _ugs_config_runnable: Optional[UgsConfigRunnable] = None
    _pending_ugs_config_refresh = False

    @classmethod
    def p4_refresh_ugs_config(cls):
        if DeviceUnreal.ugs_config is None:
            # One-time initialization
            def refresh():
                DeviceUnreal.p4_refresh_ugs_config()

            CONFIG.P4_ENABLED.signal_setting_changed.connect(refresh)
            CONFIG.P4_ENGINE_PATH.signal_setting_changed.connect(refresh)
            CONFIG.P4_PROJECT_PATH.signal_setting_changed.connect(refresh)

        if not CONFIG.P4_ENABLED.get_value():
            DeviceUnreal.ugs_config = ugs_utils.IniParser()
            return

        if DeviceUnreal._pending_ugs_config_refresh:
            return
        elif DeviceUnreal._ugs_config_runnable:
            DeviceUnreal._pending_ugs_config_refresh = True
            return

        engine_dir = CONFIG.P4_ENGINE_PATH.get_value() or None
        project_dir = CONFIG.P4_PROJECT_PATH.get_value() or None
        DeviceUnreal._ugs_config_runnable = DeviceUnreal.UgsConfigRunnable(
            engine_dir, project_dir)

        DeviceUnreal._ugs_config_runnable.signals.result.connect(
            DeviceUnreal._on_updated_ugs_config)
        QtCore.QThreadPool.globalInstance().start(DeviceUnreal._ugs_config_runnable)

    @classmethod
    def _on_updated_ugs_config(cls, ugs_config: ugs_utils.IniParser):
        cls.ugs_config = ugs_config

        assert cls._ugs_config_runnable is not None
        cls._ugs_config_runnable.signals.result.disconnect(
            cls._on_updated_ugs_config)
        cls._ugs_config_runnable = None

        if cls._pending_ugs_config_refresh:
            cls._pending_ugs_config_refresh = False
            cls.p4_refresh_ugs_config()
            return

        sync_filters = ugs_utils.SyncFilters()
        sync_filters.read_categories_from_ini_parser(cls.ugs_config)
        sync_filters.categories = dict(sorted(
            sync_filters.categories.items(), key=lambda item: item[1].name))

        # The UUIDs are stable, but the names/object identities change
        include_setting = DeviceUnreal.csettings['included_sync_categories']
        include_setting.possible_values.clear()
        for category in sync_filters.categories.values():
            option = SyncCategoryOption(category.catid, category.name)
            include_setting.possible_values.append(option)

        cls.static_signals.ugs_config_updated_signal.emit(ugs_config)

    @classmethod
    def get_designated_local_builder(cls) -> Optional[DeviceUnreal]:
        '''
        Selects and returns a local `DeviceUnreal` (or derived class) device.

        This is the device tasked with building the multiuser server and
        listener executables.
        '''

        def ips_for_host(host: str) -> list[IPv4Address]:
            try:
                (primary_name, aliases, ipstrs) = socket.gethostbyname_ex(host)
                return [IPv4Address(ipstr) for ipstr in ipstrs]
            except Exception as exc:
                LOGGER.error('get_designated_local_builder(): '
                             f"gethostbyname_ex('{host}') exception", exc)
                return []

        # Use the set intersection of local IPs and device IPs to pick a device
        address_to_device_map = dict[IPv4Address, DeviceUnreal]()
        for device in cls.active_unreal_devices:
            for ip in ips_for_host(device.address):
                address_to_device_map[ip] = device

        local_addr_set = set(ips_for_host(SETTINGS.ADDRESS.get_value()))
        device_addr_set = set(address_to_device_map.keys())

        local_device_addrs = local_addr_set.intersection(device_addr_set)
        for local_device_addr in local_device_addrs:
            local_device = address_to_device_map[local_device_addr]
            if local_device.is_disconnected:
                continue
            else:
                return local_device

        try:
            return address_to_device_map[list(local_device_addrs)[0]]
        except (IndexError, KeyError):
            return None

    def is_designated_local_builder(self) -> bool:
        return self is DeviceUnreal.get_designated_local_builder()

    def is_connected_and_authenticated(self) -> bool:
        ''' Returns true if it is connected to the listener and authenticated.
        If true, it means that the application layer can have normal communiations with
        the device
        '''
        return self.unreal_client.is_connected and self.unreal_client.is_authenticated

    @QtCore.Slot()
    def _queue_notify_redeploy(self):
        # Ensure this code is run from the main thread
        if threading.current_thread() is not threading.main_thread():
            QtCore.QMetaObject.invokeMethod(
                self, '_queue_notify_redeploy', QtCore.Qt.QueuedConnection)
            return

        # Brief debounce in case we're still connecting multiple clients in
        # quick succession
        if not DeviceUnreal._pending_notify_redeploy:
            QtCore.QTimer.singleShot(100, self._notify_redeploy)
            DeviceUnreal._pending_notify_redeploy = True

    def _notify_redeploy(self):
        dlg = RedeployListenerDialog(
            DeviceUnreal.active_unreal_devices, DeviceUnreal.listener_watcher)
        dlg.exec()
        DeviceUnreal._pending_notify_redeploy = False

    def __init__(self, name, address, **kwargs):
        super().__init__(name, address, **kwargs)

        self.unreal_client = ListenerClient(
            address=self.address,
            port=DeviceUnreal.csettings['port'].get_value(self.name),
            buffer_size=DeviceUnreal.csettings['buffer_size'].get_value(self.name),
            parent=self,
        )

        roles = kwargs.get("roles", [])

        self.setting_roles = MultiOptionSetting(
            attr_name="roles",
            nice_name="Roles",
            value=roles,
            tool_tip="List of roles for this device",
            category="UE Settings",
        )

        self.setting_ddc_build_platforms = MultiOptionSetting(
            attr_name="ddc_build_platforms",
            nice_name="DDC Build Platforms",
            value=["Windows", "WindowsEditor", "Linux", "LinuxEditor"],
            tool_tip="List of platforms for which to build the DDC cache for this device",
            category="General Settings",
        )

        autojoin_mu_server = kwargs.get("autojoin_mu_server", True)
        self.autojoin_mu_server = BoolSetting(
            attr_name="autojoin_mu_server",
            nice_name="Auto join Multi-user Server",
            value=autojoin_mu_server,
            show_ui=False,
        )

        self.last_launch_command = StringSetting(
            attr_name="last_launch_command",
            nice_name="Last Launch Command",
            value=kwargs.get("last_launch_command", ''),
            show_ui=False,
        )

        self.last_log_path = FilePathSetting(
            attr_name="last_log_path",
            nice_name="Last Log Path",
            value=kwargs.get("last_log_path", ''),
            show_ui=False,
        )

        self.last_trace_path = FilePathSetting(
            attr_name="last_trace_path",
            nice_name="Last Insights Trace Path",
            value=kwargs.get("last_trace_path", ''),
            show_ui=False,
        )

        self.last_sync_filter_hash = StringSetting(
            attr_name="last_sync_filter_hash",
            nice_name="Last UGS Sync Filter Hash",
            value=kwargs.get('last_sync_filter_hash', ''),
            show_ui=False,
        )

        self.exclude_from_build = BoolSetting(
            attr_name="exclude_from_build",
            nice_name="Exclude from build",
            value=kwargs.get("exclude_from_build", False),
            tool_tip="Whether to exclude this device from builds",
            category="General Settings",
        )

        self.exclude_from_insights = BoolSetting(
            attr_name="exclude_from_insights",
            nice_name="Exclude from Insights trace",
            value=kwargs.get("exclude_from_insights", False),
            tool_tip="Whether to exclude device from Unreal Insights traces",
            category="General Settings",
        )

        self.setting_address.signal_setting_changed.connect(
            self.on_setting_address_changed)
        DeviceUnreal.csettings['port'].signal_setting_changed.connect(
            self.on_setting_port_changed)
        CONFIG.ENGINE_SYNC_METHOD.signal_setting_changed.connect(
            self.on_engine_sync_method_changed)

        weak_self = weakref.ref(self)

        osc_port_setting = DeviceUnreal.csettings['osc_port']

        osc_port_setting.signal_setting_changed.connect(
            lambda: weak_self() and weak_self()._check_recreate_osc_client()
        )
        osc_port_setting.signal_setting_overridden.connect(
            lambda: weak_self() and weak_self()._check_recreate_osc_client()
        )

        self.auto_connect = False

        self.runtime_str = ""

        # The CLs returned via syncstatus query, or optimistically
        # updated with the "inflight" values upon successful sync
        # (Defined in the base class, documented here for context)
        #self._project_changelist: Optional[int | str] = None
        #self._engine_changelist: Optional[int | str] = None

        # The CLs for which outstanding p4 syncs have been queued
        self.inflight_project_cl = None
        self.inflight_engine_cl = None
        self.inflight_sync_hash: Optional[str] = None

        # The fields read from Build.version JSON
        # (Written during sync by UGS/sbl_helper.py)
        self._synced_changelist: Optional[int] = None
        self._synced_compatible_cl: Optional[int] = None

        # The fields read from UnrealEditor.version JSON
        # (Written by UBT following a successful build of ANY project)
        self._built_eng_changelist: Optional[int] = None
        self._built_eng_compatible_cl: Optional[int] = None

        # The fields read from (Project)Editor.target JSON
        # (Written by UBT following a successful build of the active project)
        self._built_proj_changelist: Optional[int] = None
        self._built_proj_compatible_cl: Optional[int] = None

        listener_qt_handler = self.unreal_client.listener_qt_handler
        listener_qt_handler.listener_connecting.connect(
            super().connecting_listener, QtCore.Qt.QueuedConnection)
        listener_qt_handler.listener_connected.connect(
            super().connect_listener, QtCore.Qt.QueuedConnection)
        listener_qt_handler.listener_connected.connect(
            self._on_listener_connected, QtCore.Qt.QueuedConnection)
        listener_qt_handler.listener_connection_failed.connect(
            self._on_listener_connection_failed, QtCore.Qt.QueuedConnection)

        # Set a delegate method if the device gets a disconnect signal
        self.unreal_client.disconnect_delegate = (self.on_listener_disconnect)
        self.unreal_client.receive_file_completed_delegate = (self.on_file_received)
        self.unreal_client.receive_file_failed_delegate = (self.on_file_receive_failed)
        self.unreal_client.delegates["state"] = (self.on_listener_state)
        self.unreal_client.delegates["kill"] = (self.on_program_killed)
        self.unreal_client.delegates["programstdout"] = (self.on_listener_programstdout)
        self.unreal_client.delegates["program ended"] = (self.on_program_ended)
        self.unreal_client.delegates["program started"] = (self.on_program_started)
        self.unreal_client.delegates["program killed"] = (self.on_program_killed)
        # This catches start failures.
        self.unreal_client.delegates["start"] = (self.on_program_started)
        self.unreal_client.delegates["live log data"] = (self.on_live_log_data)

        # Live log streaming state
        self.live_log_streaming = False
        self.live_log_file_path = None
        self.live_log_local_file = None

        # Track log files per process UUID for resuming
        self.process_log_files = {}  # UUID -> {'local_file': pathlib.Path, 'last_position': int}

        self.osc_connection_timer = QtCore.QTimer(self)
        self.osc_connection_timer.timeout.connect(self._try_connect_osc)

        # on_program_started is called from inside the connection thread but
        # QTimer can only be used from the main thread.
        # so we connect to a signal sent on the connection threat and can then
        # start the timer on the main thread.
        self.unreal_started_signal.connect(self.on_unreal_started)

        # keeps track of programs to start
        self.program_start_queue = ProgramStartQueue()

        # Populated from the listener's state message on connect.
        self.platform_binary_directory: str = ''

        # Launch rsync server
        if not DeviceUnreal.rsync_server.is_running():
            # Set incoming log directory, and update on settings change.
            def update_incoming_logs_path():
                log_dir = DeviceUnreal.get_log_download_dir()
                try:
                    DeviceUnreal.rsync_server.set_incoming_logs_path(log_dir)
                except RuntimeError as exc:
                    LOGGER.error(
                        f'update_incoming_logs_path failed for path {log_dir}',
                        exc_info=exc)

            update_incoming_logs_path()

            log_download_dir = DeviceUnreal.csettings['log_download_dir']
            log_download_dir.signal_setting_changed.connect(
                update_incoming_logs_path)

            # UPROJECT_PATH is only relevant if log_download_dir is left blank.
            def update_log_path_on_project_path_changed():
                if not log_download_dir.get_value().strip():
                    update_incoming_logs_path()

            CONFIG.UPROJECT_PATH.signal_setting_changed.connect(
                update_log_path_on_project_path_changed)

            # Relaunch the server if the port setting is changed.
            def launch_rsync_server():
                if DeviceUnreal.rsync_server.is_running():
                    DeviceUnreal.rsync_server.shutdown()

                rsync_port = DeviceUnreal.csettings['rsync_port'].get_value()
                DeviceUnreal.rsync_server.launch(port=rsync_port)

            launch_rsync_server()
            DeviceUnreal.csettings[
                'rsync_port'].signal_setting_changed.connect(
                    launch_rsync_server)

        if self.ugs_config is None and self._ugs_config_runnable is None:
            self.p4_refresh_ugs_config()

        app = QtCore.QCoreApplication.instance()
        app.aboutToQuit.connect(self._on_about_to_quit)

        self._widget_classes: set[str] = set()

        # Notify user of any invalid settings
        self.check_settings_valid()

    def init(self, widget_class, icons):
        super().init(widget_class, icons)

        weak_self = weakref.ref(self)

        self.exclude_from_build.signal_setting_changed.connect(
            lambda _, new_value: weak_self() and weak_self().on_setting_exclude_from_build_changed(new_value)
        )
        self.on_setting_exclude_from_build_changed(self.exclude_from_build.get_value())

        self.exclude_from_insights.signal_setting_changed.connect(
            lambda _, new_value: weak_self() and weak_self().on_setting_exclude_from_insights_changed(new_value)
        )
        self.on_setting_exclude_from_insights_changed(self.exclude_from_insights.get_value())

    def should_allow_exit(self, close_req_id: int) -> bool:
        # Delegate to a class method which surveys all active devices.
        return DeviceUnreal._should_allow_exit(close_req_id)

    _last_close_req_id: Optional[int] = None

    @classmethod
    def _should_allow_exit(cls, close_req_id: int) -> bool:
        # We give each request one opportunity to return False; this serves to
        # deduplicate calls to each device of this class or its subclasses.
        if close_req_id == cls._last_close_req_id:
            return True
        else:
            cls._last_close_req_id = close_req_id

        log_xfer_devices = [
            dev for dev in DeviceUnreal.active_unreal_devices
            if dev.transfer_in_progress]

        if len(log_xfer_devices) == 0:
            return True

        log_xfer_names = [device.name for device in log_xfer_devices]

        msg_result = QtWidgets.QMessageBox.warning(
            None, 'Log Transfers In Progress',
            'The following devices are still transferring their log files to '
            f'Switchboard: {", ".join(log_xfer_names)}\n\n'
            'If you exit Switchboard, these transfers will be interrupted. '
            'Do you still want to exit Switchboard?',
            QtWidgets.QMessageBox.Yes | QtWidgets.QMessageBox.No)

        return msg_result == QtWidgets.QMessageBox.Yes

    @classmethod
    def added_device(cls, device: DeviceUnreal):
        '''
        Implementation of base class function. Called when one of our plugin
        devices has been added.
        '''
        assert device not in DeviceUnreal.active_unreal_devices
        DeviceUnreal.active_unreal_devices.add(device)

        DeviceUnreal.listener_watcher.update_listener_path(
            CONFIG.listener_path())

        DeviceUnreal.rsync_server.register_client(
            device, device.name, device.address)
        device.widget.signal_device_name_changed.connect(
            device.reregister_rsync_client)
        device.setting_address.signal_setting_changed.connect(
            device.reregister_rsync_client)

    @classmethod
    def removed_device(cls, device: DeviceUnreal):
        '''
        Implementation of base class function. Called when one of our plugin
        devices has been removed.
        '''
        assert device in DeviceUnreal.active_unreal_devices
        DeviceUnreal.active_unreal_devices.remove(device)

        DeviceUnreal.rsync_server.unregister_client(device)
        device.widget.signal_device_name_changed.disconnect(
            device.reregister_rsync_client)
        device.setting_address.signal_setting_changed.disconnect(
            device.reregister_rsync_client)

        if len(DeviceUnreal.active_unreal_devices) == 0:
            if DeviceUnreal.rsync_server.is_running():
                DeviceUnreal.rsync_server.shutdown()

    @QtCore.Slot()
    def _on_listener_connected(self):
        pass

    @QtCore.Slot()
    def _on_about_to_quit(self):
        if DeviceUnreal.rsync_server.is_running():
            DeviceUnreal.rsync_server.shutdown()

    @classmethod
    def plugin_settings(cls):
        return Device.plugin_settings() + list(DeviceUnreal.csettings.values())

    def setting_overrides(self):
        overrides = super().setting_overrides() + [
            Device.csettings['is_recording_device'],
            DeviceUnreal.csettings['osc_port'],
            DeviceUnreal.csettings['command_line_arguments'],
            DeviceUnreal.csettings['exec_cmds'],
            DeviceUnreal.csettings['dp_cvars'],
            DeviceUnreal.csettings['max_gpu_count'],
            DeviceUnreal.csettings['priority_modifier'],
            DeviceUnreal.csettings['auto_decline_package_recovery'],
            DeviceUnreal.csettings['udpmessaging_unicast_endpoint'],
            DeviceUnreal.csettings['udpmessaging_extra_static_endpoints'],
            DeviceUnreal.csettings['livelink_preset'],
            DeviceUnreal.csettings['mediaprofile'],
            CONFIG.ENGINE_DIR,
            CONFIG.SOURCE_CONTROL_WORKSPACE,
            CONFIG.PROJECT_WORKSPACE,
            CONFIG.UPROJECT_PATH,
        ]

        if ENABLE_UGS_SUPPORT:
            overrides.append(DeviceUnreal.csettings['unrealgamesync_lib_dir'])

        return overrides

    @classmethod
    def sort_setting_categories_from_preferred_list(cls, categories: Dict, preferred_order: list[str]) -> OrderedDict:
        ''' Sorts using the order in the given list of preferred categories '''

        # Create a dictionary to hold the preferred categories in their given order
        # Key will be the string, value will be the index  in the preferred_order list.
        preferred_dict = {key: idx for (idx, key) in enumerate(preferred_order)}

        # Custom sort function
        def custom_sort_key(item):
            key = item[0]
            # Sort by the preferred order if it exists. If it is not in the preferred list,
            # they will go at the end and sorted alphabetically
            return (preferred_dict.get(key, len(preferred_order)), key)

        # Sort the categories using the custom sort key. The sort keys returned for a given
        # element will be compared against others by sorted.
        return OrderedDict(sorted(categories.items(), key=custom_sort_key))

    @classmethod
    def sort_setting_categories(cls, categories: Dict) -> OrderedDict:
        ''' Overrides base implementation '''

        preferred_order = [
            'General Settings',
            'Source Control Settings',
            'Command Line Args',
            'Tools Settings',
            'GPU/CPU Settings',
            'UE Settings',
            'Render Settings',
            'Network Settings',
            'Multi-User Settings',
            'Multiplayer Settings',
            'Device Settings',
            'Misc',
        ]

        return DeviceUnreal.sort_setting_categories_from_preferred_list(
            categories=categories,
            preferred_order=preferred_order)

    def device_settings(self):
        return super().device_settings() + [
            self.setting_roles,
            self.setting_ddc_build_platforms,
            self.autojoin_mu_server,
            self.last_launch_command,
            self.last_log_path,
            self.last_trace_path,
            self.exclude_from_build,
            self.exclude_from_insights,
            self.last_sync_filter_hash,
        ]

    def check_settings_valid(self) -> bool:
        valid = True

        # Check this device's settings.
        device_extra_args_lower: str = self.extra_cmdline_args_setting.lower()
        if '-udpmessaging_transport_unicast' in device_extra_args_lower:
            LOGGER.error(
                f'{self.name}: Command line arguments include '
                '-UDPMESSAGING_TRANSPORT_UNICAST; use the "Unicast Endpoint" '
                'setting instead.')
            valid = False

        if '-udpmessaging_transport_static' in device_extra_args_lower:
            LOGGER.error(
                f'{self.name}: Command line arguments include '
                '-UDPMESSAGING_TRANSPORT_STATIC; use the '
                '"Extra Static Endpoints" setting instead.')
            valid = False

        # Also check the multi-user server settings.
        muserver_extra_args_lower: str = (
            CONFIG.MUSERVER_COMMAND_LINE_ARGUMENTS.get_value().lower())
        if '-udpmessaging_transport_unicast' in muserver_extra_args_lower:
            LOGGER.error(
                f'{self.name}: Multi-user server command line arguments '
                'include -UDPMESSAGING_TRANSPORT_UNICAST; use the '
                '"Unicast Endpoint" setting instead.')
            valid = False

        return valid

    def conform_asset_gamepath(self, gamepath: str, ext: str = '.uasset') -> str:
        ''' Conforms the given gamepath, which can include file extension, to the //Game/../MyAsset.MyAsset convention
        e.g.
            /Game/Folder/MyAssetName.uasset -> /Game/Folder/MyAssetName.MyAssetName
        '''

        if gamepath.endswith(ext):
            gamepath = gamepath[:len(gamepath)-len(ext)]
        
        name = os.path.basename(os.path.normpath(gamepath))

        return f"{gamepath}.{name}"

    def exec_command_for_livelink_preset(self, livelink_preset_gamepath: str) -> str:
        ''' Returns the exec command string to enable the given livelink preset gamepath
        LiveLink presets can be applied as ExecCmds
        e.g. of command:
            "LiveLink.Preset.Apply Preset=/Game/Folder/MyLiveLinkPreset.MyLiveLinkPreset"
        '''

        if not livelink_preset_gamepath:
            raise ValueError

        return f"LiveLink.Preset.Apply Preset={self.conform_asset_gamepath(livelink_preset_gamepath)}"

    def dpcvar_for_mediaprofile(self, mediaprofile_gamepath: str) -> str:
        ''' Returns the dpcvar assignment string to enable the given mediaprofile gamepath
        Media profiles can be applied as early cvars
        e.g.:
          "MediaUtils.StartupProfile=/Game/Media/MyMediaProfile.MyMediaProfile"
        '''

        if not mediaprofile_gamepath:
            raise ValueError

        return f"MediaUtils.StartupProfile={self.conform_asset_gamepath(mediaprofile_gamepath)}"

    @classmethod
    def plugin_header_widget_config(cls):
        """
        Combination of widgets that will be visualized in the plugin header.
        """
        return (
            super().plugin_header_widget_config() |
            PluginHeaderWidgets.OPEN_BUTTON |
            PluginHeaderWidgets.CHANGELIST_LABEL |
            PluginHeaderWidgets.AUTOJOIN_MU
        )

    def device_widget_registered(self, device_widget):
        ''' Device interface method '''

        super().device_widget_registered(device_widget)

        # Store device reference in widget for live log context menu
        device_widget._device = self

        device_widget.signal_exclude_from_build_toggled.connect(self.on_toggle_exclude_from_build)

        device_widget.signal_exclude_from_insights_toggled.connect(self.on_toggle_exclude_from_insights)

        # hook to open last log signal from widget
        device_widget.signal_open_last_log.connect(self.on_open_last_log)

        # hook to open last trace signal from widget
        device_widget.signal_open_last_trace.connect(self.on_open_last_trace)

        # hook to copy last launch signal from widget
        device_widget.signal_copy_last_launch_command.connect(self.on_copy_last_launch_command)

        # hook to live log signal from widget
        device_widget.signal_start_live_log.connect(self.on_start_live_log)

        self.update_settings_menu_state()

        device_widget.autojoin_mu.signal_device_widget_autojoin_mu.connect(self.on_autojoin_mu_ui_change)

    def on_setting_address_changed(self, _, new_address):
        LOGGER.info(f"Updating address for ListenerClient to {new_address}")
        self.unreal_client.address = new_address

    def on_setting_port_changed(self, _, new_port):
        if not DeviceUnreal.csettings['port'].is_overridden(self.name):
            LOGGER.info(f"Updating port for ListenerClient to {new_port}")
            self.unreal_client.port = new_port

    def on_engine_sync_method_changed(self, _, engine_sync_method):
        if engine_sync_method == EngineSyncMethod.Build_Engine.value:
            self.widget.engine_changelist_label.show()
            if not self.is_disconnected:
                self._request_syncstatus()
        else:
            self.widget.engine_changelist_label.hide()
            self.widget.update_build_info(self)

    def on_setting_exclude_from_build_changed(self, exclude_from_build):
        self.widget.update_exclude_from_build(exclude_from_build, not self.is_disconnected)

    def on_setting_exclude_from_insights_changed(self, exclude_from_insights):
        self.widget.exclude_from_insights = exclude_from_insights

    @property
    def device_osc_port(self) -> int:
        return DeviceUnreal.csettings['osc_port'].get_value(self.name)

    def set_slate(self, value):
        if not self.is_recording_device:
            return

        self.send_osc_message(osc.SLATE, value)

    def set_take(self, value):
        if not self.is_recording_device:
            return

        self.send_osc_message(osc.TAKE, value)

    def record_stop(self):
        if not self.is_recording_device:
            return

        self.send_osc_message(osc.RECORD_STOP, 1)

    def _request_roles_file(self):
        roles_filename = DeviceUnreal.csettings["roles_filename"].get_value(self.name)
        if not roles_filename:
            LOGGER.warning(f'{self.name}: Roles filename is unset; not requesting')
            return

        uproject_path: str = CONFIG.UPROJECT_PATH.get_value(self.name).replace('"', '')
        uproject_dir = os.path.dirname(uproject_path)
        roles_file_path = os.path.join(uproject_dir, "Config", "Tags", roles_filename)
        _, msg = message_protocol.create_copy_file_from_listener_message(roles_file_path)
        self.unreal_client.send_message(msg)

    def _request_version_files(self):
        '''
        Requests the Build.version, UnrealEditor.version and $(Proj)Editor.target files.
        On success, _on_receive_version_file is called with the file content.
        '''
        engine_dir = CONFIG.ENGINE_DIR.get_value(self.name)
        uproj_path = CONFIG.UPROJECT_PATH.get_value(self.name)
        proj_dir = os.path.dirname(uproj_path)
        platform_dir = self.platform_binary_directory

        proj_name, _ = os.path.splitext(os.path.basename(uproj_path))
        editor_target = DeviceUnreal.csettings['editor_target_override'].get_value()
        if not editor_target:
            editor_target = f'{proj_name}Editor'

        build_version_path = os.path.join(engine_dir, 'Build', 'Build.version')
        _, msg = message_protocol.create_copy_file_from_listener_message(build_version_path)
        self.unreal_client.send_message(msg)

        editor_version_path = os.path.join(engine_dir, 'Binaries', platform_dir,
                                           'UnrealEditor.version')
        _, msg = message_protocol.create_copy_file_from_listener_message(editor_version_path)
        self.unreal_client.send_message(msg)

        proj_target_path = os.path.join(proj_dir, 'Binaries', platform_dir,
                                        f'{editor_target}.target')
        _, msg = message_protocol.create_copy_file_from_listener_message(proj_target_path)
        self.unreal_client.send_message(msg)

    def _request_syncstatus(self):
        if not CONFIG.P4_ENABLED.get_value():
            return

        project_path = CONFIG.UPROJECT_PATH.get_value(self.name)
        engine_dir = CONFIG.ENGINE_DIR.get_value(self.name)
        workspace = CONFIG.SOURCE_CONTROL_WORKSPACE.get_value(self.name)
        project_workspace = CONFIG.PROJECT_WORKSPACE.get_value(self.name)

        syncstatus_args = [
            f'"{self.sbl_helper_path}"',
            'syncstatus',
            f'--project="{project_path}"',
        ]

        if CONFIG.ENGINE_SYNC_METHOD.get_value() == EngineSyncMethod.Build_Engine.value:
            syncstatus_args.append(f'--engine-dir="{engine_dir}"')

        if workspace:
            syncstatus_args.append(f'--p4client={workspace}')

        if project_workspace:
            syncstatus_args.append(f'--p4client-project={project_workspace}')

        if DeviceUnreal.csettings['use_sync_filters'].get_value():
            include_str = ''
            custom_view_str = self.csettings['custom_sync_view'].get_value()

            incl_categories: list[SyncCategoryOption] = self.csettings[
                'included_sync_categories'].get_value()
            if incl_categories:
                incl_categories.sort(key=lambda cat: str(cat.catid))
                include_str = ','.join(str(cat.catid) for cat in incl_categories)
                syncstatus_args.append(f'--include-categories="{include_str}"')

            if custom_view_str:
                syncstatus_args.append(f'--custom-view="{custom_view_str}"')

        program_name = UnrealJobs.SyncStatus.value

        working_dir = os.path.dirname(CONFIG.UPROJECT_PATH.get_value(self.name))

        puuid, msg = message_protocol.create_start_process_message(
            prog_path=self.python_path,
            prog_args=' '.join(syncstatus_args),
            prog_name=program_name,
            caller=self.name,
            working_dir=working_dir,
            update_clients_with_stdout=False,
            combined_stdout_stderr=False,
            hide=True,
        )

        self.program_start_queue.add(
            ProgramStartQueueItem(
                name=program_name,
                puuid_dependency=None,
                puuid=puuid,
                msg_to_unreal_client=msg,
            ),
            unreal_client=self.unreal_client,
        )

    @QtCore.Slot()
    def connect_listener(self):
        ''' Connects to the listener '''
        # Ensure this code is run from the main thread
        if threading.current_thread() is not threading.main_thread():
            QtCore.QMetaObject.invokeMethod(
                self, 'connect_listener', QtCore.Qt.QueuedConnection)
            return

        # This will start the connection process asynchronously. The base
        # class will be notified by signal whether the connection succeeded or
        # failed and will update the device's status
        # accordingly.
        self.unreal_client.connect()

    @QtCore.Slot()
    def _on_listener_connection_failed(self):
        self.device_qt_handler.signal_device_connect_failed.emit(self)

    @QtCore.Slot()
    def disconnect_listener(self):
        ''' Disconnects from the listener '''
        # Ensure this code is run from the main thread
        if threading.current_thread() is not threading.main_thread():
            QtCore.QMetaObject.invokeMethod(
                self, 'disconnect_listener', QtCore.Qt.QueuedConnection)
            return

        # Clear process log file tracking on disconnect
        if self.process_log_files:
            LOGGER.info(f"Clearing log file tracking for {len(self.process_log_files)} processes on disconnect")
            self.process_log_files.clear()

        super().disconnect_listener()
        self.unreal_client.disconnect()

    def _remote_path_uses_windows_separators(self) -> bool:
        ''' True for Win64 remotes, and pre-connection (when the remote's
        platform isn't yet known) to preserve prior behavior on Windows
        hosts. Use posixpath join when this is False so forward slashes
        are preserved in paths sent to a Linux/Mac remote.
        '''
        return self.platform_binary_directory in ('', 'Win64')

    @property
    def python_path(self) -> str:
        engine_dir = CONFIG.ENGINE_DIR.get_value(self.name)
        if self._remote_path_uses_windows_separators():
            return os.path.normpath(os.path.join(
                engine_dir, 'Binaries', 'ThirdParty', 'Python3',
                self.platform_binary_directory or 'Win64', 'python.exe'))
        return posixpath.join(
            engine_dir, 'Binaries', 'ThirdParty', 'Python3',
            self.platform_binary_directory, 'bin', 'python3')

    @property
    def sbl_helper_path(self) -> str:
        engine_dir = CONFIG.ENGINE_DIR.get_value(self.name)
        if self._remote_path_uses_windows_separators():
            return os.path.normpath(os.path.join(
                engine_dir, 'Plugins', 'VirtualProduction', 'Switchboard',
                'Source', 'Switchboard', 'sbl_helper.py'))
        return posixpath.join(
            engine_dir, 'Plugins', 'VirtualProduction', 'Switchboard',
            'Source', 'Switchboard', 'sbl_helper.py')

    def sync(
        self,
        engine_cl: Optional[int | str],
        project_cl: Optional[int | str],
        puuid_dependency: Optional[uuid.UUID] = None
    ) -> Optional[uuid.UUID]:
        if engine_cl == '':
            engine_cl = None

        if project_cl == '':
            project_cl = None

        if not engine_cl and not project_cl:
            LOGGER.warning(
                "Neither project nor engine changelist is selected. "
                "There is nothing to sync!")
            return None

        if isinstance(engine_cl, str):
            try:
                engine_cl = int(engine_cl)
            except ValueError:
                LOGGER.error(f'Invalid engine changelist specified for sync: {engine_cl}')
                return None

        if isinstance(project_cl, str):
            try:
                project_cl = int(project_cl)
            except ValueError:
                LOGGER.error(f'Invalid project changelist specified for sync: {project_cl}')
                return None

        for device in self.devices_sharing_workspace():
            if device.status >= DeviceStatus.OPEN:
                LOGGER.warning(f'{self.name}: Unable to sync, because another '
                               f'device ({device.name}) using the same '
                               'workspace is already running.')
                return None

        project_path = CONFIG.UPROJECT_PATH.get_value(self.name)
        engine_dir = CONFIG.ENGINE_DIR.get_value(self.name)
        workspace = CONFIG.SOURCE_CONTROL_WORKSPACE.get_value(self.name)
        project_workspace = CONFIG.PROJECT_WORKSPACE.get_value(self.name)

        # If only we're syncing the project and the project has a specified workspace, use the project workspace
        # instead.
        if not engine_cl and project_cl and project_workspace:
            workspace = project_workspace

        program_name = UnrealJobs.Sync.value

        # If there are no task dependencies, check if it is already on its way:
        if not puuid_dependency:
            try:
                existing_puuid = self.program_start_queue.puuid_from_name(
                    program_name)
                LOGGER.info(
                    f"{self.name}: Already syncing with puuid {existing_puuid}")
                return None
            except KeyError:
                pass

        sync_method = CONFIG.ENGINE_SYNC_METHOD.get_value()
        generate_proj_files = sync_method == EngineSyncMethod.Build_Engine.value
        if ENABLE_UGS_SUPPORT:
            sync_precompiled_bins = sync_method == EngineSyncMethod.Sync_PCBs.value
            sync_using_ugs = sync_precompiled_bins or sync_method == EngineSyncMethod.Sync_From_UGS.value

        project_name = os.path.basename(os.path.dirname(project_path))
        LOGGER.info(
            f"{self.name}: Queuing sync for project {project_name} "
            f"(revisions: engine={engine_cl}, project={project_cl})")

        sync_args = [
            f'"{self.sbl_helper_path}"',
            # '--log-level=DEBUG',
            'sync',
            f'--project="{project_path}"',
        ]

        if engine_cl:
            sync_args.append(f'--engine-dir="{engine_dir}"')

        if not DeviceUnreal.csettings['use_fast_sync'].get_value():
            sync_args.append('--no-fast-sync')

        sync_args = ' '.join(sync_args)

        if workspace:
            sync_args += f' --p4client={workspace}'

        if engine_cl:
            sync_args += f' --engine-cl={engine_cl}'
            self.inflight_engine_cl = engine_cl
        else:
            self.inflight_engine_cl = None

        if project_cl:
            sync_args += f' --project-cl={project_cl} --clobber-project'
            self.inflight_project_cl = project_cl
        else:
            self.inflight_project_cl = None

        if generate_proj_files:
            sync_args += ' --generate'

        if DeviceUnreal.csettings['use_sync_filters'].get_value():
            include_str = ''
            custom_view_str = self.csettings['custom_sync_view'].get_value()

            incl_categories: list[SyncCategoryOption] = self.csettings[
                'included_sync_categories'].get_value()
            if incl_categories:
                incl_categories.sort(key=lambda cat: str(cat.catid))
                include_str = ','.join(str(cat.catid) for cat in incl_categories)
                sync_args += f' --include-categories="{include_str}"'

            if custom_view_str:
                sync_args += f' --custom-view="{custom_view_str}"'

            # If the user changes their sync filters, we need to rescan their
            # entire workspace to ensure we remove newly-excluded files.
            # We make this determination by concatenating and hashing their
            # selected filters, and then comparing that hash between syncs.
            new_hash_input_bytes = bytearray()
            new_hash_input_bytes.extend(include_str.encode('utf-8'))
            new_hash_input_bytes.extend(custom_view_str.encode('utf-8'))
            new_hash = hashlib.sha1(new_hash_input_bytes).hexdigest()
            prev_hash = self.last_sync_filter_hash.get_value()
            if new_hash != prev_hash:
                LOGGER.info(
                    f'Filter has changed ({prev_hash} -> {new_hash}); '
                    'finding files in workspace that need to be removed.')
                sync_args += ' --remove-excluded-workspace-files'

                # If the sync is interrupted, we'll be in an indeterminate
                # state, and we should always clean next time.
                self.last_sync_filter_hash.update_value('INVALID')
                self.inflight_sync_hash = new_hash  # persisted on sync success

        if ENABLE_UGS_SUPPORT:
            if sync_using_ugs:
                sync_args += ' --use-ugs'
                if self.unrealgamesync_lib_dir_setting:
                    sync_args += f' --ugs-lib-dir={self.unrealgamesync_lib_dir_setting}'

            if sync_precompiled_bins:
                sync_args += ' --use-pcbs'

                # If we're syncing 'Precompiled Binaries' one of those binaries may be the SwitchboardListener
                # executable which means (on Windows atleast) we need to move the executable to make way for the new one
                _, msg = message_protocol.create_free_listener_bin_message()
                self.unreal_client.send_message(msg)

        puuid, msg = message_protocol.create_start_process_message(
            prog_path=self.python_path,
            prog_args=sync_args,
            prog_name=program_name,
            caller=self.name,
            working_dir=os.path.dirname(
                CONFIG.UPROJECT_PATH.get_value(self.name)),
            update_clients_with_stdout=True,
            hide=True,
        )

        self.program_start_queue.add(
            ProgramStartQueueItem(
                name=program_name,
                puuid_dependency=puuid_dependency,
                puuid=puuid,
                msg_to_unreal_client=msg,
                launch_fn=lambda: LOGGER.info(
                    f"{self.name}: Sending sync command: "
                    f"{self.python_path} {sync_args}")
            ),
            unreal_client=self.unreal_client,
        )

        for device in self.devices_sharing_workspace():
            if device.status != DeviceStatus.SYNCING:
                device.status = DeviceStatus.SYNCING

        return puuid

    def build(self):
        if self.exclude_from_build.get_value():
            return

        # This should only be possible when invoked on individual devices
        # manually; SwitchboardDialog._sync_build_all already only invokes
        # this method for a single device per workspace.
        for device in self.devices_sharing_workspace():
            if device.status >= DeviceStatus.BUILDING:
                LOGGER.warning(f'{self.name}: Unable to build, because '
                               f'another device ({device.name}) using the '
                               'same workspace is already running/building.')
                return

        program_name = UnrealJobs.BuildProject.value

        # check if it is already on its way:
        try:
            existing_puuid = self.program_start_queue.puuid_from_name(program_name)
            LOGGER.info(f"{self.name}: Already building with puuid {existing_puuid}")
            return
        except KeyError:
            pass

        # check for any sync dependencies
        sync_puuid = None

        try:
            sync_puuid = self.program_start_queue.puuid_from_name(UnrealJobs.Sync.value)
        except KeyError:
            pass

        if sync_puuid:
            LOGGER.debug(f"{self.name} Queuing build after sync with puuid {sync_puuid}")

        # Build dependency chain
        puuid_dependency = sync_puuid

        # We've picked one device to act as build representative for our workspace;
        # however, we want to let the others weigh in, especially derived classes.
        for device in self.devices_sharing_workspace(only_connected=False):
            puuid_dependency = device._queue_all_builds(self, puuid_dependency)

    def _queue_all_builds(
        self,
        requesting_device: DeviceUnreal,
        puuid_dependency: uuid.UUID | None = None,
    ) -> uuid.UUID | None:
        '''
        Customization point for derived classes to customize build actions.

        The passed-in `requesting_device` may be a different device, sharing our workspace, to use as a proxy.

        Adding "redundant" actions should be safe; `_queue_build` (and subsequently the various `_build_*` wrapper
        methods) will early out when attempting to queue a second build with the same name.
        '''

        if (self.is_designated_local_builder() and
                CONFIG.ENGINE_SYNC_METHOD.get_value() == EngineSyncMethod.Build_Engine.value):
            # Build multi-user server
            if CONFIG.MUSERVER_AUTO_BUILD.get_value():
                if DeviceUnreal.mu_server.is_running():
                    mb_ret = QtWidgets.QMessageBox.question(
                        None, 'Terminate multi-user server?',
                        'The multi-user server is currently running. '
                        'Would you like to terminate it so that it can be '
                        'updated by the build?',
                        QtWidgets.QMessageBox.Yes | QtWidgets.QMessageBox.No)

                    if mb_ret == QtWidgets.QMessageBox.Yes:
                        DeviceUnreal.mu_server.process.kill()

                puuid_dependency = requesting_device._build_mu_server(puuid_dependency)
                puuid_dependency = requesting_device._build_mu_slate_server(puuid_dependency)

            # TODO: Build listener
            # - Handle the running listener's exe being locked
            # - Parse SwitchboardListenerVersion.h, skip if unchanged?

            # if CONFIG.LISTENER_AUTO_BUILD:
            #     puuid_dependency = requesting_device._build_listener(puuid_dependency)

        puuid_dependency = requesting_device._build_shadercompileworker(puuid_dependency)
        puuid_dependency = requesting_device._build_project(puuid_dependency)

        return puuid_dependency

    @property
    def target_platform(self) -> str:
        ''' Returns the Unreal target platform name for e.g. build actions '''
        # FIXME?: Not strictly correct, but holds for desktop host platforms
        return self.platform_binary_directory

    def _get_editor_target_name(self) -> Optional[str]:
        ''' Attempts to find the specific editor target name for the project, mimicking UGS.
        
        Uses hierarchical detection of the editor target:

            Priority 1: Config/DefaultEngine.ini
            Priotity 2: Source/{PROJECT}Editor.Target.cs
            Priority 3: Source/*Editor.Target.cs

        If not found, we expect the caller to use -Target=Editor in the UGS command line.

        Returns the full editor target name, or None if not found.
        '''

        uproject_path = CONFIG.UPROJECT_PATH.get_value(self.name)

        if not uproject_path:
            return None
            
        try:
            uproject_file = pathlib.Path(uproject_path)
            if not uproject_file.exists():
                return None
                
            project_name = uproject_file.stem
            
            # Priority 1: Config/DefaultEngine.ini
            # Looks for DefaultEditorTarget in its [/Script/BuildSettings.BuildSettings] section

            config_dir = uproject_file.parent / 'Config'
            engine_ini_path = config_dir / 'DefaultEngine.ini'
            
            if engine_ini_path.exists():
                try:
                    parser = ugs_utils.IniParser()
                    with open(engine_ini_path, 'r', encoding='utf-8') as f:
                        parser.read_file(f)
                    
                    default_target = parser.try_get_raw('/Script/BuildSettings.BuildSettings', 'DefaultEditorTarget')

                    if default_target:
                        LOGGER.debug(f"{self.name}: Found DefaultEditorTarget '{default_target}' in DefaultEngine.ini")
                        return default_target
                        
                except Exception as e:
                    LOGGER.warning(f"{self.name}: Failed to parse DefaultEngine.ini: {e}")
            
            # Priority 2: Source/{PROJECT}Editor.Target.cs

            source_dir = uproject_file.parent / 'Source'

            # No project-specific editor targets in blueprint only projects.
            if not source_dir.exists():
                return None
                
            file_suffix = '.Target.cs'
            expected_target_file = source_dir / f"{project_name}Editor{file_suffix}"
            
            if expected_target_file.exists():
                target_name = f"{project_name}Editor"
                LOGGER.debug(f"{self.name}: Found project-specific editor target: {target_name}")
                return target_name
            
            # Priority 3: Source/*Editor.Target.cs

            editor_targets = list(source_dir.glob(f'*Editor{file_suffix}'))
            
            if len(editor_targets) == 1:
                target_file = editor_targets[0]
                target_name = target_file.name[:-len(file_suffix)]
                LOGGER.debug(f"{self.name}: Found single editor target: {target_name}")
                return target_name
                    
            elif len(editor_targets) > 1:

                # Multiple targets found, but no clear winner.

                target_names = [f.name[:-len(file_suffix)] for f in editor_targets]

                LOGGER.warning(
                    f"{self.name}: Multiple editor targets found but no DefaultEditorTarget configured: {target_names}"
                )
                
        except Exception as e:
            LOGGER.warning(f"{self.name}: Failed to detect editor target name: {e}")
            
        return None

    def _build_project(self, puuid_dependency: Optional[uuid.UUID] = None) -> uuid.UUID | None:

        # Try to get specific project editor target name like UGS does
        # If found (e.g. MyProjectNameEditor), use it instead of `-TargetType=Editor`
        # Otherwise fall back to `-TargetType=Editor`

        editor_target = self._get_editor_target_name()
        
        if editor_target:
            ubt_args = (
                f'{editor_target} {self.target_platform} Development '
                f'-project="{CONFIG.UPROJECT_PATH.get_value(self.name)}" '
                '-Progress -NoHotReloadFromIDE')
        else:
            ubt_args = (
                f'{self.target_platform} Development '
                f'-project="{CONFIG.UPROJECT_PATH.get_value(self.name)}" '
                '-TargetType=Editor -Progress -NoHotReloadFromIDE')

        return self._queue_build('project', ubt_args, puuid_dependency)

    def _build_mu_server(self, puuid_dependency: Optional[uuid.UUID] = None) -> uuid.UUID | None:
        ubt_args = f'UnrealMultiUserServer {self.target_platform} Development -Progress'
        return self._queue_build('mu_server', ubt_args, puuid_dependency)

    def _build_mu_slate_server(self, puuid_dependency: Optional[uuid.UUID] = None) -> uuid.UUID | None:
        ubt_args = f'UnrealMultiUserSlateServer {self.target_platform} Development -Progress'
        return self._queue_build('mu_slate_server', ubt_args, puuid_dependency)

    def _build_listener(self, puuid_dependency: Optional[uuid.UUID] = None) -> uuid.UUID | None:
        ubt_args = f'SwitchboardListener {self.target_platform} Development -Progress'
        return self._queue_build('listener', ubt_args, puuid_dependency)

    def _build_shadercompileworker(self, puuid_dependency: Optional[uuid.UUID] = None) -> uuid.UUID | None:
        ubt_args = f'ShaderCompileWorker {self.target_platform} Development -Progress'
        return self._queue_build('shadercw', ubt_args, puuid_dependency)

    def _queue_build(
        self,
        program_name_suffix: str,
        ubt_args: str,
        puuid_dependency: Optional[uuid.UUID] = None,
    ) -> uuid.UUID | None:
        '''
        Adds a UBT invocation to the queue as "build_{program_name_suffix}",
        but only if an entry with that name does not already exist.

        If no new build was queued, returns the `puuid_dependency` which was
        passed in, in order to make chaining calls more convenient.
        '''

        program_name = f"{UnrealJobs.BuildPrefix.value}{program_name_suffix}"

        try:
            existing_puuid = self.program_start_queue.puuid_from_name(program_name)
            LOGGER.debug(f'{self.name}: Not queuing {program_name}; exists as {existing_puuid}')
            return puuid_dependency
        except KeyError:
            pass

        engine_path = CONFIG.ENGINE_DIR.get_value(self.name)
        if self.target_platform.lower().startswith('win'):
            ubt_path = os.path.join(engine_path, 'Build', 'BatchFiles',
                                    'Build.bat')
        else:
            ubt_path = posixpath.join(engine_path, 'Build', 'BatchFiles',
                                      'Linux', 'Build.sh')

        puuid, msg = message_protocol.create_start_process_message(
            prog_path=ubt_path,
            prog_args=ubt_args,
            prog_name=program_name,
            caller=self.name,
            update_clients_with_stdout=True,
            hide=True,
        )

        def launch_fn():
            LOGGER.info(
                f"{self.name}: Sending {program_name} command: "
                f"{ubt_path} {ubt_args}")

            for device in self.devices_sharing_workspace():
                device.status = DeviceStatus.BUILDING

        # Queue the build command
        self.program_start_queue.add(
            ProgramStartQueueItem(
                name=program_name,
                puuid_dependency=puuid_dependency,
                puuid=puuid,
                msg_to_unreal_client=msg,
                launch_fn=launch_fn,
            ),
            unreal_client=self.unreal_client,
        )

        return puuid

    def close(self, force=False, soft_kill_timeout=None):
        # This call only refers to "unreal" programs.
        unreal_puuids = self.program_start_queue.running_puuids_named(UnrealJobs.Unreal.value)

        # Get soft kill timeout for this device (use provided value or setting)
        if soft_kill_timeout is None:
            soft_kill_timeout = DeviceUnreal.csettings['soft_kill_timeout'].get_value(self.name)

        for unreal_puuid in unreal_puuids:
            _, msg = message_protocol.create_kill_process_message(unreal_puuid, soft_kill_timeout)
            self.unreal_client.send_message(msg)

        if not len(unreal_puuids):
            self.status = DeviceStatus.CLOSED
        else:
            self.status = DeviceStatus.CLOSING

    def disable_fso(self):
        ''' Tries to disable Full Screen Optimizations on the remote UnrealEditor.exe executable '''
        unreals = self.program_start_queue.running_programs_named(UnrealJobs.Unreal.value)

        if not len(unreals):
            return

        puuid = unreals[0].puuid

        if not puuid:
            return

        _, msg = message_protocol.create_fixExeFlags_message(puuid)
        self.unreal_client.send_message(msg)

    def minimize_windows(self):
        ''' Tries to minimize all windows. Windows-only; no-op elsewhere. '''
        if self.platform_binary_directory != 'Win64':
            return

        _, msg = message_protocol.create_minimize_windows_message()
        self.unreal_client.send_message(msg)

    @property
    def executable_filename(self):
        return DeviceUnreal.csettings['ue_exe'].get_value()

    @property
    def extra_cmdline_args_setting(self) -> str:
        return DeviceUnreal.csettings[
            'command_line_arguments'].get_value(self.name)

    @property
    def udpmessaging_unicast_endpoint_setting(self) -> str:
        return DeviceUnreal.csettings[
            'udpmessaging_unicast_endpoint'].get_value(self.name)

    @property
    def udpmessaging_extra_static_endpoints_setting(self) -> str:
        return DeviceUnreal.csettings[
            'udpmessaging_extra_static_endpoints'].get_value(self.name)

    @property
    def unrealgamesync_lib_dir_setting(self) -> str:
        return DeviceUnreal.csettings[
            'unrealgamesync_lib_dir'].get_value(self.name)

    def generate_unreal_exe_path(self):
        # CONFIG.engine_exe_path keys off the host's sys.platform, which is
        # wrong when the Switchboard host and the target node disagree
        # (e.g. Windows host driving a Linux nDisplay node). Build the
        # path from the remote device's platform_binary_directory instead.
        engine_dir = CONFIG.ENGINE_DIR.get_value(self.name)
        if self._remote_path_uses_windows_separators():
            return CONFIG.engine_exe_path(engine_dir, self.executable_filename)

        exe_basename = self.executable_filename
        if exe_basename.lower().endswith('.exe'):
            exe_basename = exe_basename[:-len('.exe')]
        return posixpath.join(
            engine_dir, 'Binaries', self.platform_binary_directory,
            exe_basename)

    @staticmethod
    def any_devices_have_roles():
        device_list = DeviceUnreal.active_unreal_devices
        for device in device_list:
            if len(device.get_vproles()[0]):
                return True
        return False

    def get_vproles(self):
        ''' Gets selected vp roles that are also present in the ini file
        Also returns any selected vp roles that are not in the ini file.
        '''

        vproles = self.setting_roles.get_value()
        missing_roles = [
            role for role in vproles
            if role not in self.setting_roles.possible_values]
        vproles = [role for role in vproles if role not in missing_roles]

        return vproles, missing_roles

    @property
    def udpmessaging_multicast_endpoint(self) -> str:
        device_multicast = DeviceUnreal.csettings[
            'udpmessaging_multicast_endpoint'].get_value().strip()
        return DeviceUnreal.csettings[
            'udpmessaging_multicast_endpoint'].get_value().strip()

    @classmethod
    def get_muserver_endpoint(cls) -> str:
        setting_val = CONFIG.MUSERVER_ENDPOINT.get_value().strip()
        if setting_val:
            return sb_utils.expand_endpoint(setting_val,
                                            SETTINGS.ADDRESS.get_value().strip())
        else:
            return ''

    @property
    def udpmessaging_unicast_endpoint(self) -> str:
        setting_val = self.udpmessaging_unicast_endpoint_setting.strip()
        if setting_val:
            return sb_utils.expand_endpoint(setting_val, self.address)
        else:
            return ''

    def build_udpmessaging_static_endpoint_list(self) -> list[str]:
        endpoints: list[str] = []

        # Multi-user server.
        if CONFIG.MUSERVER_AUTO_ENDPOINT.get_value():
            endpoints.append(DeviceUnreal.get_muserver_endpoint())

        # Any additional endpoints manually specified via settings.
        extra_endpoints = \
            self.udpmessaging_extra_static_endpoints_setting.split(',')
        endpoints.extend(endpoint.strip() for endpoint in extra_endpoints)

        endpoints = list(filter(None, endpoints))

        return endpoints

    def build_udpmessaging_args(self) -> list[str]:
        args: list[str] = []
        if self.udpmessaging_multicast_endpoint:
            server_mc = CONFIG.MUSERVER_MULTICAST_ENDPOINT.get_value().strip()
            if server_mc != self.udpmessaging_multicast_endpoint:
                LOGGER.warning(f"{self.name} contains a multicast endpoint ('{self.udpmessaging_multicast_endpoint}') that does not match configured value on Multi-user server ('{server_mc}').")
            args.append(f'-UDPMESSAGING_TRANSPORT_MULTICAST="{self.udpmessaging_multicast_endpoint}"')

        if self.udpmessaging_unicast_endpoint:
            args.append(f'-UDPMESSAGING_TRANSPORT_UNICAST="{self.udpmessaging_unicast_endpoint}"')

        static_endpoints = self.build_udpmessaging_static_endpoint_list()
        if len(static_endpoints) > 0:
            args.append(f'-UDPMESSAGING_TRANSPORT_STATIC="{",".join(static_endpoints)}"')

        return args

    def get_utrace_filepath(self):
        return self.get_remote_log_path() / f'{self.name}_{self.runtime_str}.utrace'

    @staticmethod
    def add_or_override_cvars(cvars:list[str], new_cvars:list[str]):
        ''' Adds new cvars, or overrides values if already existing.

        Args:
            cvars     : Cvars to be overridden
            new_cvars : Cvars to override or add

        Returns:
            list[str]: Resulting list of cvars.
        '''

        cvars_map = OrderedDict() # keep the original order

        for cvar in cvars + new_cvars:

            parts = cvar.strip().split('=')

            if len(parts) != 2:
                continue

            name = parts[0]
            value = parts[1]

            cvars_map[name.lower()] = (name,value)

        return [f'{cvar[0]}={cvar[1]}' for cvar in cvars_map.values()]

    def conform_dpcvars(self, cvars: list[str]) -> list[str]:
        ''' Conforms input cvar to expected format for command line (key=value)
        It will allow the user to separate key/value using space(s), like one would in Unreal's command input field.
        '''
        ocvars = []

        disallowed_chars = [
            ',',  # Comma is used as a dpcvar delimiter.
        ]

        for cvar in cvars:
            # remove extra spaces
            cvar = cvar.strip()

            # Split by '=' (allow surround by spaces) or 1+ spaces
            match = re.split(r'\s*=\s*|\s{1,}', cvar, maxsplit=1)

            if len(match) != 2:
                LOGGER.warning(f"Ignored cvar '{cvar}' because it could not be parsed")
                continue

            key, value = match

            # Check for empty key or value
            if not key or not value:
                LOGGER.warning(f"Ignored cvar '{cvar}' because it has an empty key or value")
                continue

            cvar = f"{key}={value}"

            # Check for disallowed characters
            if (any(char in cvar for char in disallowed_chars)):
                LOGGER.warning(f"Ignored cvar '{cvar}' because it contained one or more invalid characters ({disallowed_chars})")
                continue

            ocvars.append(cvar)

        return ocvars

    def generate_unreal_command_line_args(self, map_name):

        # Build command line arguments as an array that will be joined at the end.
        args: list[str] = []

        # DPCVars - accumulate these before adding to args
        dp_cvars = []

        # Project path and map
        args.append(f'"{CONFIG.UPROJECT_PATH.get_value(self.name)}"')
        if map_name:
            args.append(map_name)

        # Extra command line arguments from settings
        if self.extra_cmdline_args_setting:
            args.append(self.extra_cmdline_args_setting)

        # Logging
        args.append(f'Log={self.log_filename}')

        # Multi-user server settings
        if CONFIG.MUSERVER_AUTO_JOIN.get_value() and self.autojoin_mu_server.get_value():
            args.extend([
                '-CONCERTRETRYAUTOCONNECTONERROR',
                '-CONCERTAUTOCONNECT'
            ])

        if CONFIG.MUSERVER_FILESHARING_ENABLED.get_value() and CONFIG.MUSERVER_FILESHARING_SRV.get_value():
            filesharing = CONFIG.MUSERVER_FILESHARING_SRV.get_value()
            args.extend([
                '-CONCERTFILESHARINGENABLED=1',
                f'-CONCERTFILESHAREPATH={filesharing}'])

        mu_server = switchboard_application.get_multi_user_server_instance()
        args.extend([
            f'-CONCERTSERVER="{mu_server.configured_server_name()}"',
            f'-CONCERTSESSION="{CONFIG.MUSERVER_SESSION_NAME.get_value()}"',
            f'-CONCERTDISPLAYNAME="{self.name}"'
        ])

        # Insights tracing
        if CONFIG.INSIGHTS_TRACE_ENABLE.get_value() and not self.exclude_from_insights.get_value():
            LOGGER.warning(f"Unreal Insight Tracing is enabled for '{self.name}'. This may affect Unreal Engine performance.")
            remote_utrace_path = self.get_utrace_filepath()

            if CONFIG.INSIGHTS_STAT_EVENTS.get_value():
                args.append('-statnamedevents')

            args.extend([
                f'-tracefile="{remote_utrace_path}"',
                f'-trace="{CONFIG.INSIGHTS_TRACE_ARGS.get_value()}"'
            ])

        # Exec commands
        exec_cmds = DeviceUnreal.csettings["exec_cmds"].get_value(self.name).copy()

        # LiveLink preset
        livelink_preset_gamepath = DeviceUnreal.csettings["livelink_preset"].get_value(self.name)
        if livelink_preset_gamepath:
            exec_cmds.append(self.exec_command_for_livelink_preset(livelink_preset_gamepath))

        # Media profile
        mediaprofile_gamepath = DeviceUnreal.csettings["mediaprofile"].get_value(self.name)
        if mediaprofile_gamepath:
            dp_cvars.append(self.dpcvar_for_mediaprofile(mediaprofile_gamepath))

        # Filter and format exec commands
        exec_cmds = [cmd for cmd in exec_cmds if len(cmd.strip())]
        if len(exec_cmds):
            exec_cmds_expanded = ','.join(exec_cmds)
            args.append(f'-ExecCmds="{exec_cmds_expanded}"')

        # VP Roles
        (supported_roles, unsupported_roles) = self.get_vproles()
        if supported_roles or DeviceUnreal.any_devices_have_roles():
            args.append('-VPRole=' + '|'.join(supported_roles))

        if unsupported_roles:
            LOGGER.error(
                f"{self.name}: Omitted unsupported roles: "
                f"{'|'.join(unsupported_roles)}")

        # Session ID
        session_id = DeviceUnreal.csettings["stage_session_id"].get_value()
        if session_id > 0:
            args.append(f'-StageSessionId={session_id}')

        # Stage friendly name
        args.append(f'-StageFriendlyName="{self.name.replace(" ", "_")}"')

        # Max GPU Count (mGPU)
        max_gpu_count = DeviceUnreal.csettings["max_gpu_count"].get_value(self.name)
        try:
            if int(max_gpu_count) > 1:
                args.append(f'-MaxGPUCount={max_gpu_count}')
                dp_cvars.append('r.AllowMultiGPUInEditor=1')
        except ValueError:
            LOGGER.warning(f"Invalid Number of GPUs '{max_gpu_count}'")

        # Slate.bAllowThrottling - Makes ICVFX panel and Vcam more responsive
        slate_allow_throttling = DeviceUnreal.csettings["slate_allow_throttling"].get_value()
        dp_cvars.append(f'Slate.bAllowThrottling={int(slate_allow_throttling)}')

        # Holdout composite on Editors if this is a Switchboard config with nDisplay devices
        if DeviceUnreal.csettings["force_holdout_if_ndisplay"].get_value(self.name):
            if any(device.device_type == "nDisplay" for device in self.active_unreal_devices):
                dp_cvars.extend([
                    'r.PostProcessing.PropagateAlpha=1',
                    'r.Deferred.SupportPrimitiveAlphaHoldout=1'
                ])

        # Add user-set dp cvars, overriding any of the forced ones
        user_dp_cvars = self.conform_dpcvars(DeviceUnreal.csettings["dp_cvars"].get_value(self.name))
        dp_cvars = self.add_or_override_cvars(dp_cvars, user_dp_cvars)

        # Add accumulated DPCVars to args
        if len(dp_cvars):
            args.append(f'-DPCVars="{",".join(dp_cvars)}"')

        # Package recovery
        if DeviceUnreal.csettings['auto_decline_package_recovery'].get_value(self.name):
            args.append('-AutoDeclinePackageRecovery')

        # UdpMessaging endpoints
        args.extend(self.build_udpmessaging_args())

        # Disable ensures
        if DeviceUnreal.csettings['disable_ensures'].get_value(self.name):
            args.append('-handleensurepercent=0')

        # Concert take recorder settings
        record_on_client = 'True' if self.is_recording_device else 'False'
        ini_engine = (
            '-ini:Engine:'
            '[/Script/ConcertTakeRecorder.ConcertSessionRecordSettings]:'
            f'LocalSettings=(bRecordOnClient={record_on_client})'
        )
        args.append(ini_engine)

        # VP OSC server settings
        args.extend([
            f'-VPOSCServerAddress={self.address}',
            f'-VPOSCServerPort={self.device_osc_port}'
        ])

        # Disable auto-save.
        args.append(
            "-ini:EditorPerProjectUserSettings:"
            "[/Script/UnrealEd.EditorLoadingSavingSettings]:"
            "bAutoSaveEnable=False"
        )

        # Logging categories
        logging_args = DeviceUnreal.csettings['logging'].get_command_line_arg(
            override_device_name=self.name)
        if logging_args:
            args.append(logging_args)

        # Join all arguments with spaces
        return ' '.join(args)

    def generate_unreal_command_line(self, map_name):
        return (
            self.generate_unreal_exe_path(),
            self.generate_unreal_command_line_args(map_name))

    def fill_derived_data_cache(self, current_level_only=False, program_name=UnrealJobs.BuildDDC.value):
        platforms: list[str] = self.setting_ddc_build_platforms.get_value(self.name)
        if (len(platforms) == 0):
            LOGGER.error("No platforms selected for fill_derived_data_cache, canceling...")
            return 0

        separator = '+'
        platform_string = ''
        for platform in platforms:
            platform_string += platform + separator
        platform_string = platform_string.removesuffix(separator)

        current_level = CONFIG.CURRENT_LEVEL
        current_level_valid = current_level_only and current_level != None and current_level != DEFAULT_MAP_TEXT
        map_name = ' Map=' + current_level if current_level_valid else ''

        args = "\"" + CONFIG.UPROJECT_PATH.get_value(self.name) + map_name + "\"" + " -run=DerivedDataCache -TargetPlatform=" + platform_string + " -fill -DDC=CreateInstalledProjectPak"
        LOGGER.info('Filling ddc with arguments: ' + args)

        puuid, msg = message_protocol.create_start_process_message(
            prog_path=self.generate_unreal_exe_path(),
            prog_args=args,
            prog_name=program_name,
            caller=self.name,
            update_clients_with_stdout=True,
            priority_modifier=sb_utils.PriorityModifier.Normal.value,
            lock_gpu_clock=False,
        )

        self.program_start_queue.add(
            ProgramStartQueueItem(
                name=program_name,
                puuid_dependency=None,
                puuid=puuid,
                msg_to_unreal_client=msg,
            ),
            unreal_client=self.unreal_client,
        )

        return puuid

    def launch(self, map_name, program_name=UnrealJobs.Unreal.value):
        if map_name == DEFAULT_MAP_TEXT:
            map_name = ''

        if not self.check_settings_valid():
            LOGGER.error(f"{self.name}: Not launching due to invalid settings")
            self.widget._close()
            return

        # Launch the MU server
        if CONFIG.MUSERVER_AUTO_JOIN.get_value() \
            and self.autojoin_mu_server.get_value() \
            and CONFIG.MUSERVER_AUTO_LAUNCH.get_value():
            DeviceUnreal.mu_server.launch()

        self.compute_runtime_str()
        engine_path, args = self.generate_unreal_command_line(map_name)
        LOGGER.info(f"Launching Unreal: {engine_path} {args}")

        priority_modifier_str = self.csettings['priority_modifier'].get_value(
            self.name)
        try:
            priority_modifier = sb_utils.PriorityModifier[
                priority_modifier_str].value
        except KeyError:
            LOGGER.warning(
                f"Invalid priority_modifier '{priority_modifier_str}', "
                "defaulting to Normal")
            priority_modifier = sb_utils.PriorityModifier.Normal.value

        # TODO: Sanitize these on Qt input? Deserialization?
        args = args.replace('\r', ' ').replace('\n', ' ')

        self.last_launch_command.update_value(f'{engine_path} {args}')

        lock_gpu_clock_csetting = self.__class__.csettings.get('lock_gpu_clock', None)
        lock_gpu_clock = lock_gpu_clock_csetting.get_value() if lock_gpu_clock_csetting else False
        
        puuid, msg = message_protocol.create_start_process_message(
            prog_path=engine_path,
            prog_args=args,
            prog_name=program_name,
            caller=self.name,
            update_clients_with_stdout=False,
            priority_modifier=priority_modifier,
            lock_gpu_clock=lock_gpu_clock,
        )

        self.program_start_queue.add(
            ProgramStartQueueItem(
                name=program_name,
                puuid_dependency=None,
                puuid=puuid,
                msg_to_unreal_client=msg,
            ),
            unreal_client=self.unreal_client,
        )

    def _try_connect_osc(self):
        if self.status == DeviceStatus.OPEN:
            self.send_osc_message(
                osc.OSC_ADD_SEND_TARGET,
                [SETTINGS.ADDRESS.get_value(), CONFIG.OSC_SERVER_PORT.get_value()])
        else:
            self.osc_connection_timer.stop()

    def on_listener_disconnect(self, unexpected=False, exception=None):
        self.program_start_queue.reset()

        if unexpected:
            self.device_qt_handler.signal_device_client_disconnected.emit(self)

    def do_program_running_update(self, prog):
        if prog.name == UnrealJobs.Unreal.value:

            self.status = DeviceStatus.OPEN
            self.unreal_started_signal.emit()

        elif prog.name.startswith(UnrealJobs.BuildPrefix.value):

            for device in self.devices_sharing_workspace():
                device.status = DeviceStatus.BUILDING

        elif prog.name == UnrealJobs.Sync.value:

            for device in self.devices_sharing_workspace():
                device.status = DeviceStatus.SYNCING

                # Reset the progress.
                device.device_qt_handler.signal_device_sync_update.emit(device, "")

        elif prog.name == UnrealJobs.PackageGame.value:
            self.status = DeviceStatus.PACKAGING

        self.program_start_queue.update_running_program(prog=prog)

    def on_program_started(self, message):
        ''' Handler of the "start program" command '''
        # check if the operation failed
        if not message['bAck']:
            # When we send the start program command, the listener will use
            # the message uuid as program uuid.
            puuid = uuid.UUID(message['puuid'])

            # Tell the queue that the program ended (even before it started)
            program_name = self.program_start_queue.on_program_ended(
                puuid=puuid, unreal_client=self.unreal_client)

            # log this
            LOGGER.error(f"Could not start {program_name}: {message['error']}")

            if program_name == UnrealJobs.Sync.value or program_name.startswith(UnrealJobs.BuildPrefix.value):
                # Sync/Build failures may affect multiple devices
                for device in self.devices_sharing_workspace():
                    device.status = DeviceStatus.CLOSED

                    # This has the effect of hiding building/syncing status.
                    device.project_changelist = device.project_changelist
            else:
                # Any other program failure sets this device to CLOSED
                self.status = DeviceStatus.CLOSED

            return

        # grab the process data
        try:
            process = message['process']
        except KeyError:
            LOGGER.warning(
                f"{self.name} Received 'on_program_started' "
                "but no 'process' in the message")
            return

        if process['caller'] != self.name:
            return

        prog = ProgramStartQueueItem.from_listener_process(process)

        LOGGER.info(
            f"{self.name}: {prog.name} with id {prog.puuid} was "
            "successfully started")

        # Tell the queue that the program started
        self.program_start_queue.on_program_started(prog=prog)

        # Perform any necessary updates to the device
        self.do_program_running_update(prog=prog)

        self._update_widget_classes()

    def on_unreal_started(self):
        if self.is_recording_device:
            sleep_time_in_ms = 1000
            self.osc_connection_timer.start(sleep_time_in_ms)

    def on_program_ended(self, message):
        try:
            process = message['process']
        except KeyError:
            LOGGER.warning(
                "Received 'on_program_ended' but no 'process' in the message")
            return

        if process['caller'] != self.name:
            return

        puuid = uuid.UUID(process['uuid'])
        returncode = message['returncode']

        def get_stdout_str() -> str:
            b64bytes = base64.b64decode(message['stdoutB64'])
            return b64bytes.decode()

        def get_stderr_str() -> str:
            b64bytes = base64.b64decode(message['stderrB64'])
            return b64bytes.decode()

        LOGGER.info(
            f"{self.name}: Program with id {puuid} exited with "
            f"returncode {returncode}")

        try:
            program_name = self.program_start_queue.on_program_ended(
                puuid=puuid, unreal_client=self.unreal_client)
        except KeyError:
            LOGGER.error(
                f"{self.name}: on_program_ended with unknown id {puuid}")
            return

        # Clean up log file tracking for this process if it's an Unreal process
        if program_name == UnrealJobs.Unreal.value:
            process_uuid_str = str(puuid)
            if process_uuid_str in self.process_log_files:
                LOGGER.info(f"Removing log file tracking for ended Unreal process {process_uuid_str}")
                del self.process_log_files[process_uuid_str]

        self.do_program_ended_update(
            program_name=program_name,
            returncode=returncode,
            get_stdout_str=get_stdout_str,
            get_stderr_str=get_stderr_str,
        )

    def do_program_ended_update(
        self,
        *,
        program_name: str,
        returncode: int,
        get_stdout_str: Callable[[], str],
        get_stderr_str: Callable[[], str],
    ):
        # Update the widget classes (e.g. open button icon) once the
        # program_start_queue has been updated, since the status of the
        # queue may affect the widget classes.
        self._update_widget_classes()

        # Check if there are remaining programs named the same but with
        # different ids, which is not normal.
        remaining_homonyms = self.program_start_queue.running_puuids_named(
            program_name)
        for prog_id in remaining_homonyms:
            if program_name != UnrealJobs.Retrieve.value:
                LOGGER.warning(
                    f'{self.name}: But ({prog_id}) with the same name '
                    f'"{program_name}" is still in the list, which is unusual')

        if program_name == UnrealJobs.Unreal.value and not len(remaining_homonyms):

            if DeviceUnreal.csettings["retrieve_logs"].get_value():
                log_success = self.start_retrieve_log(unreal_exit_code=returncode)
                utrace_success = self.start_retrieve_utrace(unreal_exit_code=returncode)
                if not log_success and not utrace_success:
                    self.status = DeviceStatus.CLOSED
            else:
                self.status = DeviceStatus.CLOSED

        elif program_name == UnrealJobs.Retrieve.value and not self.transfer_in_progress:
            self.status = DeviceStatus.CLOSED

        elif program_name == UnrealJobs.Sync.value:
            if returncode == 0:
                LOGGER.info(f"{self.name}: Sync successful")
                self.last_sync_filter_hash.update_value(
                    self.inflight_sync_hash)

                for device in self.devices_sharing_workspace():
                    device.device_qt_handler.signal_device_sync_update.emit(
                        device, "Done")

            else:
                LOGGER.error(f"{self.name}: Sync failed!")
                for line in get_stdout_str().splitlines():
                    LOGGER.error(f"{self.name}: {line}")

                # flag the inflight cl as invalid
                self.inflight_engine_cl = None
                self.inflight_project_cl = None

                # notify of the failure
                self.device_qt_handler.signal_device_sync_failed.emit(self)

            # If you build and sync the engine, update its CL
            if CONFIG.ENGINE_SYNC_METHOD.get_value() == EngineSyncMethod.Build_Engine.value:
                self._request_version_files()
                if self.inflight_engine_cl:
                    self.engine_changelist = self.inflight_engine_cl
                    self.inflight_engine_cl = None

            # Update CL with the one in flight
            if self.inflight_project_cl:
                self.project_changelist = self.inflight_project_cl
                self.inflight_project_cl = None

            self.inflight_sync_hash = None

            self.status = DeviceStatus.CLOSED

            for other in self.devices_sharing_workspace(skip=self):
                sync_method = CONFIG.ENGINE_SYNC_METHOD.get_value()
                if sync_method == EngineSyncMethod.Build_Engine.value:
                    other.engine_changelist = self.engine_changelist

                other.project_changelist = self.project_changelist
                other.status = DeviceStatus.CLOSED

        elif program_name.startswith(UnrealJobs.BuildPrefix.value):
            if returncode == 0:
                LOGGER.info(f"{self.name}: {program_name} successful!")
            else:
                LOGGER.error(f"{self.name}: {program_name} failed!")

                # MSVC build tools error codes, e.g. 'error C4430' or
                # 'error LNK1104'.
                error_pattern = re.compile(r"error [A-Z]{1,3}[0-9]{4}")
                for line in get_stdout_str().splitlines():
                    if error_pattern.search(line):
                        LOGGER.error(f"{self.name}: {line}")

            remaining_build_names = list(filter(lambda name: name.startswith(UnrealJobs.BuildPrefix.value),
                                                self.program_start_queue.rd_puuid_by_name.keys()))

            if not remaining_build_names:
                for device in self.devices_sharing_workspace():
                    device.status = DeviceStatus.CLOSED

                # Forces an update to the changelist field (to hide the
                # Building state).
                self._request_syncstatus()
                if CONFIG.ENGINE_SYNC_METHOD.get_value() == EngineSyncMethod.Build_Engine.value:
                    self._request_version_files()

        elif program_name == UnrealJobs.SyncStatus.value:
            output = get_stdout_str()

            try:
                syncstatus: dict = json.loads(output)
            except Exception:
                stderr = ''
                try:
                    stderr = get_stderr_str()
                except KeyError:
                    pass

                LOGGER.error(f'{self.name}: Error parsing syncstatus output: {output}')

                if stderr:
                    LOGGER.error(f'{self.name}: stderr: {stderr}')

                LOGGER.error(f'{self.name}: Could not retrieve sync status. '
                             'Are the source control settings correctly configured?')

                for device in self.devices_sharing_workspace():
                    if CONFIG.ENGINE_SYNC_METHOD.get_value() == EngineSyncMethod.Build_Engine.value:
                        device.engine_changelist = '(unknown)'
                    device.project_changelist = '(unknown)'

                return

            for device in self.devices_sharing_workspace():
                dirty_str = '*' if syncstatus.get('dirty') else ''
                if engine_cl := syncstatus.get('engine_cl'):
                    device.engine_changelist = f'{engine_cl}{dirty_str}'
                if project_cl := syncstatus.get('project_cl'):
                    device.project_changelist = f'{project_cl}{dirty_str}'

        elif program_name == UnrealJobs.PackageGame.value:
            self.status = DeviceStatus.CLOSED
            percentstr = '100%'
            if returncode != 0:
                percentstr = 'fail'
            self.widget.update_package_status(device=self, step='', percentstr=percentstr)

    def on_program_killed(self, message):
        '''
        Handler of killed program. Expect on_program_ended for anything other
        than a fail.
        '''
        self._update_widget_classes()

        if not message['bAck']:
            # remove from list of puuids (if it exists)
            puuid = uuid.UUID(message['puuid'])
            self.program_start_queue.on_program_ended(
                puuid=puuid, unreal_client=self.unreal_client)

            LOGGER.error(
                f"{self.name} Unable to close program with id {str(puuid)}. "
                f"Error was '{message['error']}''")

    def on_file_received(self, source_path, content):
        if source_path.endswith('.version') or source_path.endswith('.target'):
            self._on_receive_version_file(source_path, content)
            return

        roles_filename = DeviceUnreal.csettings["roles_filename"].get_value(self.name)
        if roles_filename and source_path.endswith(roles_filename):
            self._on_receive_roles(content)
            return

    def _on_receive_roles(self, content):
        decoded_content = base64.b64decode(content).decode()
        tags = parse_unreal_tag_file(decoded_content.splitlines())
        self.setting_roles.possible_values = tags
        LOGGER.info(f"{self.name}: All possible roles: {tags}")
        unsupported_roles = [
            role for role in self.setting_roles.get_value()
            if role not in tags]
        if len(unsupported_roles) > 0:
            LOGGER.error(
                f"{self.name}: Found unsupported roles: "
                f"{unsupported_roles}")
            LOGGER.error(
                f"{self.name}: Please change the roles for this device in "
                "the settings or in the unreal project settings!")

    def _on_receive_version_file(self, source_path: str, content):
        '''
        Handles .version/.target files (see _request_version_files)
        '''
        decoded_content = base64.b64decode(content).decode()
        data = json.loads(decoded_content)

        if source_path.endswith('.target'):
            data = data.get('Version')

        changelist = data.get('Changelist')
        compatible_cl = data.get('CompatibleChangelist')

        for device in self.devices_sharing_workspace():
            if source_path.endswith('Build.version'):
                device._synced_changelist = changelist
                device._synced_compatible_cl = compatible_cl

                # Warn the user if the remote workspace points to a different branch
                sb_buildver = ugs_utils.get_sb_buildver()
                local_branch = sb_buildver.get('BranchName')
                remote_branch = data.get('BranchName')
                if local_branch is None or local_branch != remote_branch:
                    LOGGER.warn(f'{self.name}: Workspace BranchName differs! ({remote_branch})')

            elif source_path.endswith('UnrealEditor.version'):
                device._built_eng_changelist = changelist
                device._built_eng_compatible_cl = compatible_cl
            elif source_path.endswith('.target'):
                device._built_proj_changelist = changelist
                device._built_proj_compatible_cl = compatible_cl

        self.device_qt_handler.signal_device_built_changelist_changed.emit(self)

    def on_file_receive_failed(self, source_path, error):
        roles_filename = DeviceUnreal.csettings["roles_filename"].get_value(self.name)
        if roles_filename and source_path.endswith(roles_filename):
            roles = self.setting_roles.get_value()
            if len(roles) > 0:
                LOGGER.error(
                    f"{self.name}: Error receiving role file from listener and "
                    f"device claims to have these roles: {' | '.join(roles)}")

        LOGGER.error(f"{self.name}: Error retrieving file {source_path}: {error}")

    def on_listener_programstdout(self, message):
        ''' Handles updates to stdout of programs
        Particularly useful to update build progress
        '''
        process = message['process']

        if process['caller'] != self.name:
            return

        stdoutbytes = base64.b64decode(message['partialStdoutB64'])
        stdout = stdoutbytes.decode()
        lines = list(filter(None, stdout.splitlines()))

        already_logged = False

        # see if this is an update to the build
        #
        # example lines:
        #
        #    @progress push 5%
        #    @progress 'Generating code...' 0%
        #    @progress 'Generating code...' 67%
        #    @progress 'Generating code...' 100%
        #    @progress pop
        #
        if process['name'].startswith(UnrealJobs.BuildPrefix.value):
            for line in lines:
                if '@progress' not in line:
                    continue

                stepparts = line.split("'")

                if len(stepparts) < 2:
                    break

                step = stepparts[-2].strip()
                percent = line.split(' ')[-1].strip()

                if percent[-1] != '%':
                    continue

                for device in self.devices_sharing_workspace():
                    device.device_qt_handler.signal_device_build_update.emit(
                            device, step, percent)

        elif process['name'] == UnrealJobs.Sync.value:
            for line in lines:
                already_logged = True

                if 'ERROR' in line:
                    log_fn = LOGGER.error
                elif 'WARNING' in line:
                    log_fn = LOGGER.warning
                else:
                    log_fn = LOGGER.debug

                log_fn(f"{self.name} {process['name']}: {line}")

                if 'Progress:' not in line:
                    continue

                match = re.search(r'Progress: (\d{1,3}\.\d\d%)', line)
                if not match:
                    continue

                sync_progress = match.group(1)
                for device in self.devices_sharing_workspace():
                    device.device_qt_handler.signal_device_sync_update.emit(
                        device, sync_progress)

        elif process['name'] == UnrealJobs.PackageGame.value:
            step = ''
            sync_progress = ''
            self.device_qt_handler.signal_device_package_update.emit(
                        self, step, sync_progress)

        if not already_logged:
            for line in lines:
                LOGGER.debug(f"{self.name} {process['name']}: {line}")

    @classmethod
    def devices(
        cls, *,
        where: Callable[[DeviceUnreal], bool] = lambda _: True,
        only_connected: bool = True,
        skip: Optional[DeviceUnreal] = None,
    ) -> Generator[DeviceUnreal, None, None]:
        for other in cls.active_unreal_devices:
            if other == skip:
                continue

            if only_connected and other.is_disconnected:
                continue

            if where(other):
                yield other

    def devices_sharing_workspace(
        self, *,
        only_connected: bool = True,
        skip: Optional[DeviceUnreal] = None,
    ) -> Generator[DeviceUnreal, None, None]:

        def get_ws(device):
            return (CONFIG.SOURCE_CONTROL_WORKSPACE.get_value(device.name) or '').casefold()

        def get_prj_ws(device):
            return (CONFIG.PROJECT_WORKSPACE.get_value(device.name) or '').casefold()

        self_ws = get_ws(self)
        self_prj_ws = get_prj_ws(self)

        def predicate(device: DeviceUnreal):
            if self_ws or self_prj_ws:
                same_ws = (get_ws(device) == self_ws) if self_ws else False
                same_prj_ws = (get_prj_ws(device) == self_prj_ws) if self_prj_ws else False
                return same_ws or same_prj_ws
            else:
                return device is self

        return DeviceUnreal.devices(
            where=predicate,
            only_connected=only_connected,
            skip=skip,
        )

    def on_listener_state(self, message):
        '''
        Message expected to be received upon connection with the listener.

        It contains the state of the listener. Particularly useful when
        Switchboard reconnects.
        '''
        self.program_start_queue.clear_running_programs()

        server_version = version_helpers.listener_ver_from_state_message(
            message)
        if not server_version:
            LOGGER.error('Unable to parse listener version. Disconnecting...')
            self.disconnect_listener()
            return

        redeploy_version = self.listener_watcher.listener_ver

        # If incompatible or newer patch release available, prompt to redeploy.
        if not version_helpers.listener_is_compatible(server_version):
            self._queue_notify_redeploy()
        elif (redeploy_version is not None and
                server_version < redeploy_version):
            self._queue_notify_redeploy()

        if not version_helpers.listener_is_compatible(server_version):
            server_version_str = version_helpers.version_str(server_version)
            compat_version_str = version_helpers.version_str(
                version_helpers.LISTENER_COMPATIBLE_VERSION)
            LOGGER.error(
                f"{self.name}: Listener version {server_version_str} not "
                f"compatible ({compat_version_str}.x required)")
            self.disconnect_listener()
            return

        self.os_version_label = message.get('osVersionLabel', '')
        self.os_version_label_sub = message.get('osVersionLabelSub', '')
        self.os_version_number = message.get('osVersionNumber', '')
        self.processor_smt = message.get('bProcessorSMT', False)
        self.total_phys_mem = message.get('totalPhysicalMemory', 0)
        self.platform_binary_directory = message.get('platformBinaryDirectory', '')

        # update list of running processes
        for process in message['runningProcesses']:
            prog = ProgramStartQueueItem.from_listener_process(process)

            if process['caller'] == self.name:
                LOGGER.warning(
                    f"{self.name} already running {prog.name} {prog.puuid}")
                self.do_program_running_update(prog=prog)

        # override listener "inactive" timeout
        _, msg = message_protocol.create_set_inactive_timeout_message(
            DeviceUnreal.csettings['listener_inactive_timeout'].get_value()
        )
        self.unreal_client.send_message(msg)

        # request roles and changelists
        self._request_roles_file()
        self._request_syncstatus()

        if CONFIG.ENGINE_SYNC_METHOD.get_value() == EngineSyncMethod.Build_Engine.value:
            self._request_version_files()

    def transport_paths(self, device_recording):
        '''
        Do not transport UE4 paths as they will be checked into source control.
        '''
        return []

    @property
    def log_filename(self):
        return f'{self.name}.log'

    def compute_runtime_str(self):
        """
        Insight tracing requires a unique file name.  We use the current time
        from the start of the build.
        """
        now = datetime.now()
        self.runtime_str = now.strftime('%Y.%m.%d-%H.%M.%S')

    @classmethod
    def get_log_download_dir(cls) -> Optional[pathlib.Path]:
        log_download_dir_setting = \
            DeviceUnreal.csettings['log_download_dir'].get_value().strip()

        if log_download_dir_setting:
            log_download_dir = pathlib.Path(log_download_dir_setting)
            if log_download_dir.is_dir() and log_download_dir.is_absolute():
                return log_download_dir
            else:
                LOGGER.error(
                    f'Invalid log download dir: {log_download_dir_setting}')
        else:
            local_project_dir = pathlib.Path(
                CONFIG.UPROJECT_PATH.get_value()).parent
            try:
                if local_project_dir.is_dir():
                    log_download_dir = \
                        local_project_dir / 'Saved' / 'Logs' / 'Switchboard'
                    log_download_dir.mkdir(parents=True, exist_ok=True)
                    return log_download_dir
            except OSError as e:
                # This can happen if the drive is locked (e.g. by Bitlocker)
                LOGGER.error(e)

        return None

    @QtCore.Slot()
    def reregister_rsync_client(self):
        DeviceUnreal.rsync_server.unregister_client(self)
        DeviceUnreal.rsync_server.register_client(
            self, self.name, self.address)

    def backup_file(self, filename):
        ''' Rotate existing filename to a timestamped backup, a la Unreal. '''

        try:
            src = self.make_local_filepath_to_fetch(filename)
        except:
            return

        if not src.is_file():
            return

        modtime = datetime.fromtimestamp(src.stat().st_mtime)
        modtime_str = modtime.strftime('%Y.%m.%d-%H.%M.%S')
        dest_filename = f'{src.stem}-backup-{modtime_str}{src.suffix}'

        src.rename(self.make_local_filepath_to_fetch(dest_filename))

    def get_remote_log_path(self):
        remote_project_path = \
            pathlib.Path(CONFIG.UPROJECT_PATH.get_value(self.name)).parent
        return remote_project_path / 'Saved' / 'Logs'

    def get_rsync_path(self):
        if self._remote_path_uses_windows_separators():
            return pathlib.Path(
                CONFIG.ENGINE_DIR.get_value(self.name),
                'Extras', 'ThirdPartyNotUE', 'cwrsync', 'bin', 'rsync.exe')
        return 'rsync'

    def fetch_file(self, remote_path, dest_filename = ""):
        """
        Copies a remote file to the local machine. Optionally renames
        the file if a new name is provided.
        """

        program_name = UnrealJobs.Retrieve.value

        rsync_path = self.get_rsync_path()

        remote_cygpath = \
            DeviceUnreal.rsync_server.make_cygdrive_path(remote_path)

        dest_endpoint = \
            f'{SETTINGS.ADDRESS.get_value()}:{DeviceUnreal.rsync_server.port}'
        dest_module = DeviceUnreal.rsync_server.INCOMING_LOGS_MODULE
        dest_path = f'rsync://{dest_endpoint}/{dest_module}/{dest_filename}'

        rsync_args = f'"{remote_cygpath}" "{dest_path}"'

        puuid, msg = message_protocol.create_start_process_message(
            prog_path=str(rsync_path),
            prog_args=rsync_args,
            prog_name=program_name,
            caller=self.name,
            hide=True,
        )

        self.program_start_queue.add(
            ProgramStartQueueItem(
                name=program_name,
                puuid_dependency=None,
                puuid=puuid,
                msg_to_unreal_client=msg,
            ),
            unreal_client=self.unreal_client,
        )
        # TODO: sync crash log?
        return True

    def make_local_filepath_to_fetch(self, filename):
        """
        Makes the destination path of a file to be fetched
        """

        log_download_dir = self.get_log_download_dir()

        if not log_download_dir:
            raise NotADirectoryError

        return log_download_dir / filename

    def start_retrieve_utrace(self, unreal_exit_code:int ):
        """
        Retrieve the utrace file if tracing is enabled.
        """
        if not CONFIG.INSIGHTS_TRACE_ENABLE.get_value():
            return False

        remote_path = self.get_utrace_filepath()

        filename = os.path.basename(remote_path)
        self.backup_file(filename)

        # remember the path to the trace to be fetched, so that it can later be opened from the device's context menu
        try:
            self.last_trace_path.update_value(str(self.make_local_filepath_to_fetch(filename)))
        except:
            pass

        return self.fetch_file(remote_path)

    def start_retrieve_log(self, unreal_exit_code: int):
        """
        Retrieve the log file if logging is enabled.
        """
        self.backup_file(self.log_filename)

        # remember the path to the log file to be fetched, so that it can later be opened from the device's context menu
        try:
            self.last_log_path.update_value(str(self.make_local_filepath_to_fetch(self.log_filename)))
        except:
            pass

        remote_log_path = self.get_remote_log_path() / self.log_filename
        return self.fetch_file(remote_log_path)

    @property
    def transfer_in_progress(self) -> bool:
        return len(
            self.program_start_queue.running_puuids_named(UnrealJobs.Retrieve.value)) > 0

    def get_widget_classes(self):
        widget_classes = list(self._widget_classes)

        widget_classes.append(f'status_{self.status.name.lower()}')

        if self.transfer_in_progress:
            widget_classes.append('download')

        return widget_classes

    @QtCore.Slot()
    def _update_widget_classes(self):
        # Ensure this code is run from the main thread
        if threading.current_thread() is not threading.main_thread():
            QtCore.QMetaObject.invokeMethod(
                self, '_update_widget_classes', QtCore.Qt.QueuedConnection)
            return

        classes = self.get_widget_classes()
        sb_widgets.set_qt_property(self.widget, 'widget_classes', classes,
                                   update_box_model=False,
                                   recursive_refresh=True)

    def add_widget_class(self, widget_class: str):
        if widget_class not in self._widget_classes:
            self._widget_classes.add(widget_class)
            self._update_widget_classes()

    def remove_widget_class(self, widget_class: str):
        if widget_class in self._widget_classes:
            self._widget_classes.remove(widget_class)
            self._update_widget_classes()

    def update_settings_menu_state(self):
        autojoin = self.widget.autojoin_mu
        autojoin.set_autojoin_mu(self.autojoin_mu_server.get_value())

    def on_autojoin_mu_ui_change(self):
        autojoin = self.widget.autojoin_mu
        self.autojoin_mu_server.update_value(autojoin.is_autojoin_enabled())
        self.update_settings_menu_state()

    def on_toggle_exclude_from_build(self):
        self.exclude_from_build.update_value(
            not self.exclude_from_build.get_value())
        CONFIG.save()

    def on_toggle_exclude_from_insights(self):
        self.exclude_from_insights.update_value(
            not self.exclude_from_insights.get_value())
        CONFIG.save()

    def on_open_last_log(self):
        ''' Opens the last log in your preferred editor '''
        path_str = self.last_log_path.get_value()
        if path_str and pathlib.Path(path_str).is_file():
            url = QtCore.QUrl.fromLocalFile(path_str)
            QtGui.QDesktopServices.openUrl(url)

    def on_open_last_trace(self):
        ''' Opens the last unreal insights trace '''

        tracepath = self.last_trace_path.get_value()

        if not pathlib.Path(tracepath).is_file():
            LOGGER.error(f"Could not find '{tracepath}'")
            return

        insights = InsightsLauncher()
        insights.launch(args=[f'"{str(tracepath)}"'], allow_duplicate=True)

    def on_start_live_log(self):
        ''' Starts live log streaming from the remote device. 
        
        This ends up being called from the device context menu 
        when the user wants to view the live log
        '''

        # Check if another device is already streaming the live log

        if DeviceUnreal.active_live_log_device and DeviceUnreal.active_live_log_device != self:
            LOGGER.info(f"Stopping live log streaming on {DeviceUnreal.active_live_log_device.name} to start on {self.name}")
            DeviceUnreal.active_live_log_device._stop_live_log_internal()

        # If already streaming, see if we need to launch an instance of the log viewer.

        if self.live_log_streaming and self.live_log_local_file:
            if self.live_log_local_file.exists():

                # Check if we have a running process for this device
                existing_process = log_viewer_watcher.get_process_for_device(self.name)

                if existing_process and existing_process.poll() is None:
                    LOGGER.info(f"Log viewer for '{self.name}' is already running.")
                else:
                    LOGGER.info(f"Launching live log viewer for '{self.name}'.")
                    self._launch_log_viewer()
                return
            else:
                # Local file was unexpectedly deleted, so we stop it because we need to get the stream from the beginning.
                self._stop_live_log_internal()

        if not self.unreal_client.is_connected:
            LOGGER.error(f"Cannot start live log because device {self.name} is not connected")
            return

        if self.status < DeviceStatus.OPEN:
            LOGGER.error(f"Cannot start live log because Unreal is not running on device {self.name}")
            return

        # Get the UUID of the running Unreal process (if any)

        unreal_process_uuid = None
        unreal_puuids = self.program_start_queue.running_puuids_named(UnrealJobs.Unreal.value)

        if unreal_puuids:
            # Grab the first running Unreal, although we expect there to only be up to one per device.
            unreal_process_uuid = str(unreal_puuids[0])
        else:
            LOGGER.error("Live log: no running Unreal processes found, can't start the live log")
            return

        # Construct the remote log file path
        remote_log_path = self.get_remote_log_path() / self.log_filename
        LOGGER.info(f"Live log: requesting streaming for remote path: {remote_log_path}")

        # Check if we have an existing log file for this process UUID
        start_offset = None
        if unreal_process_uuid in self.process_log_files:
            existing_log = self.process_log_files[unreal_process_uuid]
            if existing_log['local_file'].exists():
                # Reuse the existing log file and resume from where we left off
                self.live_log_local_file = existing_log['local_file']
                start_offset = existing_log['local_file'].stat().st_size
                LOGGER.info(f"Resuming live log from existing file {self.live_log_local_file}, offset: {start_offset} bytes")
            else:
                # File was deleted, remove from tracking
                del self.process_log_files[unreal_process_uuid]
                LOGGER.info(f"Previous log file was deleted, will create new one")

        # Create new local log file if we don't have one
        if self.live_log_local_file is None or not self.live_log_local_file.exists():
            log_dir = DeviceUnreal.get_log_download_dir()
            timestamp = datetime.now().strftime("%Y%m%d_%H%M%S")
            local_filename = f"live_{self.name}_{timestamp}.log"
            self.live_log_local_file = pathlib.Path(log_dir) / local_filename

            # Create the local live log file
            try:
                self.live_log_local_file.parent.mkdir(parents=True, exist_ok=True)
                self.live_log_local_file.touch()
                start_offset = 0  # New file, start from beginning
                LOGGER.info(f"Created new live log file: {self.live_log_local_file}")
            except Exception as exception:
                LOGGER.error(f"Failed to create live log file {self.live_log_local_file}: {exception}")
                return

            # Track this new log file for the process
            self.process_log_files[unreal_process_uuid] = {
                'local_file': self.live_log_local_file,
                'last_position': 0
            }

        # Send start live log message to listener

        msgid, msg = message_protocol.create_start_live_log_stream_message(
            str(remote_log_path), tail_only=False, process_uuid=unreal_process_uuid, start_offset=start_offset)

        try:
            self.unreal_client.send_message(msg)
            self.live_log_streaming = True
            self.live_log_file_path = str(remote_log_path)
            DeviceUnreal.active_live_log_device = self

            LOGGER.info(f"Started live log streaming for {self.name} -> {self.live_log_local_file}")

            # Launch the log viewer (and will also register with its process watcher)
            self._launch_log_viewer()

        except Exception as exception:
            LOGGER.error(f"Failed to start live log streaming: {exception}")
            self.live_log_streaming = False
            self.live_log_file_path = None

    def on_live_log_data(self, message):
        ''' Handles incoming live log data from the listener '''

        if not self.live_log_streaming or not self.live_log_local_file:
            LOGGER.warning(f"Received live log data but streaming not active or no local file: streaming={self.live_log_streaming}, file={self.live_log_local_file}")
            return

        # Update tracked position for this process
        unreal_puuids = self.program_start_queue.running_puuids_named(UnrealJobs.Unreal.value)
        if unreal_puuids:
            process_uuid = str(unreal_puuids[0])
            if process_uuid in self.process_log_files:
                # Track the file position (could also use file_position from message)
                self.process_log_files[process_uuid]['last_position'] = self.live_log_local_file.stat().st_size

        try:
            # Check for compressed content first

            if 'content_compressed_base64' in message:
                compressed_base64 = message['content_compressed_base64']
                compressed_bytes = base64.b64decode(compressed_base64)

                # Decompress
                content_bytes = zlib.decompress(compressed_bytes)

                # Uncomment the following to get an idea of the compression ratios achieved.
                #
                #compression_ratio = len(content_bytes) / len(compressed_bytes) if len(compressed_bytes) > 0 else 1.0
                #LOGGER.info(f"Live log compression: {len(compressed_bytes)} -> {len(content_bytes)} bytes (ratio: {compression_ratio:.2f}x)")

                # write to live log file
                with open(self.live_log_local_file, 'ab') as f:
                    f.write(content_bytes)

            elif 'content_base64' in message:

                # If data is not compressed, check the uncompressed content

                content_base64 = message['content_base64']
                content_bytes = base64.b64decode(content_base64)

                # write to live log file
                with open(self.live_log_local_file, 'ab') as f:
                    f.write(content_bytes)

        except Exception as e:
            LOGGER.error(f"Failed to write live log data to {self.live_log_local_file}: {e}")

    def _launch_log_viewer(self):
        ''' Launch the log viewer with the live log file '''

        # Currently default viewer for this filetype.
        url = QtCore.QUrl.fromLocalFile(str(self.live_log_local_file))
        QtGui.QDesktopServices.openUrl(url)

    def _stop_live_log_internal(self):
        ''' Stops live log streaming '''

        if not self.live_log_streaming:
            return

        # Note: We intentionally do NOT clear process_log_files here
        # so that we can resume if the same process is still running

        if self.live_log_file_path and self.unreal_client.is_connected:

            # Send stop live log message to listener
            msgid, msg = message_protocol.create_stop_live_log_stream_message(
                self.live_log_file_path)

            try:
                self.unreal_client.send_message(msg)
                LOGGER.info(f"Stopped live log streaming for {self.name}")
            except Exception as e:
                LOGGER.error(f"Failed to send stop live log message: {e}")

        # Terminate any running log viewer processes for this device
        log_viewer_watcher.terminate_log_viewers_for_device(self.name)

        # Reset state
        self.live_log_streaming = False
        self.live_log_file_path = None
        self.live_log_local_file = None

        # Clear global active device, if it was this device.
        if DeviceUnreal.active_live_log_device == self:
            DeviceUnreal.active_live_log_device = None

    @QtCore.Slot()
    def _on_log_viewer_closed(self):
        '''
        Called by the log viewer watcer when the log viewer process associated with this device terminates,
        e.g. the user closed it. When this happens, we automatically stop the live streaming since nobody is
        watching. Currently there is not support for log viewers different to the default.
        '''
        LOGGER.info(f"Log viewer closed for device '{self.name}', stopping live log streaming")

        # Stop live log streaming since the viewer is no longer available
        if self.live_log_streaming:
            self._stop_live_log_internal()

        # Note: We keep the process_log_files tracking so we can resume from the same position
        # if the user reopens the log viewer while the process is still running

    def on_copy_last_launch_command(self):
        ''' Copies the last launch command to the clipboard'''
        QtGui.QGuiApplication.clipboard().setText(
            self.last_launch_command.get_value())

    @classmethod
    def all_devices_added(cls):
        ''' Device interface implementation '''

        super().all_devices_added()

        # Trigger a project cache if this is a DeviceUnreal class (not a subclass)
        # and the project was not cached when opened
        if cls == DeviceUnreal:
            if not CONFIG.PROJECTWASINCACHE:
                DeviceUnreal.analyze_project_assets()

    @classmethod
    def analyze_project_assets(cls):
        ''' Traverses project and content plugins and caches a list of assets of interest.
        e.g. nDisplay configs, live link presets, and media profiles.
        '''

        search_paths: list[pathlib.Path] = []
        search_paths.append(pathlib.Path(CONFIG.get_project_content_dir()))
        search_paths.extend([plugin.plugin_content_path
                            for plugin in CONFIG.get_unreal_content_plugins()])

        asset_names: list[str] = []
        asset_paths: list[str] = []
        asset_classnames: list[str] = []

        assets = []

        # show a progress bar if it is taking more a trivial amount of time
        progressDiag = QtWidgets.QProgressDialog(
            'Finding assets...', 'Cancel', 0, 0, parent=None)

        progressDiag.setWindowTitle('Unreal Asset Finder')
        progressDiag.setModal(True)
        progressDiag.setMinimumDuration(1000)  # time before it shows up
        # Looks much better without the window frame.
        progressDiag.setWindowFlag(QtCore.Qt.FramelessWindowHint)

        # create an event object to signal when the function is done
        done_event = threading.Event()

        # convenience find assets worker. This is a step before parsing the assets, and can
        # also take some time depending on the number of assets in the project.
        def find_assets_work():
            for path in search_paths:
                for dirpath, _, file_names in os.walk(path):
                    for file_name in file_names:
                        if progressDiag.wasCanceled():
                            done_event.set()
                            return

                        if not file_name.lower().endswith(('.uasset', '.ndisplay')):
                            continue

                        if file_name not in asset_names:
                            asset_path = os.path.join(dirpath, file_name)
                            ext = os.path.splitext(file_name)[1]

                            # Since .uasset is a generic asset container, only add
                            # assets of the right class.
                            if ext.lower() == '.uasset':
                                assets.append({
                                    'name': file_name,
                                    'path': asset_path,
                                })
                            else:
                                asset_names.append(file_name)
                                asset_paths.append(asset_path)
                                asset_classnames.append(DeviceUnreal.NDISPLAY_CLASS_NAMES[0]) # so that it passes the filter later on

            done_event.set()

        thread = threading.Thread(target=find_assets_work)
        thread.start()

        # wait for the event to be set or the progress dialog to be canceled
        while not done_event.is_set():
            progressDiag.setValue(progressDiag.value() + 1)
            QtWidgets.QApplication.processEvents()
            time.sleep(0.050)  # The worker thread will run faster if we sleep.

        find_canceled = progressDiag.wasCanceled()

        progressDiag.close()

        thread.join()

        if find_canceled:
            return

        # process the assets in a multi-threaded fashion

        # show a progress bar if it is taking more a trivial amount of time
        progressDiag = QtWidgets.QProgressDialog(
            'Parsing assets...', 'Cancel', 0, 0, parent=None)

        progressDiag.setWindowTitle('Unreal Asset Analyzer')
        progressDiag.setModal(True)
        progressDiag.setMinimumDuration(1000)  # time before it shows up
        progressDiag.setRange(0, len(assets))

        # Looks much better without the window frame.
        progressDiag.setWindowFlag(QtCore.Qt.FramelessWindowHint)

        INTERESTING_CLASS_NAMES = \
            DeviceUnreal.NDISPLAY_CLASS_NAMES \
            + DeviceUnreal.LIVELINKPRESET_CLASS_NAMES \
            + DeviceUnreal.MEDIAPROFILE_CLASS_NAMES

        def validateInterestingAsset(asset):
            ''' Returns the asset if it is an interesting asset '''

            with open(asset['path'], 'rb') as file:

                aparser = UassetParser(file, allowUnversioned=True)

                for assetdata in aparser.aregdata:
                    if assetdata.ObjectClassName in INTERESTING_CLASS_NAMES:
                        asset['assetdata'] = assetdata
                        return asset

            raise ValueError

        numThreads = 8
        doneAssetCount = 0

        with concurrent.futures.ThreadPoolExecutor(
                max_workers=numThreads) as executor:

            progressDiag.canceled.connect(lambda: executor.shutdown(cancel_futures=True))

            futures = [
                executor.submit(validateInterestingAsset, asset)
                for asset in assets]

            for future in concurrent.futures.as_completed(futures):
                if progressDiag.wasCanceled():
                    return

                # Update progress bar.
                doneAssetCount += 1
                progressDiag.setValue(doneAssetCount)

                # Get the future result and add to list of config names and
                # paths.
                try:
                    asset = future.result()

                    # first make sure they all exist
                    name = asset['name']
                    path = asset['path']
                    classname = asset['assetdata'].ObjectClassName

                    # now append to lists
                    asset_names.append(name)
                    asset_paths.append(path)
                    asset_classnames.append(classname)

                except Exception:
                    pass

        if progressDiag.wasCanceled():
            return

        # close progress bar window
        progressDiag.close()

        asset_path_to_game_path: dict[str, Optional[PurePosixPath]] = {
            x: CONFIG.resolve_content_path(x) for x in asset_paths
        }

        def disambiguate_asset_path(path: str) -> str:
            if game_path := asset_path_to_game_path[path]:
                file_name = game_path.stem
                rel_path = game_path.parent
            else:
                LOGGER.error(f"Couldn't map game path for asset {path}")
                file_name = os.path.basename(path)
                rel_path = path.removesuffix(file_name)

            if rel_path == "/":
                return f"{file_name}"
            else:
                return f"{file_name} ({rel_path})"

        asset_names, _ = sb_dialog.SwitchboardDialog.disambiguated_base_names(
            asset_paths, disambiguate_asset_path)

        # collect the found config files into the assets list

        assets = []

        project = SBCache().query_or_create_project(CONFIG.UPROJECT_PATH.get_value())

        for idx, asset_name in enumerate(asset_names):
            gamepath = asset_path_to_game_path.get(asset_paths[idx])
            if gamepath is None:
                continue  # We will have logged an error already.

            assettype = SBCache().query_or_create_assettype(asset_classnames[idx])

            asset = Asset(
                id=0,
                project=project,
                assettype=assettype,
                gamepath=str(gamepath),
                name=asset_name,
                localpath=asset_paths[idx])

            assets.append(asset)

        # save to the cache
        SBCache().update_project_assets(project=project, assets=assets)

        # Emit signal to notify UI components that project assets were updated
        cls.static_signals.project_assets_updated_signal.emit()

    @classmethod
    def handle_clipboard_mime_text(cls, text:str):
        ''' Catches pasting of content browser assets.
        The main purpose is to auto-add them to switchboard's caches of assets and levels.

        Expected syntax:
            /Script/Module.Class'Gamepath.Name'
        
        E.g. umap:
            /Script/Engine.World'/Game/Tests/AA/LLF/L_LLF.L_LLF

        When multiple objects are selected, they are listed one per line.
        '''  
        asset_reference_re = re.compile(r"""
            ^/Script/                              # /Script/
            (?P<module>[A-Za-z_]\w+)               # Module
            \.                                     # .
            (?P<classname>[A-Za-z_]\w+)            # Class
            '                                      # '
            (?P<gamepath>[A-Za-z0-9_/\-]+)         # Gamepath (allow -)
            \.                                     # .
            (?P<name>[A-Za-z0-9_\-]+)              # Name (allow -)
            '                                      # '
            $""", re.VERBOSE)

        INTERESTING_CLASS_NAMES = \
            DeviceUnreal.NDISPLAY_CLASS_NAMES \
            + DeviceUnreal.LIVELINKPRESET_CLASS_NAMES \
            + DeviceUnreal.MEDIAPROFILE_CLASS_NAMES
        
        assets = []
        levels = []

        project = SBCache().query_or_create_project(CONFIG.UPROJECT_PATH.get_value())

        # When selecting multiple assets, they are listed on separate lines.
        for line in text.splitlines():

            match = asset_reference_re.match(line)

            if not match:
                LOGGER.warning(f"Invalid asset reference: {line!r}")
                continue
            
            module    = match.group("module")
            classname = match.group("classname")
            gamepath  = match.group("gamepath")
            name      = match.group("name")

            namespaced_classname = f"/Script/{module}.{classname}"

            if namespaced_classname == "/Script/Engine.World":
                levels.append(gamepath)

            elif namespaced_classname in INTERESTING_CLASS_NAMES:
                filepath = CONFIG.ue_plugin_mgr.content_to_file_path(gamepath)

                # The class type implies the file extension.
                uasset_ext = '.uasset'

                file_exists = False

                if filepath:
                    # Append the extension it since content_to_file_path(..) doesn't.
                    filepath = str(filepath) + uasset_ext
                    file_exists = os.path.exists(filepath)
                else:
                    filepath = ''

                if not file_exists:
                    # Don't save invalid filepaths
                    filepath = ''

                    # The only case that requires a valid filepath is nDisplay configs, since they are parsed
                    # during launch. The other asset references are ok with just a valid gamepath.
                    if namespaced_classname in DeviceUnreal.NDISPLAY_CLASS_NAMES:
                        LOGGER.warning(f"Skipping nDisplay config '{gamepath}' because its file on disk could not be located")
                        continue
                    else:
                        LOGGER.warning(f"Could not resolve to an existing filepath for '{gamepath}'")

                assettype = SBCache().query_or_create_assettype(namespaced_classname)
                asset_name = name + uasset_ext

                asset = Asset(
                    id=0,
                    project=project,
                    assettype=assettype,
                    gamepath=str(gamepath),
                    name=asset_name,
                    localpath=filepath
                )
                
                assets.append(asset)

        # Add levels
        if len(levels):

            levels_text = '\n'.join(levels)

            reply = QtWidgets.QMessageBox.question(
                None,
                "Add Unreal Maps",
                f"The following maps will be added:\n\n{levels_text}",
                QtWidgets.QMessageBox.Ok | QtWidgets.QMessageBox.Cancel
            )

            if reply == QtWidgets.QMessageBox.Ok:
                for level in levels:
                    CONFIG.add_single_level(level)
                    cls.static_signals.clipboard_level_added_signal.emit(level)
                    LOGGER.info(f"Added level '{level}'")

        # Add assets
        if len(assets):

            assets_text = '\n'.join([asset.name for asset in assets])

            reply = QtWidgets.QMessageBox.question(
                None,
                "Add Unreal Assets",
                f"The following assets will be added:\n\n{assets_text}",
                QtWidgets.QMessageBox.Ok | QtWidgets.QMessageBox.Cancel
            )

            if reply == QtWidgets.QMessageBox.Ok:
                SBCache().merge_project_assets(project, assets)

                for asset in assets:
                    LOGGER.info(f"Added '{asset.name}'")

                # Emit signal to notify UI components that assets were added
                cls.static_signals.clipboard_assets_added_signal.emit()


def parse_unreal_tag_file(file_content):
    tags = []
    for line in file_content:
        if line.startswith("GameplayTagList"):
            tag = line.split("Tag=")[1]
            tag = tag.split(',', 1)[0]
            tag = tag.strip('"')
            tags.append(tag)
    return tags


BASE_ENGINE_CL_TOOLTIP = "Current Engine Changelist"
BASE_PROJECT_CL_TOOLTIP = "Current Project Changelist"


class DeviceWidgetUnreal(DeviceWidget):

    signal_exclude_from_build_toggled = QtCore.Signal()
    signal_exclude_from_insights_toggled = QtCore.Signal()
    signal_open_last_log = QtCore.Signal(object)
    signal_open_last_trace = QtCore.Signal(object)
    signal_copy_last_launch_command = QtCore.Signal(object)
    signal_device_widget_fill_ddc = QtCore.Signal(object, bool)
    signal_start_live_log = QtCore.Signal(object)

    def __init__(self, name, device_hash, address, icons, parent=None):
        self._autojoin_visible = True
        self._is_engine_synched = True
        self._is_project_synched = True
        self._needs_rebuild = False
        self._exclude_from_build = False
        self._desired_build_button_tooltip = "Build changelist"
        self.exclude_from_insights = False

        super().__init__(name, device_hash, address, icons, parent=parent)

        CONFIG.P4_ENABLED.signal_setting_changed.connect(
            lambda _, enabled: self.sync_button.setVisible(enabled))

    @property
    def app_name(self):
        return 'Unreal'

    @property
    def exclude_from_build(self):
        return self._exclude_from_build

    def update_exclude_from_build(self, exclude_from_build, update_ui=True):
        self._exclude_from_build = exclude_from_build
        if not update_ui:
            return

        if exclude_from_build:
            self.engine_changelist_label.hide()
            self._update_build_button_tooltip()
            sb_widgets.set_qt_property(self.build_button, 'not_built', False)
        else:
            self.engine_changelist_label.show()
            self._refresh_build_info_ui()

        self.build_button.setDisabled(exclude_from_build)
        self._update_engine_cl_label()

    def _add_control_buttons(self):
        super()._add_control_buttons()

        changelist_layout = QtWidgets.QVBoxLayout()
        self.engine_changelist_label = QtWidgets.QLabel()
        self.engine_changelist_label.setObjectName('changelist')
        self.engine_changelist_label.setToolTip(BASE_ENGINE_CL_TOOLTIP)
        self.engine_changelist_label.setTextInteractionFlags(QtCore.Qt.TextSelectableByMouse)
        changelist_layout.addWidget(self.engine_changelist_label)

        self.project_changelist_label = QtWidgets.QLabel()
        self.project_changelist_label.setObjectName('changelist')
        self.project_changelist_label.setToolTip(BASE_PROJECT_CL_TOOLTIP)
        self.project_changelist_label.setTextInteractionFlags(QtCore.Qt.TextSelectableByMouse)
        changelist_layout.addWidget(self.project_changelist_label)

        spacer = QtWidgets.QSpacerItem(
            0, 20, QtWidgets.QSizePolicy.Expanding,
            QtWidgets.QSizePolicy.Minimum)

        self.layout.addLayout(changelist_layout)

        CONTROL_BUTTON_ICON_SIZE = QtCore.QSize(21, 21)

        self.sync_button = self.add_control_button(
            icon_size=CONTROL_BUTTON_ICON_SIZE,
            checkable=False,
            tool_tip='Sync changelist',
            hover_focus=False,
            name='sync')

        self.build_button = self.add_control_button(
            icon_size=CONTROL_BUTTON_ICON_SIZE,
            checkable=False,
            tool_tip='Build changelist',
            hover_focus=False,
            name='build')

        self.layout.addItem(spacer)

        self.autojoin_mu = DeviceAutoJoinMUServerUI("")
        button = self.autojoin_mu.make_button(self)
        button.setVisible(self._autojoin_visible)
        self.layout.setAlignment(button, QtCore.Qt.AlignVCenter)
        self.add_widget_to_layout(button)

        self.assign_button_to_name("autojoin_mu", button)

        self.open_button = self.add_control_button(
            icon_size=CONTROL_BUTTON_ICON_SIZE,
            tool_tip=f'Start {self.app_name}',
            hover_focus=False,
            name='open')

        self.connect_button = self.add_control_button(
            icon_size=CONTROL_BUTTON_ICON_SIZE,
            tool_tip='Connect to listener',
            hover_focus=False,
            name='connect')

        self.sync_button.clicked.connect(self.sync_button_clicked)
        self.build_button.clicked.connect(self.build_button_clicked)
        self.connect_button.clicked.connect(self.connect_button_clicked)
        self.open_button.clicked.connect(self.open_button_clicked)

        # Disable UI when not connected
        self.open_button.setDisabled(True)
        self.sync_button.setDisabled(True)
        self.build_button.setDisabled(True)

        self.project_changelist_label.hide()
        self.engine_changelist_label.hide()
        self.sync_button.hide()

    def can_sync(self):
        return self.sync_button.isEnabled()

    def can_build(self):
        return self.build_button.isEnabled()

    def _open(self):
        # Make sure the button is in the correct state
        self.open_button.setChecked(True)
        # Emit Signal to Switchboard
        self.signal_device_widget_open.emit(self)

    def _close(self):
        # Make sure the button is in the correct state
        self.open_button.setChecked(False)
        # Emit Signal to Switchboard
        self.signal_device_widget_close.emit(self)

    def _connect(self):
        self._update_connected_ui()

        # Emit Signal to Switchboard
        self.signal_device_widget_connect.emit(self)

    def _disconnect(self):
        ''' Called when user disconnects '''
        self._update_disconnected_ui()

        # Emit Signal to Switchboard
        self.signal_device_widget_disconnect.emit(self)

    def _update_disconnected_ui(self):
        ''' Updates the UI of the device to reflect disconnected status '''
        # Make sure the button is in the correct state
        self.connect_button.setChecked(False)
        self.connect_button.setToolTip('Connect to listener')

        # Don't show the changelist
        self.project_changelist_label.hide()
        self.engine_changelist_label.hide()
        self.sync_button.hide()

        # Disable the buttons
        self.open_button.setDisabled(True)
        self.sync_button.setDisabled(True)
        self.build_button.setDisabled(True)

        sb_widgets.set_qt_property(self.sync_button, 'not_synched', False)
        sb_widgets.set_qt_property(self.sync_button, 'not_built', False)
        sb_widgets.set_qt_property(self.build_button, 'not_synched', False)
        sb_widgets.set_qt_property(self.build_button, 'not_built', False)

    def _update_connected_ui(self):
        ''' Updates the UI of the device to reflect connected status. '''
        # Make sure the button is in the correct state
        self.connect_button.setChecked(True)
        self.connect_button.setToolTip('Disconnect from listener')

        self.open_button.setDisabled(False)
        self.sync_button.setDisabled(False)
        self.build_button.setDisabled(self.exclude_from_build)

    def update_status(self, status, previous_status, device=None):
        super().update_status(status, previous_status, device)

        if status <= DeviceStatus.CONNECTING:
            self._update_disconnected_ui()
        else:
            self._update_connected_ui()

        # The connect/disconnect button is enabled in all states except for
        # CONNECTING.
        self.connect_button.setDisabled(False)

        if status == DeviceStatus.CONNECTING:
            self.connect_button.setDisabled(True)
            self.connect_button.setToolTip('Connecting to listener...')
        elif status == DeviceStatus.CLOSED:
            self.open_button.setDisabled(False)
            self.open_button.setChecked(False)
            self.sync_button.setDisabled(False)
            self.build_button.setDisabled(self.exclude_from_build)
            if self.exclude_from_build:
                self.engine_changelist_label.hide()
        elif status == DeviceStatus.CLOSING:
            self.open_button.setDisabled(True)
            self.open_button.setChecked(True)
            self.sync_button.setDisabled(True)
            self.build_button.setDisabled(True)
        elif status == DeviceStatus.SYNCING:
            self.open_button.setDisabled(True)
            self.sync_button.setDisabled(True)
            self.build_button.setDisabled(True)
            self.engine_changelist_label.hide()
            self.project_changelist_label.setText('Syncing...')
        elif status == DeviceStatus.BUILDING:
            self.open_button.setDisabled(True)
            self.sync_button.setDisabled(True)
            self.build_button.setDisabled(True)
            self.engine_changelist_label.hide()
            self.project_changelist_label.setText('Building...')
        elif status == DeviceStatus.OPEN:
            self.open_button.setDisabled(False)
            self.open_button.setChecked(True)
            self.sync_button.setDisabled(True)
            self.build_button.setDisabled(True)
        elif status == DeviceStatus.READY:
            self.open_button.setDisabled(False)
            self.open_button.setChecked(True)
            self.sync_button.setDisabled(True)
            self.build_button.setDisabled(True)
        elif status == DeviceStatus.PACKAGING:
            self.open_button.setDisabled(True)
            self.sync_button.setDisabled(True)
            self.build_button.setDisabled(True)
            self.engine_changelist_label.hide()
            self.project_changelist_label.setText('Packaging...')

        if self.open_button.isChecked():
            if self.open_button.isEnabled():
                self.open_button.setToolTip(f'Stop {self.app_name}')
            else:
                self.open_button.setToolTip(f'Stopping {self.app_name}...')
        else:
            self.open_button.setToolTip(f'Start {self.app_name}')

    def update_project_changelist(self, required_cl: str | None, current_device_cl: str):
        self.project_changelist_label.setText(f'P: {current_device_cl}')
        self.project_changelist_label.setToolTip('Project CL')

        self.project_changelist_label.show()
        self.sync_button.show()
        self.build_button.show()

        try:
            required_cl_int = int(required_cl)
        except (TypeError, ValueError):
            self.sync_button.hide()

        is_synched = required_cl is None or str(required_cl) == str(current_device_cl)
        self._set_project_changelist_is_synched(is_synched)

    def update_build_status(self, device, step, percent):
        self.project_changelist_label.setText(f'Building...{percent}')
        self.project_changelist_label.setToolTip(step)

        self.project_changelist_label.show()

    def update_sync_status(self, device, percent):
        self.project_changelist_label.setText(f'Syncing...{percent}')
        self.project_changelist_label.setToolTip(
            'Syncing from Version Control')

        self.project_changelist_label.show()

    @QtCore.Slot(Device, str, str)
    def update_package_status(self, device: Device, step: str, percentstr: str):
        if threading.current_thread() is not threading.main_thread():
            QtCore.QMetaObject.invokeMethod(
                self,
                'update_package_status',
                QtCore.Qt.QueuedConnection,
                QtCore.Q_ARG(Device, device),
                QtCore.Q_ARG(str, step),
                QtCore.Q_ARG(str, percentstr)
            )
            return

        self.project_changelist_label.setText(f'Packaging...{percentstr}')
        self.project_changelist_label.setToolTip(step)

        self.project_changelist_label.show()

    def update_engine_changelist(
        self,
        required_cl: str | None,
        device: DeviceUnreal,
    ):
        if not CONFIG.ENGINE_SYNC_METHOD.get_value() == EngineSyncMethod.Build_Engine.value:
            return

        synced_cl = device.engine_changelist

        self.engine_changelist_label.setText(f'E: {synced_cl}')
        if not self.exclude_from_build:
            self.engine_changelist_label.show()

        self.sync_button.show()
        self.build_button.show()

        try:
            required_cl_int = int(required_cl)
        except (TypeError, ValueError):
            self.sync_button.hide()

        is_synched = required_cl is None or str(required_cl) == str(synced_cl)
        self._set_engine_changelist_is_synched(is_synched)
        self.update_build_info(device)

    def _set_project_changelist_is_synched(self, is_synched: bool):
        self._is_project_synched = is_synched
        sb_widgets.set_qt_property(self.project_changelist_label, 'not_synched', not is_synched)

        self._update_cl_widget_tooltip(self.project_changelist_label, BASE_PROJECT_CL_TOOLTIP, self._is_project_synched)
        self._update_sync_button()

    def _set_engine_changelist_is_synched(self, is_synched: bool):
        self._is_engine_synched = is_synched
        self._update_engine_cl_label()

        self._update_cl_widget_tooltip(self.engine_changelist_label, BASE_ENGINE_CL_TOOLTIP, self._is_engine_synched)
        self._update_sync_button()

    def _update_sync_button(self):
        needs_resync = not self._is_project_synched or not self._is_engine_synched
        sb_widgets.set_qt_property(self.sync_button, 'not_synched', needs_resync)

    def update_build_info(
        self,
        device: DeviceUnreal,
    ):
        self._needs_rebuild = False
        details = ''

        if CONFIG.ENGINE_SYNC_METHOD.get_value() == EngineSyncMethod.Build_Engine.value:
            eng_differs = device._built_eng_changelist != device._synced_changelist
            eng_code_differs = device._built_eng_compatible_cl != device._synced_compatible_cl
            if (eng_differs or eng_code_differs):
                self._needs_rebuild = True
                details += (f"\nBuilt engine CL: {device._built_eng_changelist}"
                            f" (Code: {device._built_eng_compatible_cl})")

            proj_differs = device._built_proj_changelist != device._synced_changelist
            proj_code_differs = device._built_proj_compatible_cl != device._synced_compatible_cl
            if (proj_differs or proj_code_differs):
                self._needs_rebuild = True
                details += (f"\nBuilt project CL: {device._built_proj_changelist}"
                            f" (Code: {device._built_proj_compatible_cl})")

        desired_tooltip = 'Build changelist'

        if self._needs_rebuild:
            desired_tooltip += "\n\nBuild required.\n\n"
        else:
            desired_tooltip += " (not required - built CL matches synced CL)\n\n"

        desired_tooltip += (f"Synced CL: {device._synced_changelist} "
                            f"(Code: {device._synced_compatible_cl})")

        desired_tooltip += details

        self._desired_build_button_tooltip = desired_tooltip
        self._update_build_button_tooltip()
        self._refresh_build_info_ui()

    def _refresh_build_info_ui(self):
        should_update_ui = not self.exclude_from_build
        if should_update_ui:
            sb_widgets.set_qt_property(self.build_button, 'not_built', self._needs_rebuild)
            self._update_build_button_tooltip()
            self._update_engine_cl_label()
            self._update_cl_widget_tooltip(self.engine_changelist_label, BASE_ENGINE_CL_TOOLTIP, self._is_engine_synched)

    def _update_build_button_tooltip(self):
        if self.exclude_from_build:
            self.build_button.setToolTip("Excluded from build (see device settings)")
        else:
            self.build_button.setToolTip(self._desired_build_button_tooltip)

    def _update_engine_cl_label(self):
        if self.exclude_from_build:
            sb_widgets.set_qt_property(self.engine_changelist_label, 'not_synched', False)
            sb_widgets.set_qt_property(self.engine_changelist_label, 'not_built', False)
        if not self._is_engine_synched:
            sb_widgets.set_qt_property(self.engine_changelist_label, 'not_synched', True)
            sb_widgets.set_qt_property(self.engine_changelist_label, 'not_built', False)
        elif self._needs_rebuild:
            sb_widgets.set_qt_property(self.engine_changelist_label, 'not_synched', False)
            sb_widgets.set_qt_property(self.engine_changelist_label, 'not_built', True)
        else:
            sb_widgets.set_qt_property(self.engine_changelist_label, 'not_synched', False)
            sb_widgets.set_qt_property(self.engine_changelist_label, 'not_built', False)

    def _update_cl_widget_tooltip(self, label: QtWidgets.QLabel, base_tooltip: str, is_synched: bool):
        tooltip = base_tooltip

        if not is_synched:
            tooltip += "\nNot synced to selected CL"

        if self._needs_rebuild:
            tooltip += "\nSynced CL not built"

        label.setToolTip(tooltip)

    def sync_button_clicked(self):
        self.signal_device_widget_sync.emit(self)

    def build_button_clicked(self):
        self.signal_device_widget_build.emit(self)

    def open_button_clicked(self):
        if self.open_button.isChecked():
            self._open()
        else:
            self._close()

    def connect_button_clicked(self):
        if self.connect_button.isChecked():
            self._connect()
        else:
            self._disconnect()

    def populate_context_menu(self, cmenu: QtWidgets.QMenu):
        ''' Called to populate the given context menu with any desired actions '''

        # Remove device is already added by parent widget
        cmenu.addSeparator()

        # Exclude options with checkboxes
        exclude_build_action = cmenu.addAction("Exclude from build")
        exclude_build_action.setCheckable(True)
        exclude_build_action.setChecked(self.exclude_from_build)
        exclude_build_action.triggered.connect(lambda: self.signal_exclude_from_build_toggled.emit())

        exclude_insights_action = cmenu.addAction("Exclude from Insights traces")
        exclude_insights_action.setCheckable(True)
        exclude_insights_action.setChecked(self.exclude_from_insights)
        exclude_insights_action.triggered.connect(lambda: self.signal_exclude_from_insights_toggled.emit())

        cmenu.addSeparator()

        # Log and trace options
        cmenu.addAction("Open fetched log", lambda: self.signal_open_last_log.emit(self))

        # View Live Log - only if Unreal is running
        if hasattr(self, '_device') and self._device and self._device.status >= DeviceStatus.OPEN:
            cmenu.addAction("View Live Log", lambda: self.signal_start_live_log.emit(self))

        cmenu.addAction("Open fetched trace", lambda: self.signal_open_last_trace.emit(self))
        cmenu.addAction("Copy last launch command", lambda: self.signal_copy_last_launch_command.emit(self))

        cmenu.addSeparator()

        # Fill DDC option - only if node is connected
        if self.connect_button.isChecked():
            fill_ddc_menu = cmenu.addMenu("Fill DDC (Prepare Shaders)")
            fill_ddc_menu.addAction("Current Level", lambda: self.signal_device_widget_fill_ddc.emit(self, True))
            fill_ddc_menu.addAction("All Levels", lambda: self.signal_device_widget_fill_ddc.emit(self, False))
