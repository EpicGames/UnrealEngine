# Copyright Epic Games, Inc. All Rights Reserved.

from __future__ import annotations

import base64
import hashlib
import os
import threading
from typing import Dict, Optional

from PySide6 import QtCore, QtWidgets

from switchboard import message_protocol
from switchboard.listener_client import ListenerClient
from switchboard.switchboard_logging import LOGGER
from . import version_helpers


COLOR_GREEN = '#4f4'
COLOR_RED = '#f44'
COLOR_YELLOW = '#ff4'

CONNECT_TIMEOUT_SECONDS = 5


class RedeployListenerEndpoint(QtCore.QObject):
    signal_refresh_ui = QtCore.Signal()
    signal_result = QtCore.Signal(bool, str, BaseException) # (bSuccess, details, exc_info)

    def __init__(
        self,
        parent: RedeployListenerDialog,
        address: str,
        port: int
    ):
        super().__init__(parent)

        self.dlg_parent = parent
        self.dlg_parent.signal_listener_changed.connect(self.refresh_ui)

        self.address = address
        self.port = port

        self.client = ListenerClient(self.address, self.port, parent=self)
        self.client.listener_qt_handler.listener_connecting.connect(
            lambda _: self.version_label.setText('Connecting...'))
        self.client.listener_qt_handler.listener_connection_failed.connect(
            lambda _: self.version_label.setText('Connection failed'))

        self.client.delegates['state'] = self.on_listener_state
        self.client.delegates['redeploy server status'] = self.on_redeploy_server_status

        # These are broadcast and could hit our version check client unsolicited.
        # TODO: Maybe default these in ListenerClient?
        self.client.delegates['program started'] = lambda _: None
        self.client.delegates['program ended'] = lambda _: None
        self.client.delegates['program killed'] = lambda _: None
        self.client.delegates['programstdout'] = lambda _: None

        self.devices = []
        self.version: Optional[version_helpers.ListenerVersion] = None

        self.signal_refresh_ui.connect(self.refresh_ui)

        self.endpoint_label = QtWidgets.QLabel(f"{self.address}:{self.port}")
        self.endpoint_label.setObjectName('endpoint_label')

        self.devices_label = QtWidgets.QLabel('')
        self.devices_label.setObjectName('devices_label')

        self.version_label = QtWidgets.QLabel('')
        self.version_label.setObjectName('version_label')

        self.redeploy_checkbox = QtWidgets.QCheckBox()
        self.redeploy_checkbox.setObjectName('redeploy_checkbox')

    @QtCore.Slot()
    def refresh_ui(self):
        if not self.version:
            return

        version_str = version_helpers.version_str(self.version)
        if not version_helpers.listener_is_compatible(self.version):
            version_color = COLOR_RED
            self.redeploy_checkbox.setChecked(True)
        elif (self.dlg_parent.listener_ver and
              self.dlg_parent.listener_ver > self.version):
            version_color = COLOR_YELLOW
            self.redeploy_checkbox.setChecked(True)
        else:
            version_color = COLOR_GREEN
            self.redeploy_checkbox.setChecked(False)

        rich_ver = f'<span style="color: {version_color}">{version_str}</span>'

        self.version_label.setText(f"Version {rich_ver}")

    @QtCore.Slot()
    def on_redeploy_clicked(self):
        if self.redeploy_checkbox.isChecked():
            self.do_redeploy()

    def do_redeploy(self):
        if not version_helpers.listener_supports_redeploy(self.version):
            return

        if not self.dlg_parent.listener_exe_base64 or not self.dlg_parent.listener_exe_sha1sum:
            self.signal_result.emit(False, 'Tried to do_redeploy, but listener unavailable')
            return

        _, req = message_protocol.create_redeploy_listener_message(self.dlg_parent.listener_exe_base64, self.dlg_parent.listener_exe_sha1sum)
        self.client.send_message(req)

    @QtCore.Slot()
    def connect_client(self):
        # Ensure this code is run from the main thread
        if threading.current_thread() is not threading.main_thread():
            QtCore.QMetaObject.invokeMethod(self, 'connect_client', QtCore.Qt.QueuedConnection)
            return

        # Suppress on_disconnect for this expected disconnect
        self.client.disconnect()
        self.client.disconnect_delegate = self.on_disconnect
        self.client.connect(timeout=CONNECT_TIMEOUT_SECONDS)

    def on_disconnect(self, unexpected, exception):
        self.version = None

        if unexpected:
            self.signal_result.emit(False, 'Unexpected disconnect', exception)

        self.signal_refresh_ui.emit()
        self.connect_client()

    def on_listener_state(self, message):
        self.version = version_helpers.listener_ver_from_state_message(message)
        if not self.version:
            self.signal_result.emit(False, 'Unable to parse listener version')
            return

        self.signal_refresh_ui.emit()

    def on_redeploy_server_status(self, message):
        ''' "redeploy server status" tells us whether the redeploy succeeded from the server's POV '''
        if message.get('bAck', False) != True:
            self.signal_result.emit(False, f"Server redeploy NACK: {message}")
            return

        # Suppress on_disconnect for this expected disconnect
        self.client.disconnect_delegate = None
        self.connect_client()


class RedeployListenerDialog(QtWidgets.QDialog):
    signal_listener_changed = QtCore.Signal()

    def __init__(self, devices, listener_watcher, parent=None):
        super().__init__(parent=parent, f=QtCore.Qt.WindowCloseButtonHint)

        self.listener_watcher = listener_watcher

        self.endpoints: Dict[tuple, RedeployListenerEndpoint] = {}

        self.listener_ver: Optional[version_helpers.ListenerVersion] = None
        self.listener_exe_base64: Optional[str] = None
        self.listener_exe_sha1sum: Optional[str] = None

        self.setWindowTitle('Listener Redeployer')

        self.finished.connect(self.on_dialog_finished)

        NUM_COLS = 4

        layout = QtWidgets.QGridLayout(self)
        self.setLayout(layout)
        layout.setHorizontalSpacing(20)
        row_idx = 0

        # Local version header row
        layout.addWidget(QtWidgets.QLabel('<b>Local Listener</b>'), row_idx, 0, 1, NUM_COLS)
        row_idx += 1

        # Local listener info row
        self.listener_path_label = QtWidgets.QLabel('')
        layout.addWidget(self.listener_path_label, row_idx, 0, 1, 3)

        self.listener_ver_label = QtWidgets.QLabel('')
        layout.addWidget(self.listener_ver_label, row_idx, 3)

        row_idx += 1

        # Spacer row
        layout.addWidget(QtWidgets.QLabel(''), row_idx, 0, 1, 3)
        row_idx += 1

        # Listeners header row
        layout.addWidget(QtWidgets.QLabel('<b>Remote Listeners</b>'), row_idx, 0, 1, NUM_COLS)
        row_idx += 1

        # Remote listener models and rows
        for device in devices:
            ep_addr = (device.unreal_client.address, device.unreal_client.port)
            if ep_addr not in self.endpoints:
                endpoint = RedeployListenerEndpoint(self, device.unreal_client.address, device.unreal_client.port)
                endpoint.signal_result.connect(self.on_endpoint_result)
                endpoint.signal_refresh_ui.connect(self.refresh_ui)
                endpoint.redeploy_checkbox.stateChanged.connect(
                    lambda: self.redeploy_btn.setEnabled(
                        any(ep.redeploy_checkbox.isChecked()
                            for ep in self.endpoints.values()))
                )
                endpoint.client.connect()
                layout.addWidget(endpoint.redeploy_checkbox, row_idx, 0)
                layout.addWidget(endpoint.endpoint_label,    row_idx, 1)
                layout.addWidget(endpoint.devices_label,     row_idx, 2)
                layout.addWidget(endpoint.version_label,     row_idx, 3)
                self.endpoints[ep_addr] = endpoint
                row_idx += 1

            self.endpoints[ep_addr].devices.append(device)

        # Spacer row
        layout.addWidget(QtWidgets.QLabel(''), row_idx, 0, 1, 3)
        row_idx += 1

        # Button row
        self.redeploy_btn = QtWidgets.QPushButton('Redeploy Selected')
        self.redeploy_btn.clicked.connect(self.on_redeploy_clicked)
        self.redeploy_btn.setEnabled(False)
        layout.addWidget(self.redeploy_btn, row_idx, 0, 1, NUM_COLS)

        for endpoint in self.endpoints.values():
            endpoint.devices_label.setText(f"{', '.join(d.name for d in endpoint.devices)}")

        self.try_refresh_listener()
        self.listener_watcher.signal_listener_changed.connect(self.on_listener_changed)

    def try_refresh_listener(self):
        # Read and hash local listener executable for redeploy
        self.listener_path = self.listener_watcher.listener_path
        self.listener_ver = self.listener_watcher.listener_ver
        self.listener_exe_base64 = None
        self.listener_exe_sha1sum = None
        if self.listener_ver:
            try:
                with open(self.listener_path, 'rb') as f:
                    listener_bytes = f.read()
                    self.listener_exe_base64 = base64.b64encode(listener_bytes).decode()
                    self.listener_exe_sha1sum = hashlib.sha1(listener_bytes).hexdigest()
            except BaseException as e:
                LOGGER.error(f"Error reading listener from {self.listener_path}", exc_info=e)

        self.signal_listener_changed.emit()
        self.refresh_ui()

    def redeploy_available(self) -> bool:
        if not self.listener_ver or not self.listener_exe_base64 or not self.listener_exe_sha1sum:
            return False

        if not version_helpers.listener_is_compatible(self.listener_ver):
            return False

        return True

    @QtCore.Slot()
    def refresh_ui(self):
        listener_path_label_str = f"<u>{os.path.split(self.listener_path)[1]}</u>"
        if not self.listener_ver or not self.listener_exe_base64 or not self.listener_exe_sha1sum:
            listener_path_label_str = (f'<span style="color: {COLOR_RED}">'
                                       f'{listener_path_label_str}</span>')

        self.listener_path_label.setText(listener_path_label_str)
        self.listener_path_label.setToolTip(self.listener_path)

        listener_ver_str = version_helpers.version_str(self.listener_ver) if self.listener_ver else '(N/A)'
        if not self.listener_ver or not version_helpers.listener_is_compatible(self.listener_ver):
            listener_ver_str = (f'<span style="color: {COLOR_RED}">'
                                f'{listener_ver_str}</span>')

        self.listener_ver_label.setText(f"Version {listener_ver_str}")

    @QtCore.Slot()
    def on_listener_changed(self):
        self.try_refresh_listener()

    @QtCore.Slot(int)
    def on_dialog_finished(self, result: int):
        for (ep_addr, endpoint) in self.endpoints.items():
            endpoint.client.disconnect()

    @QtCore.Slot()
    def on_redeploy_clicked(self):
        for (ep_addr, endpoint) in self.endpoints.items():
            endpoint.on_redeploy_clicked()

    @QtCore.Slot(bool, str)
    def on_endpoint_result(self, success, details, exc_info: Optional[BaseException]):
        sender = self.sender()
        assert isinstance(sender, RedeployListenerEndpoint)
        logfn = LOGGER.info if success else LOGGER.error
        logfn(f"Redeploy endpoint {sender.address}:{sender.port}: {details}", exc_info=exc_info)
