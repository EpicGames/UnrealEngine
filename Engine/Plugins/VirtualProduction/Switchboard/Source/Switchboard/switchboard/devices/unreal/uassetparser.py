# Copyright Epic Games, Inc. All Rights Reserved.

'''
Parse some parts of .uasset files directly from Python (notably, the asset registry).

Find the relevant C++ code in the UE codebase:
  * PackageFileSummary.cpp
  * PackageReader.cpp
  * PackageUtilities.cpp
'''

from enum import IntEnum
import logging
import os
import struct
from typing import BinaryIO


LOGGER = logging.getLogger(__name__)


class EUnrealEngineObjectUE5Version(IntEnum):
    ADD_SOFTOBJECTPATH_LIST = 1008
    METADATA_SERIALIZATION_OFFSET = 1014
    VERSE_CELLS = 1015
    PACKAGE_SAVED_HASH = 1016
    IMPORT_TYPE_HIERARCHIES = 1018


class UassetSummary(object):
    ''' Class to hold the PacketSummary of the asset.
    '''
    def print(self):
        from pprint import pprint
        pprint(vars(self))


class AssetRegistryData(object):
    ''' Class to hold the asset registry data of the asset
    '''
    def __init__(self):
        self.ObjectPath = ''
        self.ObjectClassName = ''
        self.tags = {}


class FStringError(Exception):
    pass


class AssetData(object):
    ''' Class to hold a single asset data from the asset registry
    '''
    def __init__(self):
        self.AssetClassName = ''
        self.ObjectPathWithoutPackageName = ''
        self.FileOffset = -1


class UassetParser(object):
    ''' Class with the logic to parse the asset.

    Example usage:
        with open('file.uasset', 'rb') as file:
            aparser = UassetParser(file)

            for assetdata in aparser.aregdata:
                print('ObjectPath     : {}'.format(assetdata.ObjectPath))
                print('ObjectClassName: {}'.format(assetdata.ObjectClassName))
                print('Tags')

                for k,v in assetdata.tags.items():
                    print('Tag {}: {}'.format(k,v))
    '''

    _names = None
    _aregdata = None
    _thumbnailCache = None

    def __init__(self, fileObj: BinaryIO, allowUnversioned=True):
        ''' Parses `fileObj` and reads the summary, which has the most basic information about the asset. '''

        self.allowUnversioned = allowUnversioned

        self.f = fileObj

        self.f.seek(0, os.SEEK_END)
        self.PackageFileSize = self.f.tell()

        self.asummary = self._readUassetSummary()

    @property
    def thumbnailCache(self):
        ''' Lazy read of the thumbnailCache '''

        if self._thumbnailCache is not None:
            return self._thumbnailCache

        self._thumbnailCache = self._readAssetDataFromThumbnailCache()
        return self._thumbnailCache

    @property
    def names(self):
        ''' Lazy read of the "names" of the asset '''
        if self._names is not None:
            return self._names

        self._names = self._readNames()
        return self._names

    @property
    def aregdata(self):
        ''' Lazy read of the asset registry data '''
        if self._aregdata is None:
            self._aregdata = self._readAssetRegistryData()

        return self._aregdata

    def readFString(self):
        ''' Reads an FString from the current file position '''
        sz, = struct.unpack('<i',self.f.read(4))

        if sz == 0:
            return ''

        LoadUCS2Char = False

        if sz < 0:
            LoadUCS2Char = True
            sz = -sz

        if LoadUCS2Char:
            sz *= 2

        msg, nullTerminator = struct.unpack('%dsB' % (sz-1), self.f.read(sz))

        if LoadUCS2Char:
            msg = msg[:-1].decode('utf-16')
        else:
            msg = msg.decode('utf-8')

        return msg

    def skipBytes(self,n):
        ''' Skips the specified number of bytes from current file position '''
        self.f.seek(self.f.tell() + n)

    def skipTArray(self, typeSz, maxElements=100000):
        ''' Skips a TArray that holds the specified type size '''
        n = self.readInt32()

        if not (0 <= n <= maxElements):
            raise ValueError("Invalid TArray.Num() == {}".format(n))

        self.skipBytes(n * typeSz)

    def readBool(self):
        v, = struct.unpack('<?', self.f.read(1))
        return v

    def readUInt16(self):
        v, = struct.unpack('<H', self.f.read(2))
        return v

    def readUInt32(self):
        v, = struct.unpack('<I', self.f.read(4))
        return v

    def readInt32(self):
        v, = struct.unpack('<i', self.f.read(4))
        return v

    def readInt64(self):
        v, = struct.unpack('<q', self.f.read(8))
        return v

    def readUInt64(self):
        v, = struct.unpack('<Q', self.f.read(8))
        return v

    def readTArray(self, elementreaderfn, maxElements=100000):
        ''' Reads TArray of any type.

        Args:
            elementreaderfn: A function that reads the element at current file offset.
        '''
        arr = []
        n = self.readInt32()

        if not (0 <= n <= maxElements):
            raise ValueError("Invalid TArray.Num() == {}".format(n))

        for _ in range(n):
            arr.append(elementreaderfn())

        return arr

    def _readNames(self):
        ''' Read the names of the asset '''

        if self.asummary.NameCount <= 0:
            return []

        fos = self.asummary.NameOffset

        if (fos <= 0) or (fos > self.PackageFileSize):
            return []

        self.f.seek(fos)

        names = []

        for _ in range(self.asummary.NameCount):
            name = self.readFString()
            self.skipBytes(4) # precalculated hashes
            names.append(name)

        return names

    def _readAssetRegistryData(self):
        ''' Read the asset registry data. Find code in PackageUtilities.cpp and PackageReader.cpp
        '''

        fos = self.asummary.AssetRegistryDataOffset

        if (fos <= 0) or (fos > self.PackageFileSize):
            return []

        self.f.seek(fos)

        DependencyDataOffset = self.readInt64()

        self.checkFileOffset(DependencyDataOffset)

        nAssets = self.readInt32()

        if nAssets < 0:
            raise ValueError('Invalid number of {} AssetRegistryDatas'.format(nAssets))

        assets = []

        for _ in range(nAssets):
            asset = AssetRegistryData()
            asset.ObjectPath = self.readFString()
            asset.ObjectClassName = self.readFString()

            assets.append(asset)

            nTags = self.readInt32()
            asset.tags = {}

            for _ in range(nTags):
                try:
                    key = self.readFString()
                    val = self.readFString()
                    asset.tags[key] = val
                except FStringError:
                    return assets

        return assets

    def checkFileOffset(self, fos):
        ''' Convenience function to sanity-check file offsets '''

        # fos of 0 is valid but also indicates that there is no data, so not raising an exception in such case
        if (fos < 0) or (fos > self.PackageFileSize):
            raise ValueError('Invalid file offset of {} given file size of {}'.format(fos, self.PackageFileSize))

    def checkCompressionFlags(self, flags):
        ''' Checks the compression flags for invalid values '''
        COMPRESS_DeprecatedFormatFlagsMask = 0x0F
        COMPRESS_OptionsFlagsMask = 0xF0

        CompressionFlagsMask = COMPRESS_DeprecatedFormatFlagsMask | COMPRESS_OptionsFlagsMask

        if (flags & (~CompressionFlagsMask)):
            raise ValueError("Invalid CompressionFlags")

    def checkAssetVersion(self, Major, Minor, Patch):
        ''' Checks the asset version '''
        MinMajor = 4
        MinMinor = 27

        if Major == 0:
            if not self.allowUnversioned:
                raise ValueError('Unversioned asset parsing not allowed')
        elif(Major < MinMajor or (Major == MinMajor and Minor < MinMinor)):
            raise ValueError('We cannot parse assets older than {}.{}, and this one was created with {}.{}'.format(
                MinMajor, MinMinor, Major, Minor))

    def _readUassetSummary(self):
        ''' Read the package summary of the asset
        '''
        self.f.seek(0)

        s = UassetSummary()

        s.Tag = self.readUInt32()

        if s.Tag != 0x9e2a83c1:
            raise ValueError('Not a valid uasset!')

        s.LegacyFileVersion = self.readInt32()

        if s.LegacyFileVersion not in [-7,-8,-9]:
            raise ValueError('LegacyFileVersion {} is non handled by this parser'.format(s.LegacyFileVersion))

        s.LegacyUE3Version = self.readInt32()
        s.FileVersionUE4 = self.readInt32()

        if s.LegacyFileVersion <= -8:
            s.FileVersionUE5 = self.readInt32()
        else:
            s.FileVersionUE5 = 0

        s.FileVersionLicenseeUE4 = self.readUInt32()

        KNOWN_SUPPORTED_UE5VER = 1018
        if s.FileVersionUE5 > KNOWN_SUPPORTED_UE5VER:
            LOGGER.warning(f"{self.f.name}: ObjectUE5Version {s.FileVersionUE5} too new; "
                           f"newest known supported version {KNOWN_SUPPORTED_UE5VER}")
            LOGGER.warning(f"{self.f.name}: Parsing will attempt to continue, but there "
                           "may be errors reading the file")

        if s.FileVersionUE5 >= EUnrealEngineObjectUE5Version.PACKAGE_SAVED_HASH:
            s.SavedHash = struct.unpack('<20c', self.f.read(20))
            s.TotalHeaderSize = self.readInt32()

        s.CustomVersions = self.readTArray(lambda : self.f.read(5*4))

        if s.FileVersionUE5 < EUnrealEngineObjectUE5Version.PACKAGE_SAVED_HASH:
            s.TotalHeaderSize = self.readInt32()

        s.PackageName = self.readFString()
        s.PackageFlags, s.NameCount, s.NameOffset = struct.unpack('<Iii', self.f.read(4*3))

        if s.FileVersionUE5 >= EUnrealEngineObjectUE5Version.ADD_SOFTOBJECTPATH_LIST:
            s.SoftObjectPathsCount = self.readInt32()
            s.SoftObjectPathsOffset = self.readInt32()

        s.LocalizationId = self.readFString()

        def remainingBytesInHeader():
            ''' Convenience local function to calculate remaning bytes in header. Useful for sanity checks during parsing '''
            return s.TotalHeaderSize - self.f.tell() + 1

        s.GatherableTextDataCount, s.GatherableTextDataOffset, \
        s.ExportCount, s.ExportOffset, s.ImportCount, s.ImportOffset \
            = struct.unpack('<6i', self.f.read(6*4))

        if s.FileVersionUE5 >= EUnrealEngineObjectUE5Version.VERSE_CELLS:
            s.CellExportCount, s.CellExportOffset, s.CellImportCount, s.CellImportOffset \
                = struct.unpack('<4i', self.f.read(4*4))

        if s.FileVersionUE5 >= EUnrealEngineObjectUE5Version.METADATA_SERIALIZATION_OFFSET:
            s.MetaDataOffset = self.readInt32()

        s.DependsOffset, s.SoftPackageReferencesCount, s.SoftPackageReferencesOffset, \
        s.SearchableNamesOffset, s.ThumbnailTableOffset \
            = struct.unpack('<5i', self.f.read(5*4))

        if s.FileVersionUE5 >= EUnrealEngineObjectUE5Version.IMPORT_TYPE_HIERARCHIES:
            s.ImportTypeHierarchiesCount, s.ImportTypeHierarchiesOffset \
                = struct.unpack('<2i', self.f.read(2*4))

        if s.FileVersionUE5 < EUnrealEngineObjectUE5Version.PACKAGE_SAVED_HASH:
            s.Guid = struct.unpack('<16s', self.f.read(16))

        s.PersistentGuid = struct.unpack('<16s', self.f.read(16))

        self.checkFileOffset(s.GatherableTextDataOffset)
        self.checkFileOffset(s.ExportOffset)
        self.checkFileOffset(s.ImportOffset)
        self.checkFileOffset(s.DependsOffset)
        self.checkFileOffset(s.SoftPackageReferencesOffset)
        self.checkFileOffset(s.SearchableNamesOffset)
        self.checkFileOffset(s.ThumbnailTableOffset)

        s.Generations = self.readTArray(lambda : self.f.read(8), maxElements=remainingBytesInHeader()/20)

        s.SavedByEngineVersionMajor, s.SavedByEngineVersionMinor, s.SavedByEngineVersionPatch, \
        s.SavedByEngineVersionChangelist = struct.unpack('<HHHI', self.f.read(2*3+4))
        s.SavedByEngineVersionName = self.readFString()

        s.CompatibleEngineVersionMajor, s.CompatibleEngineVersionMinor, s.CompatibleEngineVersionPatch, \
        s.CompatibleEngineVersionChangelist = struct.unpack('<HHHI', self.f.read(2*3+4))
        s.CompatibleEngineVersionName = self.readFString()

        self.checkAssetVersion(s.SavedByEngineVersionMajor, s.SavedByEngineVersionMinor, s.SavedByEngineVersionPatch)

        s.CompressionFlags = self.readUInt32()
        self.checkCompressionFlags(s.CompressionFlags)

        self.CompressedChunks = self.readTArray(lambda : self.f.read(4*4), maxElements=remainingBytesInHeader()/(4*4))

        if len(self.CompressedChunks) > 0:
            raise ValueError("We can't parse with CompressedChunks")

        s.PackageSource = self.readUInt32()

        s.AdditionalPackagesToCook = self.readTArray(self.readFString, maxElements=remainingBytesInHeader())

        s.AssetRegistryDataOffset = self.readInt32()
        s.BulkDataStartOffset = self.readInt64()

        self.checkFileOffset(s.AssetRegistryDataOffset)
        self.checkFileOffset(s.BulkDataStartOffset)

        return s

    def _readAssetDataFromThumbnailCache(self):
        ''' Read the asset data from the thumbnail cache.
        '''
        fos = self.asummary.ThumbnailTableOffset

        if (fos <= 0) or (fos > self.PackageFileSize):
            return []

        self.f.seek(fos)

        ObjectCount = self.readInt32()

        AssetDataList = []

        for _ in range(ObjectCount):
            assetData = AssetData()

            assetData.AssetClassName = self.readFString()
            assetData.ObjectPathWithoutPackageName = self.readFString()
            assetData.FileOffset = self.readInt32()

            AssetDataList.append(assetData)

        return AssetDataList    

def printAssetData(aparser, bAssetRegistry, bTags, bNames, bThumbnailCache):
    ''' Convenience function to print and visualize the asset data '''

    aparser.asummary.print()

    if bAssetRegistry:
        for idx,assetdata in enumerate(aparser.aregdata):
            print('\nAssetData {}\n'.format(idx))
            print('ObjectPath     : {}'.format(assetdata.ObjectPath))
            print('ObjectClassName: {}'.format(assetdata.ObjectClassName))
            if bTags:
                print('Tags')
                for k,v in assetdata.tags.items():
                    print('Tag {}: {}'.format(str(k),str(v)))

    if bNames:
        print('\nNames\n')
        for idx,name in enumerate(aparser.names):
            print('Name {}: {}'.format(idx,name))

    if bThumbnailCache:
        print('\nThumbnailCache')
        for assetdata in aparser.thumbnailCache:
            print()
            print('AssetClassName              : {}'.format(assetdata.AssetClassName))
            print('ObjectPathWithoutPackageName: {}'.format(assetdata.ObjectPathWithoutPackageName))
            print('FileOffset                  : {}'.format(assetdata.FileOffset))

if __name__ == '__main__':
    ''' When this script is run standalone, it can be used to parse and print the asset in the given path '''
    import sys

    fpath = os.path.dirname(os.path.abspath(__file__))
    fpath = os.path.join(fpath, "../../../../../../../../../Templates/TP_InCamVFXBP/Content/InCamVFXBP/ExampleConfigs/nDisplayConfig_Curved.uasset")

    if(len(sys.argv) > 1):
        fpath = sys.argv[1]

    argvlower = [s.lower() for s in sys.argv[1:]]

    bAssetRegistry = '-assetregistry' in argvlower
    bTags = '-tags' in argvlower
    bNames = '-names' in argvlower
    bThumbnailCache = '-thumbnailcache' in argvlower

    fpath = os.path.abspath(fpath)

    with open(fpath, 'rb') as file:
        aparser = UassetParser(file)
        printAssetData(aparser, bAssetRegistry, bTags, bNames, bThumbnailCache)
