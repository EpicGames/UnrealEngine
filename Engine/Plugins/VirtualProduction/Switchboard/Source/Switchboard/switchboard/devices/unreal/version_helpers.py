# Copyright Epic Games, Inc. All Rights Reserved.

import typing


LISTENER_COMPATIBLE_VERSION = (3,5)

# Version where redeploy support was initially introduced.
LISTENER_MIN_REDEPLOY_VERSION = (1,4,0)


ListenerVersion = typing.Tuple[int, int, int]


def version_str(ver_tuple: typing.Tuple[int, ...]) -> str:
    return '.'.join(str(i) for i in ver_tuple)

def listener_supports_redeploy(listener_ver: ListenerVersion):
    return listener_ver >= LISTENER_MIN_REDEPLOY_VERSION

def listener_is_compatible(listener_ver: ListenerVersion):
    return listener_ver[:2] == LISTENER_COMPATIBLE_VERSION

def listener_ver_from_state_message(
    state_msg: dict
) -> typing.Optional[ListenerVersion]:
    try:
        version = int(state_msg['version'])
    except (KeyError, ValueError):
        return None

    major = (version >> 16) & 0xFF
    minor = (version >>  8) & 0xFF
    patch = (version >>  0) & 0xFF
    return (major, minor, patch)
