# Copyright Epic Games, Inc. All Rights Reserved.

from __future__ import annotations

import asyncio
import logging
import ssl
import threading
import traceback
import uuid

from abc import ABC, abstractmethod
from asyncio import Future
from collections import deque
from datetime import datetime, timedelta
from typing import Callable, Dict, Optional, cast

from aioquic.asyncio.client import connect
from aioquic.asyncio.protocol import QuicConnectionProtocol
from aioquic.quic.configuration import QuicConfiguration
from aioquic.quic.events import QuicEvent, HandshakeCompleted
from cryptography.hazmat.primitives import hashes
from cryptography.x509.oid import NameOID

from PySide6 import QtCore, QtGui, QtWidgets
from PySide6.QtWidgets import QMessageBox

from . import message_protocol
from .credential_store import CredentialStore, CREDENTIAL_STORE
from .switchboard_logging import LOGGER


SWITCHBOARD_ALPN = 'ue-switchboard'


class SwitchboardClientProtocol(QuicConnectionProtocol):
    def __init__(self, *args, **kwargs):
        super().__init__(*args, **kwargs)

        self.client: Optional[ListenerClient] = None
        self.fingerprint_sha2: Optional[bytes] = None
        self.subject_cn = ''

    # @override
    def quic_event_received(self, event: QuicEvent) -> None:
        super().quic_event_received(event)

        if isinstance(event, HandshakeCompleted):
            server_cert = self._quic.tls._peer_certificate
            if not server_cert:
                return  # We get here on failed connection attempts

            self.fingerprint_sha2 = server_cert.fingerprint(hashes.SHA256())

            subject = server_cert.subject
            cn_attrs = subject.get_attributes_for_oid(NameOID.COMMON_NAME)
            self.subject_cn = '+'.join([str(attr.value) for attr in cn_attrs])

    async def authenticate(self) -> bool:
        assert self.client
        assert self.fingerprint_sha2

        fingerprint_str = self.fingerprint_sha2.hex()

        # truncated to 128 bits for visual comparison
        fingerprint_display_str = self.fingerprint_sha2[:16].hex(':').upper()

        host = self.client.address
        credential = CREDENTIAL_STORE.get(key=host)
        unsaved = credential is None

        if not credential:
            password = await self.client.listener_qt_handler.prompt_password(
                host, self.subject_cn, fingerprint_display_str)

            if password:
                credential = CredentialStore.Credential('', password)
            else:
                logging.error('No credential for %s', host)
                return False

        if not credential.username:
            # First time update with expected remote fingerprint
            credential.username = fingerprint_str
        elif credential.username != fingerprint_str:
            # Warn the user if the key changed
            result = await self.client.listener_qt_handler.prompt_fp_mismatch(
                host, self.subject_cn, fingerprint_display_str)

            if not result.proceed:
                return False

            if result.remember:
                unsaved = True
                credential.username = fingerprint_str

        auth_jwt: Optional[str] = None
        auth_password: Optional[str] = None
        if not unsaved and not CREDENTIAL_STORE.encrypted_at_rest():
            auth_jwt = credential.blob
        else:
            auth_password = credential.blob

        msgid, msg = message_protocol.create_authenticate_message(
            jwt=auth_jwt, password=auth_password)

        auth_response = asyncio.get_running_loop().create_future()
        self.client.get_message_response(msgid, msg, future=auth_response)

        try:
            response = await auth_response
            auth_succeeded = response['bAuthenticated'] == 'true'

            if auth_succeeded:
                if CREDENTIAL_STORE.encrypted_at_rest():
                    if unsaved and credential:
                        logging.info(f'Saving password for {host}')
                        CREDENTIAL_STORE.set(host, credential)
                elif 'jwt' in response:
                    credential = CredentialStore.Credential(
                        fingerprint_str, response['jwt'])
                    logging.info(f'Saving authentication token for {host}')
                    CREDENTIAL_STORE.set(host, credential)
            else:
                # Clear invalid saved credential
                if not unsaved:
                    logging.info(f'Forgetting incorrect password for {host}')
                    CREDENTIAL_STORE.set(host, None)

            return auth_succeeded
        except (TimeoutError, KeyError):
            return False


class FingerprintMismatchUserAction:
    def __init__(self):
        self.proceed: bool = False
        self.remember: bool = False


class ListenerQtHandler(QtCore.QObject):
    listener_connecting = QtCore.Signal(object)
    listener_connected = QtCore.Signal(object)
    listener_connection_failed = QtCore.Signal(object)

    class Prompt(ABC):
        @abstractmethod
        def display(self):
            pass

    class PromptPassword(Prompt):
        def __init__(
            self,
            host: str,
            subject_cn: str,
            fingerprint_desc: str,
        ):
            self.host = host
            self.subject_cn = subject_cn
            self.fingerprint_desc = fingerprint_desc
            self.future: Optional[Future[str]] = None

        def display(self):
            blob, ok = QtWidgets.QInputDialog.getText(
                None,
                'Enter Password',
                f'Enter password for {self.host} ({self.subject_cn})\n\n'
                f'(Fingerprint: {self.fingerprint_desc})',
                QtWidgets.QLineEdit.EchoMode.Password)

            if blob and ok:
                result = blob
            else:
                result = ''

            assert self.future
            future = self.future
            future.get_loop().call_soon_threadsafe(lambda: future.set_result(result))


    class PromptFingerprintMismatch(Prompt):
        def __init__(
            self,
            host: str,
            subject_cn: str,
            fingerprint_desc: str,
        ):
            self.host = host
            self.subject_cn = subject_cn
            self.fingerprint_desc = fingerprint_desc
            self.future: Optional[Future[FingerprintMismatchUserAction]] = None

        def display(self):
            title = 'Warning: Potential security breach!'
            text = (
                f'{title}'
                '\n\n'
                'The Switchboard Listener key does not match the key that has '
                'been cached. This means that either another user has changed '
                'the key, or you are actually trying to connect to another '
                'computer pretending to be the device.'
                '\n\n'
                'If the key change was not expected, please contact your '
                'administrator.'
                '\n\n'
                'Details for new key:\n'
                f'Host: {self.host} ({self.subject_cn})\n'
                f'Fingerprint: {self.fingerprint_desc}'
                '\n\n'
                'Trust the new key and carry on connecting?\n'
            )
            check_text = 'Update cached key for this device'

            dialog = QMessageBox()
            dialog.setWindowTitle(title)
            dialog.setText(text)

            checkbox = QtWidgets.QCheckBox(check_text)
            dialog.setCheckBox(checkbox)

            dialog.setIconPixmap(QtGui.QIcon(
                ':icons/images/alert-triangle-64.svg').pixmap(48, 48))

            ok_btn = dialog.addButton(QMessageBox.StandardButton.Ok)
            dialog.addButton(QMessageBox.StandardButton.Cancel)
            dialog.setDefaultButton(QMessageBox.StandardButton.NoButton)

            dialog.exec()

            result = FingerprintMismatchUserAction()

            if dialog.checkBox().isChecked():
                result.remember = True

            if dialog.clickedButton() == ok_btn:
                result.proceed = True

            assert self.future
            self.future.set_result(result)

    def __init__(self, parent=None):
        super().__init__(parent)

        self._queued_prompts: list[ListenerQtHandler.Prompt] = []

    async def prompt_password(
        self,
        host: str,
        subject_cn: str,
        fingerprint_desc: str,
    ) -> str:
        prompt = ListenerQtHandler.PromptPassword(host, subject_cn,
                                                  fingerprint_desc)
        prompt.future = asyncio.get_running_loop().create_future()
        self._queued_prompts.append(prompt)
        self._show_queued_prompts_soon()
        return await prompt.future

    async def prompt_fp_mismatch(
        self,
        host: str,
        subject_cn: str,
        fingerprint_desc: str,
    ) -> FingerprintMismatchUserAction:
        prompt = ListenerQtHandler.PromptFingerprintMismatch(
            host, subject_cn, fingerprint_desc)
        prompt.future = asyncio.get_running_loop().create_future()
        self._queued_prompts.append(prompt)
        self._show_queued_prompts_soon()
        return await prompt.future

    @QtCore.Slot()
    def _show_queued_prompts_soon(self):
        # Ensure this code is run from the main thread
        if threading.current_thread() is not threading.main_thread():
            QtCore.QMetaObject.invokeMethod(
                self, '_show_queued_prompts_soon', QtCore.Qt.QueuedConnection)
            return

        for prompt in self._queued_prompts:
            prompt.display()

        self._queued_prompts.clear()


class ListenerClient(QtCore.QObject):
    '''Connects to a server running SwitchboardListener.

    Runs a thread to service its socket, and upon receiving complete messages,
    invokes a handler callback from that thread (`handle_connection()`).

    `disconnect_delegate` is invoked on `disconnect()` or on socket errors.

    Handlers in the `delegates` map are passed a dict containing the entire
    JSON response from the listener, routed according to the "command" field
    string value. All new messages and handlers should follow this pattern.

    The other, legacy (VCS/file) delegates are each passed different (or no)
    arguments; for details, see `route_message()`.
    '''

    KEEPALIVE_INTERVAL_SEC = 1.0
    SELECT_TIMEOUT_SEC = 0.1
    EXTRA_HITCH_TOLERANCE_SEC = 0.4
    HITCH_THRESHOLD_SEC = (KEEPALIVE_INTERVAL_SEC + SELECT_TIMEOUT_SEC
                           + EXTRA_HITCH_TOLERANCE_SEC)
    IDLE_TIMEOUT_SEC = 5.0

    def __init__(self, address, port, buffer_size=1024, parent=None):
        super().__init__(parent)

        self.address = address
        self.port = port
        self.buffer_size = buffer_size

        self.listener_qt_handler = ListenerQtHandler()

        self.message_queue = deque()
        self.protocol: Optional[SwitchboardClientProtocol] = None
        self.loop: Optional[asyncio.AbstractEventLoop] = None
        self.loop_thread: Optional[threading.Thread] = None
        self.close_connection = False
        self.is_authenticated = False

        # TODO: Consider converting these delegates to Signals and sending dict

        self.disconnect_delegate = None

        self.command_accepted_delegate = None
        self.command_declined_delegate = None

        self.vcs_init_completed_delegate = None
        self.vcs_init_failed_delegate = None
        self.vcs_report_revision_completed_delegate = None
        self.vcs_report_revision_failed_delegate = None
        self.vcs_sync_completed_delegate = None
        self.vcs_sync_failed_delegate = None

        self.send_file_completed_delegate = None
        self.send_file_failed_delegate = None
        self.receive_file_completed_delegate = None
        self.receive_file_failed_delegate = None

        self.delegates: Dict[str, Optional[Callable[[Dict], None]]] = {
            "state": None,
            "get sync status": None,
            "live log data": None,
            "start live log stream": None,
            "stop live log stream": None,
        }

        self.async_msg_states: dict[uuid.UUID, dict] = {}

        self.last_activity = datetime.now()

    @property
    def server_address(self):
        if self.address:
            return (self.address, self.port)
        return None

    @property
    def is_connected(self):
        return self.protocol is not None

    def connect(self, address=None, *, timeout: Optional[float] = None):
        '''
        Initiates a connection.

        Calling this function starts the connection setup on a background
        thread. The caller should connect slots to the listener_qt_handler's
        "listener_connected" and "listener_connection_failed" signals to
        determine whether the connection was created successfully.
        '''
        self.disconnect()

        if address:
            self.address = address
        elif not self.address:
            LOGGER.debug('No address has been set. Cannot connect')
            self.listener_qt_handler.listener_connection_failed.emit(self)
            return False

        self.close_connection = False
        self.is_authenticated = False
        self.last_activity = datetime.now()

        self.listener_qt_handler.listener_connecting.emit(self)
        self.loop_thread = threading.Thread(target=self._loop_threadproc,
                                            kwargs={'timeout': timeout})
        self.loop_thread.start()

        return True

    def disconnect(self, unexpected=False, exception=None):
        if self.protocol and self.loop and (not self.loop.is_closed()):
            _, msg = message_protocol.create_disconnect_message()
            self.send_message(msg)
            self.loop.call_soon_threadsafe(self.protocol.close)

        self.close_connection = True

        if self.loop_thread:
            self.loop_thread.join()
            self.loop_thread = None

    def update_last_activity(self):
        now = datetime.now()
        delta_sec = (now - self.last_activity).total_seconds()
        self.last_activity = now
        if delta_sec > self.HITCH_THRESHOLD_SEC:
            LOGGER.warning(
                f'Hitch detected; {delta_sec:.1f} seconds since last keepalive'
                f' to {self.address}')

    def _loop_threadproc(self, **kwargs):
        try:
            _ = asyncio.run(self._async_main(**kwargs))
        except Exception as exc:
            self.listener_qt_handler.listener_connection_failed.emit(self)
            if self.disconnect_delegate:
                self.disconnect_delegate(unexpected=True, exception=exc)

        self.reader = None
        self.writer = None
        self.protocol = None
        self.loop = None
        self.is_authenticated = False

    async def _async_main(self, timeout: Optional[float]):
        LOGGER.info(f"Connecting to {self.address}:{self.port}")

        self.loop = asyncio.get_event_loop()

        configuration = QuicConfiguration(
            alpn_protocols=[SWITCHBOARD_ALPN],
            is_client=True)

        configuration.idle_timeout = timeout or self.IDLE_TIMEOUT_SEC

        # We deal exclusively with self-signed certificates in practice,
        # and depend trust-on-first-use and validating the fingerprint.
        configuration.verify_mode = ssl.CERT_NONE

        async with connect(
            self.address,
            self.port,
            configuration=configuration,
            create_protocol=SwitchboardClientProtocol,
        ) as protocol:
            self.protocol = cast(SwitchboardClientProtocol, protocol)
            self.protocol.client = self

            (self.reader, self.writer) = await protocol.create_stream()
            self.reader._limit = 10 * (2 ** 20)  # 10 MiB max message (default 64 KiB)
            message_recv_task = asyncio.create_task(self._message_recv_loop())

            self.is_authenticated = await self.protocol.authenticate()
            if self.is_authenticated:
                self.listener_qt_handler.listener_connected.emit(self)
                await asyncio.wait_for(message_recv_task, timeout=None)
            else:
                LOGGER.error('Authentication failed!')
                self.listener_qt_handler.listener_connection_failed.emit(self)

        if self.disconnect_delegate:
            self.disconnect_delegate(
                unexpected=(not self.close_connection),
                exception=None)

    async def _message_recv_loop(self):
        # Coroutine consumes bytes until we receive a message delimeter.
        read_task: Optional[asyncio.Task] = None
        while ((not self.close_connection) and
                (not self.protocol._closed.is_set())):

            if not read_task:
                read_task = asyncio.create_task(
                    self.reader.readuntil(b'\x00'))

            done, pending = await asyncio.wait(
                [read_task], timeout=self.SELECT_TIMEOUT_SEC)

            now = datetime.now()

            # Send keepalives
            delta = now - self.last_activity
            if delta.total_seconds() > self.KEEPALIVE_INTERVAL_SEC:
                _, msg = message_protocol.create_keep_alive_message()
                self.send_message(msg)

            # Expire timed out response futures
            for msg_id in list(self.async_msg_states.keys()):
                async_state = self.async_msg_states[msg_id]
                if async_state['timeout'] < now:
                    LOGGER.warning(f'Message timed out: {msg_id}')
                    async_state['future'].set_exception(TimeoutError)
                    self.async_msg_states.pop(msg_id)

            if read_task in done:
                try:
                    message_bytes = read_task.result()
                except asyncio.exceptions.IncompleteReadError:
                    LOGGER.error('Got EOF during message receive')
                    break

                read_task = None

                # route message to its assigned delegate
                message = message_protocol.decode_message(
                    message_bytes[:-1])  # trim null terminator

                try:
                    self.route_message(message)
                except Exception:
                    LOGGER.error(f"Error while parsing message: \n\n=== Traceback BEGIN ===\n{traceback.format_exc()}=== Traceback END ===\n")

        # If we exit the loop and there's a lingering read_task, clean up.
        if read_task is not None:
            read_task.cancel()

    def route_message(self, message):
        ''' Routes the received message to its delegate '''
        if delegate := self.delegates.get(message['command'], None):
            delegate(message)
            return

        message_id = uuid.UUID(message['id']) if 'id' in message else None

        # Command accepted/rejected is handled first
        if "command accepted" in message:
            if message['command accepted'] is True:
                if self.command_accepted_delegate:
                    self.command_accepted_delegate(message_id)
            else:
                if self.command_declined_delegate:
                    self.command_declined_delegate(message_id, message["error"])

            return

        if async_state := self.async_msg_states.get(message_id):
            async_state['future'].set_result(message)
            self.async_msg_states.pop(message_id)
            return

        # Legacy delegates. TODO: Deprecate
        if "vcs init complete" in message:
            if message['vcs init complete'] is True:
                if self.vcs_init_completed_delegate:
                    self.vcs_init_completed_delegate()
            else:
                if self.vcs_init_failed_delegate:
                    self.vcs_init_failed_delegate(message['error'])

        elif "vcs report revision complete" in message:
            if message['vcs report revision complete'] is True:
                if self.vcs_report_revision_completed_delegate:
                    self.vcs_report_revision_completed_delegate(message['revision'])
            else:
                if self.vcs_report_revision_failed_delegate:
                    self.vcs_report_revision_failed_delegate(message['error'])

        elif "vcs sync complete" in message:
            if message['vcs sync complete'] is True:
                if self.vcs_sync_completed_delegate:
                    self.vcs_sync_completed_delegate(message['revision'])
            else:
                if self.vcs_sync_failed_delegate:
                    self.vcs_sync_failed_delegate(message['error'])

        elif "send file complete" in message:
            if message['send file complete'] is True:
                if self.send_file_completed_delegate:
                    self.send_file_completed_delegate(message['destination'])
            else:
                if self.send_file_failed_delegate:
                    self.send_file_failed_delegate(message['destination'],
                                                   message['error'])

        elif "receive file complete" in message:
            if message['receive file complete'] is True:
                if self.receive_file_completed_delegate:
                    self.receive_file_completed_delegate(message['source'],
                                                         message['content'])
            else:
                if self.receive_file_failed_delegate:
                    self.receive_file_failed_delegate(message['source'],
                                                      message['error'])

        elif "start live log stream" in message:
            if message['start live log stream'] is True:
                LOGGER.debug("Live log stream started successfully")
            else:
                LOGGER.warning(f"Failed to start live log stream: {message.get('error', 'Unknown error')}")

        elif "stop live log stream" in message:
            if message['stop live log stream'] is True:
                LOGGER.debug("Live log stream stopped successfully")
            else:
                # This may happen when SBL auto-stops the stream when the UE process dies before Python tries to stop it.
                LOGGER.debug(f"Stop live log stream response: {message.get('error', 'Unknown error')}")

        elif "live log stream terminated" in message:
            if message['live log stream terminated'] is True:
                log_file_path = message.get('log_file_path', 'unknown')
                reason = message.get('reason', 'unknown reason')
                LOGGER.info(f"Live log stream terminated for {log_file_path}: {reason}")
            else:
                LOGGER.warning(f"Live log stream termination notification failed: {message.get('error', 'Unknown error')}")

        else:
            LOGGER.error(f'Unhandled message: {message}')
            raise ValueError

    def get_message_response(
        self,
        message_id: uuid.UUID,
        message_bytes: bytes,
        *,
        max_wait=IDLE_TIMEOUT_SEC,
        future: Optional[Future[dict[str, str]]] = None
    ) -> Future[dict[str, str]]:
        if self.loop and self.protocol and self.writer:
            if not future:
                future = self.loop.create_future()

            timeout = datetime.now() + timedelta(seconds=max_wait)

            assert message_id not in self.async_msg_states
            self.async_msg_states[message_id] = {
                'future': future,
                'timeout': timeout
            }

            self.send_message(message_bytes)
            return future
        else:
            raise ConnectionError

    def send_message(self, message_bytes):
        if self.loop and self.protocol and self.writer:
            if threading.current_thread() is not self.loop_thread:
                self.loop.call_soon_threadsafe(self.send_message,
                                               message_bytes)
                return

            LOGGER.message(f'Message: Sending ({self.address}): {message_bytes}')
            self.writer.write(message_bytes)
            self.update_last_activity()
        else:
            LOGGER.error(f'Message: Failed to send ({self.address}): '
                         f'{message_bytes}. No socket connected')

            if self.disconnect_delegate:
                self.disconnect_delegate(unexpected=True, exception=None)
