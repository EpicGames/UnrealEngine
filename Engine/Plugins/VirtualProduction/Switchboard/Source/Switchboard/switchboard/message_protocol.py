# Copyright Epic Games, Inc. All Rights Reserved.

import base64
import json
from typing import Optional
import uuid
from enum import IntFlag
from functools import reduce


class SyncStatusRequestFlags(IntFlag):
    ''' Bit flag to represent the sync status being requested.
    We support this granularity because certain sync status requests
    can be heavier on the system than others.
    It should match its counterpart's definition in SwitchboardTasks.h.
    '''
    Nothing = 0
    SyncTopos = 1 << 0
    MosaicTopos = 1 << 1
    FlipModeHistory = 1 << 2
    ProgramLayers = 1 << 3
    DriverInfo = 1 << 4
    Taskbar = 1 << 5
    PidInFocus = 1 << 6
    CpuUtilization = 1 << 7
    AvailablePhysicalMemory = 1 << 8
    GpuUtilization = 1 << 9
    GpuCoreClockKhz = 1 << 10
    GpuTemperature = 1 << 11

    def __contains__(self, flags):
        ''' Returns true if the given bit flag is set'''
        return (self & flags) == flags

    @classmethod
    def all(cls):
        ''' Returns in intance of this enum with all flags set'''
        return reduce(lambda x, y: x | y, cls)


def create_authenticate_message(
    *,
    jwt: Optional[str] = None,
    password: Optional[str] = None,
):
    assert jwt or password

    cmd_id = uuid.uuid4()
    message = {
        'command': 'authenticate',
        'id': str(cmd_id),
    }

    if jwt:
        message['jwt'] = jwt

    if password:
        message['password'] = password

        # TODO: for backward compatibility with pre-release 5.4, remove
        message['token'] = password

    message_json = json.dumps(message).encode() + b'\x00'
    return (cmd_id, message_json)


def create_start_process_message(
    prog_path: str,
    prog_args: str,
    prog_name: str,
    caller: str,
    working_dir: str = "",
    *,
    update_clients_with_stdout: bool = False,
    combined_stdout_stderr: bool = True,
    priority_modifier: int = 0,
    lock_gpu_clock: bool = False,
    hide: bool = False,
):
    cmd_id = uuid.uuid4()
    start_cmd = {
        'command': 'start',
        'id': str(cmd_id),
        'exe': prog_path,
        'args': prog_args,
        'name': prog_name,
        'caller': caller,
        'working_dir': working_dir,
        'bUpdateClientsWithStdout': update_clients_with_stdout,
        'bCombinedStdoutStderr': combined_stdout_stderr,
        'priority_modifier': priority_modifier,
        'bLockGpuClock': lock_gpu_clock,
        'bHide': hide,
    }

    message = json.dumps(start_cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_kill_process_message(program_id, soft_kill_timeout=None):
    cmd_id = uuid.uuid4()
    kill_cmd = {'command': 'kill', 'id': str(cmd_id), 'uuid': str(program_id)}
    if soft_kill_timeout is not None:
        kill_cmd['soft_kill_timeout'] = soft_kill_timeout
    message = json.dumps(kill_cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_vcs_init_message(provider, vcs_settings):
    cmd_id = uuid.uuid4()
    vcs_init_cmd = {'command': 'vcs init', 'id': str(cmd_id), 'provider': provider, 'vcs settings': vcs_settings}
    message = json.dumps(vcs_init_cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_vcs_report_revision_message(path):
    cmd_id = uuid.uuid4()
    vcs_revision_cmd = {'command': 'vcs report revision', 'id': str(cmd_id), 'path': path}
    message = json.dumps(vcs_revision_cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_vcs_sync_message(revision, path):
    cmd_id = uuid.uuid4()
    vcs_sync_cmd = {'command': 'vcs sync', 'id': str(cmd_id), 'revision': revision, 'path': path}
    message = json.dumps(vcs_sync_cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_disconnect_message():
    cmd_id = uuid.uuid4()
    disconnect_cmd = {'command': 'disconnect', 'id': str(cmd_id)}
    message = json.dumps(disconnect_cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_send_filecontent_message(file_content: bytes, destination_path: str, *, force_overwrite: bool = False):
    encoded_content = base64.b64encode(file_content)

    cmd_id = uuid.uuid4()
    transfer_file_cmd = {
        'command': 'send file',
        'id': str(cmd_id),
        'destination': destination_path,
        'content': encoded_content.decode(),
        'force_overwrite': force_overwrite
    }
    message = json.dumps(transfer_file_cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_send_file_message(path_to_source_file: str, destination_path: str, *, force_overwrite: bool = False):
    with open(path_to_source_file, 'rb') as f:
        file_content = f.read()
    return create_send_filecontent_message(file_content, destination_path, force_overwrite=force_overwrite)


def create_copy_file_from_listener_message(path_on_listener_machine):
    cmd_id = uuid.uuid4()
    copy_file_cmd = {'command': 'receive file', 'id': str(cmd_id), 'source': path_on_listener_machine}
    message = json.dumps(copy_file_cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_keep_alive_message():
    cmd_id = uuid.uuid4()
    keep_alive_cmd = {'command': 'keep alive', 'id': str(cmd_id)}
    message = json.dumps(keep_alive_cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_get_sync_status_message(program_id, request_flags: SyncStatusRequestFlags):
    cmd_id = uuid.uuid4()
    cmd = {
        'command': 'get sync status',
        'id': str(cmd_id),
        'uuid': str(program_id),
        'request_flags': int(request_flags),
        'bEcho': False
    }
    message = json.dumps(cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_refresh_mosaics_message():
    cmd_id = uuid.uuid4()
    cmd = {'command': 'refresh mosaics', 'id': str(cmd_id)}
    message = json.dumps(cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_redeploy_listener_message(base64listener: str, sha1digest: str):
    ''' Sends a command to replace the remote server's listener executable. '''
    cmd_id = uuid.uuid4()
    redeploy_cmd = {'command': 'redeploy listener', 'id': str(cmd_id), 'sha1': sha1digest, 'content': base64listener}
    message = json.dumps(redeploy_cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_free_listener_bin_message():
    ''' Sends a command to the listener to move its executable. '''
    cmd_id = uuid.uuid4()
    rename_proc_cmd = {'command': 'free binary', 'id': str(cmd_id)}
    message = json.dumps(rename_proc_cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_fixExeFlags_message(puuid):
    cmd_id = uuid.uuid4()
    cmd = {'command': 'fixExeFlags', 'id': str(cmd_id), 'uuid': str(puuid)}
    message = json.dumps(cmd).encode() + b'\x00'
    return (cmd_id, message)


def decode_message(msg_as_bytes: bytes):
    msg_as_str = msg_as_bytes.decode()
    msg_json = json.loads(msg_as_str)
    return msg_json


def create_minimize_windows_message():
    cmd_id = uuid.uuid4()
    cmd = {'command': 'minimize windows', 'id': str(cmd_id)}
    message = json.dumps(cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_set_inactive_timeout_message(timeout_seconds: int):
    cmd_id = uuid.uuid4()
    cmd = {'command': 'set inactive timeout', 'id': str(cmd_id),
           'seconds': timeout_seconds}
    message = json.dumps(cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_start_live_log_stream_message(log_file_path: str, tail_only: bool = False, process_uuid: str = None, start_offset: Optional[int] = None):
    ''' Creates the message to start SBL streaming the live log of the running Unreal/nDisplay process '''

    cmd_id = uuid.uuid4()
    cmd = {
        'command': 'start live log stream',
        'id': str(cmd_id),
        'log_file_path': log_file_path,
        'tail_only': tail_only
    }
    if process_uuid:
        cmd['process_uuid'] = process_uuid

    if start_offset is not None and start_offset >= 0:
        cmd['start_offset'] = start_offset

    message = json.dumps(cmd).encode() + b'\x00'
    return (cmd_id, message)


def create_stop_live_log_stream_message(log_file_path: str):
    ''' Creates the message to stop SBL streaming the live log of the running Unreal/nDisplay process. '''

    cmd_id = uuid.uuid4()
    cmd = {
        'command': 'stop live log stream',
        'id': str(cmd_id),
        'log_file_path': log_file_path
    }
    message = json.dumps(cmd).encode() + b'\x00'
    return (cmd_id, message)
