# Copyright Epic Games, Inc. All Rights Reserved.

from dataclasses import dataclass
from functools import wraps
import logging
import marshal
import pathlib
import subprocess
from typing import Optional

# NOTE: This file should not depend on Qt, directly or indirectly.

from . import switchboard_utils as sb_utils
from .devices.unreal.version_helpers import LISTENER_COMPATIBLE_VERSION


# Metadata attached to p4 commands for server analytics
meta_zprog = 'switchboard'
meta_zversion = '.'.join(str(x) for x in LISTENER_COMPATIBLE_VERSION)  # FIXME?


def p4_login(f):
    @wraps(f)
    def wrapped(*args, **kwargs):
        try:
            return f(*args, **kwargs)
        except Exception as e:
            logging.error(f'{repr(e)}')
            logging.error('Error running P4 command. Please make sure you are logged into Perforce and environment variables are set')
            return None

    return wrapped


@p4_login
def p4_latest_changelists(
    p4_paths: str | list[str],
    *,
    limit: int = 10,
    exclude_automation: bool = True,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> list[tuple[int, str]]:
    ''' Return (cl: int, description: str) for `limit` most recent CLs '''
    if isinstance(p4_paths, str):
        p4_paths = [p4_paths]

    changes = p4_changes(p4_paths, limit=limit, user=user, client=client)

    if exclude_automation:
        changes = filter(lambda x: x[b'user'] != b'buildmachine', changes)

    # TODO: Support UnrealGameSync.ini [Options] ExcludeChanges regexes?

    cl_desc_pairs = [(int(change[b'change']), change[b'desc'].decode())
                     for change in changes]

    return sorted(cl_desc_pairs, reverse=True)[:limit]


def run(cmd, args=[], input=None):
    ''' Runs the provided p4 command and arguments with -G python marshaling 
    Args:
        cmd(str): The p4 command to run. e.g. clients, where
        args(list): List of extra string arguments to include after cmd.
        input: Python object that you want to marshal into p4, if any.

    Returns:
        list: List of marshalled objects output by p4.
    '''

    c = "p4 -z tag -G " + cmd + " " + " ".join(args)

    p = subprocess.Popen(c, stdin=subprocess.PIPE, stdout=subprocess.PIPE, shell=True)

    if input:
        marshal.dump(input, p.stdin, 0)
        p.stdin.close()

    r = []

    try:
        while True:
            x = marshal.load(p.stdout)
            r = r + [ x ]
    except EOFError: 
        pass

    return r


def valueForMarshalledKey(obj: dict, key: str):
    ''' The P4 marshal is using bytes as keys instead of strings,
    so this makes the conversion and returns the desired value for the given key.

    Args:
        obj: The marshalled object, typically the element of a list returned by a p4 -G command.
        key(str): The key identifying the dict key desired from the object.
    '''
    return obj[key.encode('utf-8')].decode()


def hasValueForMarshalledKey(obj: dict, key: str):
    '''
    Checks whether a key exists.
    See valueForMarshalledKey.
    '''
    return key.encode('utf-8') in obj


def workspaceInPath(ws, localpath):
    ''' Validates if the give localpath can correspond to the given workspace.

    Args:
        localpath(str): Local path that we are checkings against
        ws: Workspace, as returned by p4 -G clients

    Returns:
        bool: True if the workspace is the same or a base folder of the give localpath
    '''
    localpath = pathlib.Path(localpath)
    wsroot = pathlib.Path(valueForMarshalledKey(ws,'Root'))

    return (wsroot == localpath) or (wsroot in localpath.parents)


def p4_from_localpath(localpath, workspaces, preferredClient):
    ''' Returns the first client and p4 path that matches the given local path.
    Normally you would pre-filter the workspaces by hostname.
    
    Args:
        localpath(str): The local path that needs to match the workspace
        workspaces(list): The list of candidate p4 workspaces to consider, as returned by p4 -G clients.
        preferredClient(str): Client to prefer, if there are multiple candidates.

    Returns:
        str,str: The workspace name and matching p4 path.
    '''

    # Normalize path. In particular, a trailing slash may not be accepted by the 'p4 where' command.
    localpath = pathlib.Path(localpath).as_posix()

    # Only take into account workspaces with the same give local path
    wss = [ws for ws in workspaces if workspaceInPath(ws, localpath)]

    # Having a single candidate ws is unambiguous, so we use it regardless of manual entry. 
    # But if we have more candidate workspaces, prefer manual entry and if none then pick the first candidate.
    if len(wss) == 0:
        raise FileNotFoundError
    elif len(wss) == 1:
        client = valueForMarshalledKey(wss[0],'client')
    else:
        client = next((valueForMarshalledKey(ws, 'client') for ws in wss if valueForMarshalledKey(ws, 'client')==preferredClient), preferredClient)
        if not client:
            client = valueForMarshalledKey(wss[0],'client')

    # now we need to determine the corresponding p4 path to the full engine path
    wheres = run(cmd=f'-c {client} where', args=[localpath])

    if not len(wheres):
        raise FileNotFoundError

    p4path = valueForMarshalledKey(wheres[0], 'depotFile')

    return client, p4path


##########


P4Env = dict[str, str]
_cached_p4_envs: dict[pathlib.Path, P4Env] = {}

def p4_get_env(path: pathlib.Path | None) -> P4Env:
    global _cached_p4_envs

    if path is None:
        path = pathlib.Path.cwd()

    env = _cached_p4_envs.get(path)
    if env is None:
        env = {}
        logging.debug(f'Querying `p4 set` for path: {path}')
        try:
            output = subprocess.check_output(['p4', 'set', '-q'], cwd=path).decode()
            for line in output.splitlines():
                var_name, _, var_value = line.partition('=')
                env[var_name] = var_value
        except Exception as exc:
            logging.exception(f'Querying `p4 set` failed with exception', exc_info=exc)

        _cached_p4_envs[path] = env

    return env

def p4_get_var(var: str, path: pathlib.Path | None) -> str | None:
    '''
    Look up Perforce variable (e.g. `P4CONFIG`, `p4 set`) key/value pairs.
    '''

    env = p4_get_env(path)
    return env.get(var)


def p4(
    cmd: str,
    cmd_opts: list[str],
    global_opts: Optional[list[str]] = None,
    *,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> subprocess.Popen:
    ''' Runs a p4 command and includes some common/required switches. '''
    args = ['p4', f'-zprog={meta_zprog}', f'-zversion={meta_zversion}',
            '-ztag', '-G', '-Qutf8']

    if port:
        args.extend(['-p', port])
    if user:
        args.extend(['-u', user])
    if client:
        args.extend(['-c', client])

    args.extend(global_opts or [])

    args.append(cmd)
    args.extend(cmd_opts)

    logging.debug(f'p4(): invoking subprocess: args={args}')

    return subprocess.Popen(args, stdout=subprocess.PIPE,
                            startupinfo=sb_utils.get_hidden_sp_startupinfo())


def p4_get_records(
    cmd: str,
    opts: list[str],
    global_opts: Optional[list[str]] = None,
    *,
    include_info: bool = False,
    include_error: bool = False,
    log_unreturned: bool = True,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> list[dict]:
    results: list[dict] = []
    proc = p4(cmd, opts, global_opts=global_opts, port=port, user=user, client=client)
    while True:
        try:
            record = marshal.load(proc.stdout)
        except EOFError:
            break

        # Possible codes: stat, info, error, text (only `p4 print`)
        if (record[b'code'] == b'info') and not include_info:
            if log_unreturned:
                logging.debug(f'p4_get_records: info: {record}')
            continue
        elif (record[b'code'] == b'error') and not include_error:
            if log_unreturned:
                logging.error(f'p4_get_records: error: {record}')
            continue

        results.append(record)

    proc.stdout.close()
    return results


def p4_get_decoded_records(
    cmd: str,
    opts: list[str],
    global_opts: Optional[list[str]] = None,
    *,
    include_info: bool = False,
    include_error: bool = False,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> list[dict]:
    records = p4_get_records(
        cmd, opts, global_opts, include_info=include_info, include_error=include_error,
        port=port, user=user, client=client)

    return [{k.decode(): v.decode() if isinstance(v, bytes) else v
             for k, v in record.items()}
            for record in records]


def p4_get_login(
    *,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> Optional[str]:
    ''' Returns the logged in user, or None if there's no valid ticket. '''
    records = p4_get_decoded_records('login', ['-s'],
                                     port=port, user=user, client=client)

    if records:
        return records[0].get('User')
    else:
        return None


def p4_get_client(
    get_client_name: str,
    *,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> Optional[dict]:
    records = p4_get_decoded_records('client', ['-o', get_client_name],
                                     port=port, user=user, client=client)

    if records:
        return records[0]
    else:
        return None


def p4_clients(
    query_user: str,
    *,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
):
    records = p4_get_decoded_records('clients', ['-u', query_user],
                                     port=port, user=user, client=client)

    clients: dict[str, dict] = {}
    for record in records:
        if record_client := record.get('client'):
            clients[record_client] = record
        else:
            logging.error(f'p4_clients: record missing "client" field: {record}')

    return clients


def p4_changes(
    pathspecs: list[str],
    *,
    limit: int = 10,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> list[dict]:
    limit = min(limit, 100)
    opts = ['-s', 'submitted', '-t', '-L', f'-m{limit}', *pathspecs]
    return p4_get_records('changes', opts, port=port, user=user, client=client)


def p4_fstat(
    pathspecs: list[str],
    *,
    limit: int = 0,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> list[dict]:
    opts: list[str] = []

    if limit > 0:
        opts.append(f'-m{limit}')

    opts.extend(pathspecs)

    return p4_get_records('fstat', opts, log_unreturned=False,
                          port=port, user=user, client=client)


def p4_have(
    pathspecs: list[str],
    *,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> list[dict]:
    records = p4_get_decoded_records('have', pathspecs, include_error=True,
                                     port=port, user=user, client=client)

    return records


@dataclass
class P4PrintResult:
    stat: dict
    contents: Optional[bytearray]

    @property
    def is_valid(self) -> bool:
        try:
            return ((self.stat[b'code'] == b'stat')
                    and (self.contents is not None))
        except KeyError:
            return False

    @property
    def text(self) -> Optional[str]:
        if self.contents is not None:
            return self.contents.decode()
        else:
            return None


def p4_print(
    pathspecs: list[str],
    *,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> list[P4PrintResult]:
    '''
    Returns a `P4PrintResult` for each path in `pathspecs`.
    If the file does not exist, or is deleted at the specified revision,
    fileContents will be `None`.
    '''
    results: list[P4PrintResult] = []
    records = p4_get_records('print', pathspecs, include_info=True, include_error=True,
                             port=port, user=user, client=client)

    # For each file, a stat record is returned, then zero or more text records.
    # Cases where there's no text record include errors or deleted files.
    for record in records:
        if (record[b'code'] == b'stat') or (record[b'code'] == b'error'):
            results.append(P4PrintResult(record, None))
            continue
        elif record[b'code'] == b'text':
            current_result = results[-1]
            if current_result.contents is None:
                current_result.contents = bytearray()
            current_result.contents.extend(record[b'data'])
            continue

        logging.warning(f'p4_print(): Unhandled record: {record}')

    return results


ALL_CODE_EXTS = [
    '.c', '.cc', '.cpp', '.inl', '.m', '.mm', '.rc', '.cs', '.csproj',
    '.h', '.hpp', '.usf', '.ush', '.uproject', '.uplugin', '.sln']


def p4_latest_code_change(
    paths: list[str],
    *,
    in_range: Optional[str] = None,
    exts: Optional[list[str]] = None,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> Optional[int]:
    '''
    Code CL determination compatible with UGS/precompiled binaries.
    See `WorkspaceUpdate.ExecuteAsync` in `WorkspaceUpdate.cs`
    '''
    if exts is None:
        exts = ALL_CODE_EXTS

    rangespec = f'@{in_range}' if in_range is not None else ''

    code_paths: list[str] = []
    for path in paths:
        if path.endswith('...'):
            code_paths.extend([f'{path}/*{ext}{rangespec}'
                               for ext in exts])

    code_changes = p4_changes(code_paths, limit=1,
                              port=port, user=user, client=client)

    if len(code_changes) > 0:
        return max(int(x[b'change']) for x in code_changes)
    else:
        return None
