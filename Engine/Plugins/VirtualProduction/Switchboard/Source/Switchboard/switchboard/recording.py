# Copyright Epic Games, Inc. All Rights Reserved.
import datetime
import time
from enum import IntEnum, unique, auto
import json
from .switchboard_logging import LOGGER
from .config import CONFIG, SETTINGS
from . import switchboard_utils
import threading
import random
import os
from collections import OrderedDict

from PySide6 import QtCore


class RecordingStatus(IntEnum):
    NO_DATA = auto()
    ON_DEVICE = auto()
    TRANSPORTING = auto()
    READY_FOR_PROCESSING = auto()
    PROCESSING = auto()
    COMPLETE = auto()


class RecordingManager(QtCore.QObject):
    """
    Manages all Recordings for a given Project/shoot
    The default sesssion is called 'Default'

    RecordingManager.current_recording gives back the recording that is currently taking place
    The current_recording can be added to the save stack
    """
    signal_recording_manager_saved = QtCore.Signal(object)

    def __init__(self, root_dir):
        super().__init__()

        # Cache slate/take combos so they cannot be reused
        self.slate_take_cache = {}

        self._recordings = {}
        self._latest_recording = None

        self._root_dir = root_dir

        # Find all recordings
        self.load_todays_recordings()

        # Store a list of recordings that need to be saved out
        self._save_stack = {}
        self.__auto_save_timer = None

    @property
    def latest_recording(self):
        return self._latest_recording

    @latest_recording.setter
    def latest_recording(self, value):
        self._latest_recording = value

    def load_todays_recordings(self):
        # Read Setups and Recordings from disk
        save_dir = self.save_dir()

        for dir_name, sub_dir_names, file_list in os.walk(save_dir):
            for file_name in file_list:
                file_path = os.path.join(dir_name, file_name)

                if not file_path.endswith('.json'):
                    continue

                with open(file_path) as f:
                    try:
                        data = json.load(f)
                        new_recording = Recording(data=data)
                        self.add_recording(new_recording)
                    except json.decoder.JSONDecodeError:
                        LOGGER.error(f'RecordingManager: Cannot read {file_path}')

    def save_dir(self, sequence=None):
        project_dir = os.path.join(self._root_dir, 'projects')
        os.makedirs(project_dir, exist_ok=True)

        date_string = switchboard_utils.date_to_string(datetime.date.today())

        if sequence:
            return os.path.join(project_dir, CONFIG.PROJECT_NAME.get_value(), date_string, sequence)

        return os.path.join(project_dir, CONFIG.PROJECT_NAME.get_value(), date_string)

    def add_recording(self, recording):
        # Add a new recording to the manager
        recording_key = self.recording_key(recording.sequence, recording.slate, recording.take)
        self._recordings[recording_key] = recording

        slate_take_key = self.recording_key(recording.sequence, recording.slate)
        self.slate_take_cache.setdefault(slate_take_key, []).append(recording.take)

        self.latest_recording = recording

    def add_device_to_recording(self, device_recording, recording):
        # Add a device to an existing recording
        recording.device_recordings.append(device_recording)

        # Start the auto save
        self.auto_save(recording)

    def slate_take_available(self, sequence, slate, take):
        # Return True/False if the slate/take have been used before
        try:
            return take not in self.slate_take_cache[self.recording_key(sequence, slate)]
        except:
            return True

    def auto_save(self, recording):
        if self.__auto_save_timer and self.__auto_save_timer.is_alive():
            self.__auto_save_timer.cancel()

        # Put the recording into the _save_stack if needed
        self._save_stack.setdefault(recording._hash, recording)

        # Start the save timer
        self.__auto_save_timer = threading.Timer(2, self._save)
        self.__auto_save_timer.start()

    def _save(self):
        # Loop through the save stack
        for recording in self._save_stack.values():
            recording_save_path = self.recording_save_path(recording)
            recording_save_dir = os.path.dirname(recording_save_path)

            # Build the dirs if needed
            os.makedirs(recording_save_dir, exist_ok=True)

            with open(f'{recording_save_path}', 'w') as f:
                json.dump(recording.data(), f, indent=4)

            self.signal_recording_manager_saved.emit(recording)

        # Clear the save stack
        self._save_stack = {}

    @staticmethod
    def recording_key(sequence, slate, take=None):
        return '|'.join([x.lower() for x in [sequence, slate, str(take)] if x])

    def recording_save_path(self, recording):
        # Given a recording, return the local save path
        dir_path = self.save_dir(sequence=recording.sequence)
        file_name = f'{recording.slate}_T{recording.take}.json'
        return os.path.join(dir_path, file_name)


class Recording():
    def __init__(self, data={}):
        self.project = data.get('project', None)  
        self.shoot = data.get('shoot', 'Default')       # "Season_11_w191102"
        self.sequence = data.get('sequence', 'Default')     # "Season_11_Trailer"
        self.slate_number = data.get('slate_number', None)
        self.map = data.get('map', None)
        self.slate = data.get('slate', None)
        self.take = data.get('take', None)
        self.description = data.get('description', None)
        self.date = data.get('date', None)
        self.changelist = data.get('changelist', None)
        self.multiuser_session = data.get('multiuser_session', None)
        self.multiuser_archive = data.get('multiuser_archive', None)
        self.circle_take = data.get('circle_take', False)
        self.ng = data.get('ng', False)
        self.parent_take = data.get('parent_take', None)

        self.device_recordings = []
        for device_recording_data in data.get('device_recordings', []):
            self.device_recordings.append(DeviceRecording(data=device_recording_data))

        self.assets = data.get('assets', [])

        self._hash = random.getrandbits(64)  # Store a unique identifier for a recording

    def data(self):
        data = {}
        data['project'] = self.project
        data['shoot'] = self.shoot
        data['sequence'] = self.sequence
        data['slate_number'] = self.slate_number
        data['slate'] = self.slate
        data['take'] = self.take
        data['description'] = self.description
        data['date'] = self.date
        data['map'] = self.map
        data['changelist'] = self.changelist
        data['multiuser_session'] = self.multiuser_session
        data['multiuser_archive'] = self.multiuser_archive
        data['circle_take'] = self.circle_take
        data['ng'] = self.ng
        data['parent_take'] = self.parent_take

        data['devices'] = []
        data['assets'] = []

        for device_recording in self.device_recordings:
            data['devices'].append(device_recording.data())

        return data


class DeviceRecording():
    """
    Devices return a DeviceRecording object that can be used to push into csv/json or shotgun 
    """
    def __init__(self, data={}):
        self.device_name = data.get('device_name', None)
        self.device_type = data.get('device_type', None)
        self.duration = data.get('duration', None)
        self.timecode_in = data.get('timecode_in', None)
        self.timecode_out = data.get('timecode_out', None)
        self.paths = data.get('paths', [])
        self.status = data.get('status', RecordingStatus.NO_DATA)

    def data(self):
        data = {}
        data['device_name'] = self.device_name
        data['device_type'] = self.device_type
        data['duration'] = self.duration
        data['timecode_in'] = self.timecode_in
        data['timecode_out'] = self.timecode_out
        data['paths'] = self.paths
        data['status'] = self.status.name

        return data


TRANSPORT_QUEUE_FILE_NAME = 'Transport_Queue.json'


class TransportQueue(QtCore.QObject):
    signal_transport_queue_job_started = QtCore.Signal(object)
    signal_transport_queue_job_finished = QtCore.Signal(object)

    def __init__(self, root_dir, num_concurrent_jobs=2):
        super().__init__()

        self._root_dir = root_dir

        self.transport_jobs = {}
        self.active_jobs = []

        self.load()

        self.num_concurrent_jobs = num_concurrent_jobs

    def active_queue_full(self):
        return len(self.active_jobs) >= self.num_concurrent_jobs

    def save(self):
        with open(f'{self.file_path()}', 'w') as f:
            data = [transport_job.data() for transport_job in self.transport_jobs.values()]
            json.dump(data, f, indent=4)

    def load(self):
        file_path = self.file_path()

        if not os.path.isfile(file_path):
            return

        with open(file_path) as f:
            try:
                data = json.load(f)
                for d in data:
                    job_name = self.job_name(d['slate'], d['take'], d['device_name'])
                    transport_job = TransportJob(d['job_name'], d['device_name'], d['slate'], d['take'], d['date'], d['paths'])
                    self.transport_jobs[job_name] = transport_job
            except json.decoder.JSONDecodeError:
                LOGGER.error(f'TransportQueue: Cannot read {file_path}')

    def add_transport_job(self, transport_job):
        self.transport_jobs[transport_job.job_name] = transport_job
        self.save()

        LOGGER.debug(f'add_transport_job {transport_job.job_name}')

    def run_transport_job(self, transport_job, device):
        # Add Sequence
        take_name = switchboard_utils.capture_name(transport_job.slate, transport_job.take)
        output_path = os.path.join(SETTINGS.TRANSPORT_PATH.get_value(), CONFIG.PROJECT_NAME.get_value(), transport_job.date, take_name)

        # Make sure the dirs exist
        os.makedirs(output_path, exist_ok=True)

        # Switch status to Transporting
        transport_job.transport_status = TransportStatus.TRANSPORTING

        # Add the jobs to the active jobs
        self.active_jobs.append(transport_job)

        # Start the thread
        transport_thread = threading.Thread(target=self._transport_file, args=[device, transport_job.paths, output_path, transport_job, transport_job.job_name])
        transport_thread.start()

    def _transport_file(self, device, device_paths, output_path, transport_job, job_name):
        LOGGER.debug('Start Job')
        self.signal_transport_queue_job_started.emit(transport_job)

        for path in device_paths:
            LOGGER.debug(f'run_transport_job {path}')
            device.transport_file(path, output_path)

        self._transport_complete(job_name)

    def _transport_complete(self, job_name):
        LOGGER.debug('End Job')
        transport_job = self.active_jobs[job_name]
        transport_job.transport_status = TransportStatus.COMPLETE

        self.signal_transport_queue_job_finished.emit(transport_job)

    def file_path(self):
        project_dir = os.path.join(self._root_dir, 'projects', CONFIG.PROJECT_NAME.get_value())
        os.makedirs(project_dir, exist_ok=True)

        return os.path.join(project_dir, TRANSPORT_QUEUE_FILE_NAME)

    @staticmethod
    def job_name(slate, take, device_name):
        return f'{switchboard_utils.capture_name(slate, take)} {device_name}'

    #return self.device.transport_file(self.device_path, self.output_path)


class TransportStatus(IntEnum):
    READY_FOR_TRANSPORT = auto()
    TRANSPORTING = auto()
    COMPLETE = auto()


class TransportJob():
    def __init__(self, job_name, device_name, slate, take, date, paths):
        self.job_name = job_name
        self.device_name = device_name
        self.slate = slate
        self.take = take
        self.paths = paths
        self.date = date
        self.transport_status = TransportStatus.READY_FOR_TRANSPORT

    def data(self):
        data = {}
        data['job_name'] = self.job_name
        data['device_name'] = self.device_name
        data['slate'] = self.slate
        data['take'] = self.take
        data['date'] = self.date
        data['paths'] = self.paths

        return data

