# Copyright Epic Games, Inc. All Rights Reserved.

import pathlib
import sqlite3
from typing import Iterable


ROOT_SBCACHE_PATH = pathlib.Path(__file__).parent.with_name('cache')


class Row:
    '''Base class for database rows in a table'''
    def __init__(self, id: int):
        self.id = id


class Project(Row):
    '''Represents a row in the Projects table'''
    def __init__(self, id: int, path: str):
        super().__init__(id)
        self.path = path


class Map(Row):
    '''Represents a row in the Maps table. Every map belongs to a project.'''
    def __init__(self, id: int, project: Project, gamepath: str):
        super().__init__(id)
        self.project = project
        self.gamepath = gamepath


class AssetType(Row):
    '''Represents a row in the AssetTypes table. Used to identify the type of Assets.
    The type refers to the class name of the asset.
    '''
    def __init__(self, id: int, classname: str):
        super().__init__(id)
        self.classname = classname


class Asset(Row):
    '''Represents a row in the Assets table. They all belong to a project, and have an AssetType.'''
    def __init__(
            self,
            id: int,
            project: Project,
            assettype: AssetType,
            gamepath: str,
            name: str,
            localpath: str):

        super().__init__(id)

        self.project = project
        self.gamepath = gamepath
        self.assettype = assettype
        self.name = name
        self.localpath = localpath


class SBCache:
    ''' Used to keep a local cache of data gleaned from projects but
    that can be shared between switchboard configs. Its purpose is to
    avoid repeated project content searches and ultimately speed up
    opening configs. It also helps keep the projects configs small.
    For example:
        * Maps in a project.
        * LiveLink presets and media profiles in a project.

    Schema:
        * Projects
            * ProjectId (primary key)
            * LocalPath
        * Maps
            * MapId  (primary key)
            * ProjectId
            * GamePath
        * AssetTypes
            * AssetTypeId (primary key)
            * ClassName (unique)
        * Assets
            * AssetId (primary key)
            * ProjectId
            * AssetTypeId
            * Name
            * GamePath
            * LocalPath
    '''

    version = (1, 0, 0)  # Major, Minor, Patch. Change when schema is updated.

    def __new__(cls):
        ''' Classic singleton pattern '''
        if not hasattr(cls, 'instance'):
            cls.instance = super(SBCache, cls).__new__(cls)
        return cls.instance

    def __init__(self):

        self.con = None

        try:
            self._connect()
            self._verifydb()
        except (sqlite3.DatabaseError, sqlite3.Error, ValueError):
            self._recreatedb()
            self._verifydb()

    def _recreatedb(self):
        ''' Deletes existing file and re-creates the database '''

        if self.con:
            self.con.close()

        try:
            self._dbfilepath().unlink()
        except FileNotFoundError:
            pass

        self._dbfilepath().parent.mkdir(parents=True, exist_ok=True)
        self._connect()
        self._initializedb()

    def _connect(self):
        ''' Connect to the database in the given path '''
        self.con = sqlite3.connect(str(self._dbfilepath()))

    def _initializedb(self):
        ''' Creates required tables '''

        # Write metadata
        self.con.execute('''
            CREATE TABLE IF NOT EXISTS Metadata (
                Major INTEGER NOT NULL,
                Minor INTEGER NOT NULL,
                Patch INTEGER NOT NULL
            )
            ''')

        # We will use the version to invalidate the cache
        self.con.execute('''
            INSERT INTO Metadata (Major, Minor, Patch)
            VALUES (?, ?, ?)
        ''', self.version)

        self.con.execute('''
            CREATE TABLE IF NOT EXISTS Projects (
                ProjectId INTEGER PRIMARY KEY,
                LocalPath TEXT NOT NULL
            )
        ''')

        self.con.execute('''
            CREATE TABLE IF NOT EXISTS Maps (
                MapId INTEGER PRIMARY KEY,
                GamePath TEXT NOT NULL,
                ProjectId INTEGER NOT NULL,
                FOREIGN KEY (ProjectId) REFERENCES Projects (ProjectId)
            )
        ''')

        # Create table for asset types
        self.con.execute('''
            CREATE TABLE IF NOT EXISTS AssetTypes (
                AssetTypeId INTEGER PRIMARY KEY,
                ClassName TEXT NOT NULL UNIQUE
            )
        ''')

        # Create table for assets
        self.con.execute('''
            CREATE TABLE IF NOT EXISTS Assets (
                AssetId INTEGER PRIMARY KEY,
                ProjectId INTEGER NOT NULL,
                AssetTypeId INTEGER NOT NULL,
                Name TEXT NOT NULL,
                GamePath TEXT NOT NULL,
                LocalPath TEXT NOT NULL,
                FOREIGN KEY (ProjectId) REFERENCES Projects (ProjectId),
                FOREIGN KEY (AssetTypeId) REFERENCES AssetTypes (AssetTypeId)
            )
        ''')

        self.con.commit()

    def _verifydb(self):
        ''' Makes sure that everything needed in the database exists '''

        # check the version
        cur = self.con.execute('''
            SELECT Major, Minor, Patch FROM Metadata
        ''')

        row = cur.fetchone()

        dbversion = (row[0], row[1], row[2])

        if dbversion != self.version:
            raise ValueError

    def _dbfilename(self) -> str:
        ''' Returns default filename for this database '''
        return "sbcache.db"

    def _dbfilepath(self) -> str:
        ''' Returns default filepath of this database '''
        return ROOT_SBCACHE_PATH / self._dbfilename()

    def query_or_create_project(self, filepath: str) -> Project:
        ''' Creates or returns the project with the given filepath'''

        # Check if the project exists in the database
        cur = self.con.execute('''
            SELECT ProjectId, LocalPath FROM Projects
            WHERE LocalPath = ?
        ''', (filepath,))

        row = cur.fetchone()

        if row:
            # Return the existing project
            return Project(row[0], row[1])
        else:
            # Insert a new project and return it
            cur = self.con.execute('''
                INSERT INTO Projects (LocalPath)
                VALUES (?)
            ''', (filepath,))

            self.con.commit()

            return Project(id=cur.lastrowid, path=filepath)

    def query_project(self, filepath: str) -> Project | None:
        ''' Returns the project with the given filepath. None if non-existent'''

        cur = self.con.execute('''
            SELECT ProjectId, LocalPath FROM Projects
            WHERE LocalPath = ?
        ''', (filepath,))

        row = cur.fetchone()

        if row:
            return Project(row[0], row[1])

        return None

    def query_maps(self, project: Project) -> list[Map]:
        ''' Returns a list with all the maps of the given project '''

        # Query the maps by the project id
        cur = self.con.execute('''
            SELECT MapId, GamePath FROM Maps
            WHERE ProjectId = ?
        ''', (project.id,))

        rows = cur.fetchall()

        # Convert the rows to Map objects
        maps = [Map(id=row[0], project=project, gamepath=row[1]) for row in rows]
        return maps

    def update_project_maps(self, maps: list[Map], project: Project):
        ''' Replaces all the maps of a project with those in the given list. '''

        # Delete the existing maps of the project
        self.con.execute('''
            DELETE FROM Maps
            WHERE ProjectId = ?
        ''', (project.id,))

        # Insert the new maps of the project
        self.con.executemany('''
            INSERT INTO Maps (GamePath, ProjectId)
            VALUES (?, ?)
        ''', [(map.gamepath, project.id) for map in maps])

        self.con.commit()

    def query_or_create_assettype(self, classname: str) -> AssetType:
        ''' Returns the asset type of the given classname, which should be unique.
        If it doesn't exist, then it creates it.
        '''

        # Normalize to class names without namespace to avoid duplicates.
        classname = self.normalize_classname(classname)

        # Check if the asset type exists in the database
        cur = self.con.execute('''
            SELECT AssetTypeId, ClassName FROM AssetTypes
            WHERE ClassName = ?
        ''', (classname,))

        row = cur.fetchone()

        if row:
            # Return the existing asset type
            return AssetType(row[0], row[1])
        else:
            # Insert a new asset type and return it
            cur = self.con.execute('''
                INSERT INTO AssetTypes (ClassName)
                VALUES (?)
            ''', (classname,))

            self.con.commit()

            return AssetType(id=cur.lastrowid, classname=classname)

    def normalize_classname(self, classname: str):
        ''' Sometimes classnames are fully namespaced, e.g. /Script/Module.Classname and sometimes they aren't.
        To avoid duplicate assets due to differing classnames, we opt to drop the namespace. Currently all interesting
        classes in Switchboard don't have namespace collisions.
        '''

        return classname.split('.')[-1]

    def query_assets_by_classname(self, project: Project, classnames: Iterable[str]) -> list[Asset]:
        ''' Returns a list of assets that are of any of the given classnames. '''

        # Convert the classnames to asset type ids
        assettypes = [self.query_or_create_assettype(classname) for classname in classnames]
        assettype_ids = [assettype.id for assettype in assettypes]

        # Query the assets by the project id and the asset type ids
        idstring = ','.join(['?'] * len(assettype_ids))
        cur = self.con.execute(
            f'''
            SELECT AssetId, AssetTypeId, GamePath, Name, LocalPath FROM Assets
            WHERE ProjectId = ? AND AssetTypeId IN ({idstring})
            ''',
            (project.id, *assettype_ids)
        )

        rows = cur.fetchall()

        # Convert the rows to Asset objects

        assetype_for_id = {assettype.id: assettype for assettype in assettypes}

        def AssetFromRow(row):
            assettype = assetype_for_id[row[1]]
            return Asset(
                id=row[0],
                project=project,
                assettype=assettype,
                gamepath=row[2],
                name=row[3],
                localpath=row[4])

        return [AssetFromRow(row) for row in rows]

    def update_project_assets(self, project: Project, assets: list[Asset]) -> None:
        ''' Updates all the assets of the given project. Since the project is specified in
        the arguments, the project property of the assets in the list is ignored.
        '''

        # Delete the existing assets of the project
        self.con.execute('''
            DELETE FROM Assets
            WHERE ProjectId = ?
        ''', (project.id,))

        # Insert the new assets of the project
        self.con.executemany('''
            INSERT INTO Assets (ProjectId, AssetTypeId, Name, GamePath, LocalPath)
            VALUES (?, ?, ?, ?, ?)
        ''', [(project.id, asset.assettype.id, asset.name, asset.gamepath, asset.localpath) for asset in assets])

        self.con.commit()

    def merge_project_assets(self, project: Project, assets: list[Asset]) -> None:
        ''' Inserts new assets into the Assets table.
        To resolve conflicts with existing ones (determined by the GamePath alone for the given project), 
        it updates the other fields. Pre-existing assets are left as they were.
        '''

        # ensure the unique index we need exists
        self.con.execute('''
            CREATE UNIQUE INDEX IF NOT EXISTS idx_assets_project_gamepath
            ON Assets(ProjectId, GamePath)
        ''')

        # bulk upsert
        self.con.executemany('''
            INSERT INTO Assets
            (ProjectId, AssetTypeId, Name, GamePath, LocalPath)
            VALUES (?, ?, ?, ?, ?)
            ON CONFLICT(ProjectId, GamePath) DO UPDATE
            SET AssetTypeId = excluded.AssetTypeId,
                Name        = excluded.Name,
                LocalPath   = excluded.LocalPath
        ''',
            [
                (
                    project.id,
                    asset.assettype.id,
                    asset.name,
                    asset.gamepath,
                    asset.localpath
                )
                for asset in assets
            ]
        )

        self.con.commit()
