# Copyright Epic Games, Inc. All Rights Reserved.

import os
import pathlib
from typing import Dict, Optional, Callable

from PySide6 import QtCore
from PySide6 import QtGui
from PySide6 import QtWidgets

from switchboard import config
from switchboard import switchboard_widgets as sb_widgets
from switchboard.config import CONFIG, SETTINGS, Config, UserSettings, Setting
from switchboard.settings_search import SettingsSearch
from switchboard.switchboard_widgets import CollapsibleGroupBox, DropDownMenuComboBox
from switchboard.ui.horizontal_tabs import HorizontalTabWidget
from switchboard.devices.device_base import Device

RELATIVE_PATH = os.path.dirname(__file__)


def clear_widgets(layout):
    for i in range(layout.count()):
        layout.takeAt(0)


COLLAPSE_ALL_TEXT = "Collapse all"
EXPAND_ALL_TEXT = "Expand all"


class CreateUISearchUpdater:
    ''' Updates the search based on the progress of the settings batch manager
    that does the work enqueued via its create_ui method.
    '''

    MIN_SETTINGS_ADDED_PER_UPDATE = 50

    def __init__(
            self,
            batch_completed_signal: QtCore.Signal,
            search_settings_fn: Callable):
        '''
        Initializes the search updater that subscribes to the batch manager's signal.

        Args:
            batch_completed_signal (int, bool): 
                Progress signal emitted by the batch manager.
                The first argument (int) is how many settings were added in this batch.
                The second argument (bool) is True if there is still work pending.
            search_settings_fn : The function to call to update the search.
        '''
        if batch_completed_signal is None:
            raise ValueError("batch_manager cannot be None")

        if search_settings_fn is None:
            raise ValueError("search_settings_fn cannot be None")

        self.search_settings_fn = search_settings_fn
        self.batch_completed_signal = batch_completed_signal
        self.settings_added_since_last_search_update = 0

        # Connect the batch manager's signal to the search update function
        self.batch_completed_signal.connect(self.update_search_results)

    def update_search_results(self, settings_added: int, has_pending_work: bool):
        '''
        Updates the search results incrementally as new settings are added.
        Signature matches batch_completed_signal.

        Args:
            settings_added : Number of settings added during the batch.
            has_pending_work : True if there is more pending work, False otherwise.
        '''

        self.settings_added_since_last_search_update += settings_added

        if not has_pending_work or self.settings_added_since_last_search_update > self.MIN_SETTINGS_ADDED_PER_UPDATE:
            # Reset the count
            self.settings_added_since_last_search_update = 0

            # Trigger the search
            self.search_settings_fn()  # None is not allowed by our constructor.

    def __del__(self):
        if self.batch_completed_signal and self.update_search_results:
            self.batch_completed_signal.disconnect(self.update_search_results)    


class SettingsDialog(QtCore.QObject):
    def __init__(self, settings: UserSettings, in_config: Config):
        super().__init__()
        self.plugin_widgets = {}
        self.config = in_config

        # Set the UI object
        self.ui = QtWidgets.QDialog()
        self.ui.resize(800, 800)
        dialog_layout = QtWidgets.QVBoxLayout(self.ui)
        dialog_layout.setContentsMargins(2, 2, 2, 2)
        self.ui.setWindowTitle("Settings")
        self.ui.finished.connect(self._on_finished)

        self._create_search_area(dialog_layout)
        self.general_settings_list = [
            self._create_config_path_settings(),
            self._create_switchboard_settings(settings, self.config),
            self._create_project_settings(self.config),
            self._create_multi_user_server_settings(self.config)
        ]

        self._create_tab_widget(dialog_layout)

        self.settings_search = SettingsSearch([self.ui.all_settings_scroll_area, self.ui.general_settings_scroll_area])
        self.ui.searchBar.textChanged.connect(self._on_search_text_edited)

        # Because the batch manager will be incrementally adding setting widgets, we need to keep updating
        # the search results until it is done.

        def update_search_fn():
            '''Helper function to only conduct the search when needed'''
            if len(self.ui.searchBar.text()) > 0:
                self._search_settings_again()

        batch_completed_signal = Setting.get_create_ui_batch_manager().signal_batch_completed
        self.search_updater = CreateUISearchUpdater(batch_completed_signal, update_search_fn)

    def select_all_tab(self):
        self._on_tab_changed(0)

    def _on_tab_changed(self, index: int):
        """
        Because of the All category, widgets need to be re-parented constantly.
        """
        clear_widgets(self.ui.all_settings_scroll_area.layout())
        clear_widgets(self.ui.general_settings_scroll_area.layout())

        is_all_category = index == 0
        if is_all_category:
            for general_setting_widget in self.general_settings_list:
                self.ui.all_settings_scroll_area.layout().addWidget(general_setting_widget)

            for (group_box, scroll_bar) in self.plugin_widgets.values():
                scroll_bar.takeWidget()
                self.ui.all_settings_scroll_area.layout().addWidget(group_box)

            # Pull all widgets up so they do not attempt to divide space among themselves when filtered by search
            self.ui.all_settings_scroll_area.layout().addItem(
                QtWidgets.QSpacerItem(20, 40, QtWidgets.QSizePolicy.Minimum, QtWidgets.QSizePolicy.Expanding)
            )

        # General settings may have been in All
        is_general_category = index == 1
        if is_general_category:
            for general_setting_widget in self.general_settings_list:
                self.ui.general_settings_scroll_area.layout().addWidget(general_setting_widget)
            # Pull all widgets up so they do not attempt to divide space among themselves when filtered by search
            self.ui.general_settings_scroll_area.layout().addItem(
                QtWidgets.QSpacerItem(20, 40, QtWidgets.QSizePolicy.Minimum, QtWidgets.QSizePolicy.Expanding)
            )

        # Add the plugin back to its tabs - it may have been in All
        is_plugin = not is_all_category and not is_general_category
        if is_plugin:
            plugin_name = self.ui.tab_widget.tabText(index)
            (group_box, scroll_bar) = self.plugin_widgets[plugin_name]
            is_plugin_widget_missing = scroll_bar.widget() is None
            if is_plugin_widget_missing:
                scroll_bar.setWidget(group_box)

        self._search_settings_again()
        return

    def _on_finished(self, result: int):
        self.config.P4_ENABLED.signal_setting_changed.disconnect(self._on_source_control_setting_changed)

        # Currently, the only way to dismiss the settings dialog is by using
        # the close button as opposed to ok/cancel buttons, so we intercept the
        # close to issue a warning if the config path was changed and we're
        # about to overwrite some other existing config file.
        if (self._changed_config_path in self._config_paths
                and self._changed_config_path != self._current_config_path):
            # Show the confirmation dialog using a relative path to the config.
            rel_config_path = config.CONFIG_MGR.get_relative_config_path(self._changed_config_path)
            reply = QtWidgets.QMessageBox.question(
                self.ui, 'Confirm Overwrite',
                ('Are you sure you would like to change the config path and '
                 f'overwrite the existing config file "{rel_config_path}"?'),
                QtWidgets.QMessageBox.Yes, QtWidgets.QMessageBox.No)

            if reply != QtWidgets.QMessageBox.Yes:
                # Clear the changed config path so that the dialog returns the
                # originally specified path when queried via config_path().
                self._changed_config_path = None

        # Null the search updater to break any potential circular references.
        self.search_updater = None

    def _search_settings_again(self):
        self._on_search_text_edited(self.ui.searchBar.text())

    def _on_search_text_edited(self, search_string: str):
        self.settings_search.search(search_string)

    def _create_search_area(self, dialog_layout):
        self.ui.search_area = QtWidgets.QWidget()
        search_layout = QtWidgets.QHBoxLayout(self.ui.search_area)
        search_layout.setContentsMargins(1, 1, 1, 1)

        self.ui.searchBar = QtWidgets.QLineEdit()
        self.ui.searchBar.setPlaceholderText("Search")
        search_layout.addWidget(self.ui.searchBar)

        pixmap = QtGui.QPixmap(":icons/images/view_button.png")
        self.ui.view_options = DropDownMenuComboBox(QtGui.QIcon(pixmap))
        self.ui.view_options.addItem(COLLAPSE_ALL_TEXT)
        self.ui.view_options.addItem(EXPAND_ALL_TEXT)

        self.ui.view_options.on_select_option.connect(self._on_view_option_selected)
        search_layout.addWidget(self.ui.view_options)
        dialog_layout.addWidget(self.ui.search_area)

    def _on_view_option_selected(self, selected_item):
        if selected_item == COLLAPSE_ALL_TEXT:
            self._set_categories_expanded(False)
        elif selected_item == EXPAND_ALL_TEXT:
            self._set_categories_expanded(True)

    def _set_categories_expanded(self, should_expand: bool):
        def _set_expanded_recursive(widget_or_layout, should_expand: bool):
            if isinstance(widget_or_layout, CollapsibleGroupBox):
                widget_or_layout.set_expanded(should_expand)

            layout = widget_or_layout
            if isinstance(widget_or_layout, QtWidgets.QWidget):
                if isinstance(widget_or_layout, QtWidgets.QScrollArea):
                    layout = widget_or_layout.widget().layout()
                else:
                    layout = widget_or_layout.layout()

            if layout is None:
                return

            for i in range(layout.count()):
                layout_item = layout.itemAt(i)
                if layout_item.widget():
                    _set_expanded_recursive(layout_item.widget(), should_expand)
                elif layout_item.layout():
                    _set_expanded_recursive(layout_item.layout(), should_expand)

        active_tab_content = self.ui.tab_widget.currentWidget()
        _set_expanded_recursive(active_tab_content, should_expand)

    def _create_config_path_settings(self):
        self.ui.config_path_layout = QtWidgets.QWidget()
        layout = QtWidgets.QHBoxLayout(self.ui.config_path_layout)

        self.ui.config_path_label = QtWidgets.QLabel()
        self.ui.config_path_label.setText("Config Path")

        self.ui.config_path_line_edit = QtWidgets.QLineEdit()
        self.ui.config_path_line_edit.textChanged.connect(
            self.config_path_text_changed)

        layout.addWidget(self.ui.config_path_label)
        layout.addWidget(self.ui.config_path_line_edit)

        # Store the current config paths so we can warn about overwriting an existing config.
        self._config_paths = config.CONFIG_MGR.list_config_paths()
        self.set_config_path(SETTINGS.CONFIG)

        return self.ui.config_path_layout

    def _create_switchboard_settings(self, settings: UserSettings, in_config: Config):
        self.ui.switchboard_settings_group = CollapsibleGroupBox()
        self.ui.switchboard_settings_group.setTitle("Switchboard")
        layout = QtWidgets.QVBoxLayout(self.ui.switchboard_settings_group)

        settings_dict = {
            "address": settings.ADDRESS,
            "transport_path": settings.TRANSPORT_PATH,
            "listener_exe": in_config.LISTENER_EXE,
            "sblhelper_exe": in_config.SBLHELPER_EXE,
        }

        self.ui.switchboard_settings_sub_group = QtWidgets.QGroupBox()
        form_layout = QtWidgets.QFormLayout(self.ui.switchboard_settings_sub_group)
        layout.addWidget(self.ui.switchboard_settings_sub_group)
        self._create_settings_section(settings_dict, form_layout)

        return self.ui.switchboard_settings_group

    def _create_project_settings(self, in_config: Config):
        self.ui.project_settings_group = CollapsibleGroupBox()
        self.ui.project_settings_group.setTitle("Project Settings")
        layout = QtWidgets.QVBoxLayout(self.ui.project_settings_group)

        self._create_basic_settings(in_config, layout)
        self._create_osc_settings(in_config, layout)
        self._create_source_control_settings(in_config, layout)

        return self.ui.project_settings_group

    def _create_basic_settings(self, in_config: Config, layout: QtWidgets.QFormLayout):
        self.ui.basic_settings_group = QtWidgets.QGroupBox()
        form_layout = QtWidgets.QFormLayout(self.ui.basic_settings_group)
        layout.addWidget(self.ui.basic_settings_group)
        self._create_settings_section(in_config.basic_project_settings, form_layout)

    def _create_osc_settings(self, in_config: Config, layout: QtWidgets.QFormLayout):
        self.ui.osc_settings_group = QtWidgets.QGroupBox("OSC")
        form_layout = QtWidgets.QFormLayout(self.ui.osc_settings_group)
        layout.addWidget(self.ui.osc_settings_group)
        self._create_settings_section(in_config.osc_settings, form_layout)

    def _create_source_control_settings(self, in_config: Config, layout: QtWidgets.QFormLayout):
        self.ui.source_control_settings_group = QtWidgets.QGroupBox("Source Control")
        self.ui.source_control_settings_group.setCheckable(True)
        self.ui.source_control_settings_group.setStyleSheet(
            'QGroupBox::indicator:checked:hover {image: url(:icons/images/check_box_checked_hovered.png);}'
            'QGroupBox::indicator:checked {image: url(:icons/images/check_box_checked.png);}'
            'QGroupBox::indicator:unchecked:hover {image: url(:icons/images/check_box_hovered.png);}'
            'QGroupBox::indicator:unchecked {image: url(:icons/images/check_box.png);}'
        )
        form_layout = QtWidgets.QFormLayout(self.ui.source_control_settings_group)
        layout.addWidget(self.ui.source_control_settings_group)

        # Source control setting UI is fully enabled / disabled depending on P4_ENABLED
        self.ui.source_control_settings_group.setChecked(in_config.P4_ENABLED.get_value())
        self.ui.source_control_settings_group.toggled.connect(self._on_group_source_control_changed)
        in_config.P4_ENABLED.signal_setting_changed.connect(self._on_source_control_setting_changed)
        settings_to_show = {
            key: setting for (key, setting) in in_config.source_control_settings.items()
            if setting != in_config.P4_ENABLED
        }

        self._create_settings_section(settings_to_show, form_layout)

    def _on_group_source_control_changed(self):
        CONFIG.P4_ENABLED.update_value(self.ui.source_control_settings_group.isChecked())
        CONFIG.save()

    def _on_source_control_setting_changed(self, old_value: bool, new_value: bool):
        self.ui.source_control_settings_group.setChecked(new_value)

    def _create_multi_user_server_settings(self, in_config: Config):
        self.ui.multi_user_settings = CollapsibleGroupBox()
        self.ui.multi_user_settings.setTitle("Multi User Server")
        layout = QtWidgets.QVBoxLayout(self.ui.multi_user_settings)

        self.ui.multi_user_settings_sub_group = QtWidgets.QGroupBox()
        form_layout = QtWidgets.QFormLayout(self.ui.multi_user_settings_sub_group)
        layout.addWidget(self.ui.multi_user_settings_sub_group)
        self._create_settings_section(in_config.mu_settings, form_layout)

        return self.ui.multi_user_settings

    def _create_settings_section(self, dict: Dict[str, Setting], layout: QtWidgets.QFormLayout):
        setting_list = [setting for _, setting in dict.items()]
        for setting in setting_list:
            setting.create_ui(override_device_name=None, form_layout=layout)

    def _create_tab_widget(self, parent_layout):
        self.ui.tab_widget = HorizontalTabWidget()
        self.ui.tab_widget.setTabPosition(QtWidgets.QTabWidget.West)
        parent_layout.addWidget(self.ui.tab_widget)

        # All tab
        self.ui.all_tab_root = QtWidgets.QScrollArea()
        self.ui.all_tab_root.setWidgetResizable(True)
        self.ui.all_tab_root.setSizeAdjustPolicy(QtWidgets.QAbstractScrollArea.AdjustToContents)
        self.ui.all_tab_root.setSizePolicy(QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Expanding)

        self.ui.all_settings_scroll_area = QtWidgets.QWidget()
        all_layout = QtWidgets.QVBoxLayout(self.ui.all_settings_scroll_area)
        all_layout.setContentsMargins(2, 2, 2, 2)
        self.ui.all_tab_root.setWidget(self.ui.all_settings_scroll_area)

        # General tab
        self.ui.general_tab_root = QtWidgets.QScrollArea()
        self.ui.general_tab_root.setWidgetResizable(True)
        self.ui.general_tab_root.setSizeAdjustPolicy(QtWidgets.QAbstractScrollArea.AdjustToContents)
        self.ui.general_tab_root.setSizePolicy(QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Expanding)

        self.ui.general_settings_scroll_area = QtWidgets.QWidget()
        general_layout = QtWidgets.QVBoxLayout(self.ui.general_settings_scroll_area)
        general_layout.setContentsMargins(2, 2, 2, 2)
        self.ui.general_tab_root.setWidget(self.ui.general_settings_scroll_area)

        # Register tabs after all widgets have been allocated
        self.ui.tab_widget.addTab(self.ui.all_tab_root, "All")
        self.ui.tab_widget.addTab(self.ui.general_tab_root, "General")
        # Register callback last because addTab should not trigger our callback before everything is initialized
        self.ui.tab_widget.currentChanged.connect(
            self._on_tab_changed
        )

    def config_path(self):
        if self._changed_config_path:
            return self._changed_config_path

        return self._current_config_path

    def set_config_path(self, config_path: pathlib.Path):
        self._current_config_path = config_path
        self._changed_config_path = None

        if not config_path:
            self._changed_config_path = config.Config.DEFAULT_CONFIG_PATH
            config_path_str = self._changed_config_path.stem
        else:
            # prefer relative path when possible
            try:
                config_path_str = str(
                    config.CONFIG_MGR.get_relative_config_path(config_path).with_suffix(''))
            except ValueError:
                config_path_str = str(config_path)

        self.ui.config_path_line_edit.setText(config_path_str)

    def config_path_text_changed(self, config_path_str):
        config_path = config.Config.DEFAULT_CONFIG_PATH

        sb_widgets.set_qt_property(
            self.ui.config_path_line_edit, "input_error", False)

        try:
            config_path = config.CONFIG_MGR.get_absolute_config_path(config_path_str)
        except Exception as e:
            sb_widgets.set_qt_property(
                self.ui.config_path_line_edit, "input_error", True)

            rect = self.ui.config_path_line_edit.parent().mapToGlobal(
                self.ui.config_path_line_edit.geometry().topRight())
            QtWidgets.QToolTip().showText(rect, str(e))

        self._changed_config_path = config_path

    def add_plugin_settings_widgets_to_layout(
            self,
            plugin_cls: Device,
            plugin_settings: list[Setting],
            layout: QtWidgets.QLayout,
            stylevariant: str = '',
            device_name: Optional[str] = None,
            instance_settings: list[Setting] | None = None,
    ):
        ''' Organizes plugin_settings into categories group boxes and adds them to the given layout'''

        if instance_settings is None:
            instance_settings = []

        all_settings = instance_settings + plugin_settings

        scategories: dict[str, list[Setting]] = {}

        for setting in all_settings:

            # Don't create categories for hidden properties, or you might end up with an empty category
            if not setting.show_ui:
                continue

            category = setting.category

            if category not in scategories:
                scategories[category] = []

            scategories[category].append(setting)

        # sort the categories per the preference of the plugin
        scategories = plugin_cls.sort_setting_categories(scategories)

        # Create a group box for each category and add it to the plugin layout
        for category, settings in scategories.items():

            scategory_group_box = QtWidgets.QGroupBox()
            scategory_group_box.setTitle(category)
            scategory_group_box.setLayout(QtWidgets.QVBoxLayout())
            scategory_group_box.setSizePolicy(
                QtWidgets.QSizePolicy(QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Maximum)
            )

            # Alter the style as necessary to discern group boxes inside group boxes
            scategory_group_box.setProperty('stylevariant', stylevariant)

            scategory_layout = QtWidgets.QFormLayout()
            scategory_group_box.layout().addLayout(scategory_layout)

            # Add widgets for each setting in this category
            for setting in settings:
                # Instance settings don't specify override_device_name
                override_device_name = device_name if setting in plugin_settings else None
                setting.create_ui(form_layout=scategory_layout, override_device_name=override_device_name)

            # Add the category group box to the main plugin layout
            layout.addWidget(scategory_group_box)

    # Devices
    def add_section_for_plugin(
        self,
        plugin_name: str,
        plugin_cls: Device,
        plugin_settings: list[Setting],
        device_settings: list[tuple[str, list[Setting], list[Setting]]]
    ):

        any_device_settings = (
            any([device[1] for device in device_settings]) or
            any([device[2] for device in device_settings]))

        if not any_device_settings:
            return  # no settings to show

        # Create a group box per plugin
        plugin_group_box = CollapsibleGroupBox()
        plugin_group_box.setTitle(f'{plugin_name} Settings')
        plugin_group_box.setLayout(QtWidgets.QVBoxLayout())
        plugin_group_box.setSizePolicy(
            QtWidgets.QSizePolicy(QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Maximum)
        )

        # Need to manually create scroll bar for tab
        scroll_bar = QtWidgets.QScrollArea()
        scroll_bar.setWidget(plugin_group_box)
        scroll_bar.setWidgetResizable(True)
        scroll_bar.setSizeAdjustPolicy(QtWidgets.QAbstractScrollArea.AdjustIgnored)
        scroll_bar.setSizePolicy(QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Expanding)
        scroll_layout = QtWidgets.QVBoxLayout(scroll_bar)
        scroll_layout.setContentsMargins(1, 1, 1, 1)

        self.ui.tab_widget.addTab(scroll_bar, f'{plugin_name}')
        self.plugin_widgets[plugin_name] = (plugin_group_box, scroll_bar)

        # Add the device widget (contains all widgets and layouts for the given plugin)
        # to the list of widgets to be searched for matches. There should not be a need
        # to add child widgets since that would cause double searches.
        self.settings_search.searched_widgets.append(plugin_group_box)

        # Add a layout to the plugin group box. It will contain the plugin settings and the
        # device intances and their settings.
        plugin_group_box.layout().addLayout(QtWidgets.QVBoxLayout())

        self.add_plugin_settings_widgets_to_layout(
            plugin_cls=plugin_cls,
            plugin_settings=plugin_settings,
            layout=plugin_group_box.layout())

        # add widgets for settings and overrides of individual devices
        for device_name, settings, overrides in device_settings:

            # Each device instance will have its own collapsible group box
            group_box = CollapsibleGroupBox()
            group_box.setTitle(device_name)
            group_box.setLayout(QtWidgets.QVBoxLayout())

            plugin_group_box.layout().addWidget(group_box)

            layout = QtWidgets.QVBoxLayout()
            group_box.layout().addLayout(layout)

            self.add_plugin_settings_widgets_to_layout(
                plugin_cls=plugin_cls,
                plugin_settings=overrides,
                layout=layout,
                stylevariant='device_override',
                device_name=device_name,
                instance_settings=settings,
            )

        # Pull all widgets up so they do not attempt to divide space among themselves when filtered by search
        plugin_group_box.layout().addItem(
            QtWidgets.QSpacerItem(0, 0, QtWidgets.QSizePolicy.Minimum, QtWidgets.QSizePolicy.Expanding)
        )
