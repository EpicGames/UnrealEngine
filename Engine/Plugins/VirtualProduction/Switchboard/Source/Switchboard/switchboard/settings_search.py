# Copyright Epic Games, Inc. All Rights Reserved.

from PySide6 import QtWidgets
from typing import List, Dict

from switchboard.switchboard_logging import LOGGER
from switchboard.switchboard_widgets import CollapsibleGroupBox


def as_widget_or_layout(row_item):
    return row_item.widget() if row_item.widget() else row_item.layout()


class FormLayoutRow:
    ''' Item with the information about a form layout row. Managed by FormLayoutRowVisibility '''
    def __init__(self, label: QtWidgets.QWidget, field: QtWidgets.QWidget, visible: bool):
        self.label = label
        self.field = field
        self.visible = visible


class FormLayoutRowVisibility:
    """
    Handles the showing and hiding of rows in QFormLayout

    Row widgets are removed when hiding and added back when showing instead of calling setVisible on the row's widgets:
    Even if a row only contains invisible widgets, QFormLayout.layoutVerticalSpacing still affects them and visually 
    shifts visible rows around.
    """

    def __init__(self, form: QtWidgets.QFormLayout):
        self.form = form
        # Save the original widget order so we can preserve it when adding hidden widgets back

        # Keep track of visible rows. Will help us quickly determine if there is a count mismatch
        # which would indicate that the cache should be updated.
        self.visible_rows_count = 0

        self.rows: list[FormLayoutRow] = []  # Only modify with modifier functions, to keep track of statistics.

        for i in range(form.rowCount()):
            label = form.itemAt(i, QtWidgets.QFormLayout.LabelRole)
            field = form.itemAt(i, QtWidgets.QFormLayout.FieldRole)
            if label and field:
                row = FormLayoutRow(as_widget_or_layout(label), as_widget_or_layout(field), True)
                self._append_to_rows(row)

    def update_rows_cache(self):
        '''
        Updates the internal row cache to include new rows added to the form.
        We expect SettingsBatchManagerToCreateUI to incrementally add rows to form layouts.
        We do not expect or support rows being removed from the form layout.
        '''
        for rowIdx in range(self.form.rowCount()):
            label = as_widget_or_layout(self.form.itemAt(rowIdx, QtWidgets.QFormLayout.LabelRole))
            field = as_widget_or_layout(self.form.itemAt(rowIdx, QtWidgets.QFormLayout.FieldRole))

            found = False

            for cached_row in self.rows:
                if cached_row.label is label and cached_row.field is field:
                    found = True
                    break

            # If we didn't find a corresponding entry, then we should add it to the internal row cache.
            if not found:
                self._append_to_rows(FormLayoutRow(label, field, True))

        # sanity check that the cache is in sync.
        if self.has_form_added_rows_externally():
            LOGGER.error("FormLayoutRowVisibility's row cache was updated but differences were still detected.")

    def has_form_added_rows_externally(self):
        ''' Since visible rows should still be in the layout, the number of visible rows 
        should match the number of rows in the form layout unless more rows were added externally'''
        return self.form.rowCount() != self.visible_rows_count

    def _append_to_rows(self, row: FormLayoutRow) -> None:
        ''' Appends to the internal row cache, keeping track of visible row count.'''
        self.rows.append(row)
        if row.visible:
            self.visible_rows_count += 1

    def _replace_row(self, index: int, row: FormLayoutRow) -> None:
        ''' Replaces an item of internal row cache, adjusting the visible row count.'''
        old_visible = self.rows[index].visible
        self.rows[index] = row

        # update visible count
        if not old_visible and row.visible:
            self.visible_rows_count += 1
        elif old_visible and not row.visible:
            self.visible_rows_count -= 1

    def row_count(self):
        return len(self.rows)

    def get_label_item(self, index: int):
        return self.rows[index].label

    def set_row_visible(self, index: int, visible: bool):
        row = self.rows[index]
        if visible and not row.visible:
            self._show_row(index)
        elif not visible and row.visible:
            self._hide_row(index)

    def _show_row(self, index: int):
        row = self.rows[index]
        index_to_use = 0 if index == 0 else self._find_index_to_insert_row(index)
        self.form.insertRow(index_to_use, row.label, row.field)
        self._replace_row(index, FormLayoutRow(row.label, row.field, True))

        # Widgets were hidden when the row was hidden
        self._set_visible_recursive(row.label, True)
        self._set_visible_recursive(row.field, True)

    def _hide_row(self, index: int):
        row = self.rows[index]
        # "index" is the original index of the row before we any rows were hidden.
        # Since we may have hidden certain rows already, we must search for the the row that corresponds "index"
        for i in range(self.form.rowCount()):
            label_item = self.form.itemAt(i, QtWidgets.QFormLayout.LabelRole)
            if row.label == as_widget_or_layout(label_item):
                self._take_row(i)
                self._replace_row(index, FormLayoutRow(row.label, row.field, False))
                # Found the matching row
                break

    def _take_row(self, index: int):
        label_item = self.form.itemAt(index, QtWidgets.QFormLayout.LabelRole)
        field_item = self.form.itemAt(index, QtWidgets.QFormLayout.FieldRole)

        # Otherwise there will be artifacts in the UI texture
        self._set_visible_recursive(as_widget_or_layout(label_item), False)
        self._set_visible_recursive(as_widget_or_layout(field_item), False)

        # Remove items before removeRow so the widgets are not destroyed
        self.form.removeItem(label_item)
        self.form.removeItem(field_item)
        self.form.removeRow(index)

        # Reallocate space
        self.form.invalidate()
        self.form.update()

    def _find_index_to_insert_row(self, index: int):
        row = self.rows[index - 1]
        for actual_form_index in range(self.form.rowCount()):
            actual_item = self.form.itemAt(actual_form_index, QtWidgets.QFormLayout.LabelRole)
            if as_widget_or_layout(actual_item) is row.label:
                return actual_form_index + 1

        return 0

    def _set_visible_recursive(self, element, visible: bool):
        if isinstance(element, QtWidgets.QWidget):
            element.setVisible(visible)
            return

        for i in range(element.count()):
            child_item = element.itemAt(i)
            if child_item.widget():
                child_item.widget().setVisible(visible)
            elif child_item.layout():
                self._set_visible_recursive(child_item.layout(), visible)


class SettingsSearch:
    def __init__(self, searched_widgets: List[QtWidgets.QWidget]):
        self.searched_widgets = searched_widgets

        form_layout_handlers: Dict[QtWidgets.QFormLayout, FormLayoutRowVisibility] = {}
        self.form_layout_handlers = form_layout_handlers

    def search(self, search_string: str):
        search_term_list = search_string.split()
        for widget in self.searched_widgets:
            self._update_widget_visibility(widget, search_term_list)

    def _update_widget_visibility(self, widget: QtWidgets.QWidget, search_term_list):

        # We only search labels for search terms.
        if isinstance(widget, QtWidgets.QLabel):
            matches_search = self._is_search_match(widget.text(), search_term_list)
            widget.setVisible(matches_search)
            return matches_search

        if widget.layout():
            is_group_box = isinstance(widget, QtWidgets.QGroupBox)
            is_match_on_group_box = is_group_box and self._is_search_match(widget.title(), search_term_list)

            # If group box title matched, force all children to visible by clearing the search terms.
            if is_match_on_group_box:
                search_term_list = []

            # Recursively search inside the layout of this widget
            is_any_child_visible = self._update_layout_visibility(widget.layout(), search_term_list)

            # Widget should be visible if its title matches or any of its children is visible
            widget_visibility = is_match_on_group_box or is_any_child_visible
            widget.setVisible(widget_visibility)

            # Auto-expand visible collapsible group boxes
            if widget_visibility and isinstance(widget, CollapsibleGroupBox):
                widget.set_expanded(True)

            return widget_visibility

        return False

    def _update_layout_visibility(self, layout: QtWidgets.QLayout, search_term_list):

        is_match_on_any_child = False

        # Form layouts are treated especially: It peruses its rows for QLabels that match the search terms,
        # and doesn't look any further.
        if isinstance(layout, QtWidgets.QFormLayout):
            return self._handle_form_layout(layout, search_term_list)

        # We assume that an HBox layout with 2 items is a setting (key_label | value_widget).
        if isinstance(layout, QtWidgets.QHBoxLayout) and layout.count() == 2:
            first = layout.itemAt(0)
            first_item_is_label = first and first.widget() and isinstance(first.widget(), QtWidgets.QLabel)
            if first_item_is_label:
                is_match_on_any_child = self._update_widget_visibility(first.widget(), search_term_list)
                return is_match_on_any_child

        # Recursively iterate over all child widgets and layouts, and update is_match_on_any_child accordingly.
        for idx in range(layout.count()):
            layout_item = layout.itemAt(idx)
            if layout_item.widget():
                is_match_on_any_child |= self._update_widget_visibility(layout_item.widget(), search_term_list)
            if layout_item.layout():
                is_match_on_any_child |= self._update_layout_visibility(layout_item.layout(), search_term_list)

        return is_match_on_any_child

    def _handle_form_layout(self, layout: QtWidgets.QFormLayout, search_term_list):
        '''
        Handles the visibility of rows in a QFormLayout based on a list of search terms.

        This method iterates through each row in the provided QFormLayout, checking if the label
        in each row matches any of the provided search terms. If a match is found, the corresponding
        row is made visible; otherwise, it is hidden. The method returns a boolean indicating whether
        any rows matched the search criteria.

        Args:
            layout (QtWidgets.QFormLayout): The form layout whose rows are to be checked and updated.
            search_term_list (List[str]): A list of search terms used to determine visibility.

        Returns:
            bool: True if any row in the layout matches the search terms, False otherwise.
        '''

        is_match_on_any_child = False
        form_handler = self.form_layout_handlers.setdefault(layout, FormLayoutRowVisibility(layout))

        # Update the rows cache of the form handler if SettingsBatchManagerToCreateUI as added more rows.
        if form_handler.has_form_added_rows_externally():
            form_handler.update_rows_cache()

        for idx in range(form_handler.row_count()):
            label = form_handler.get_label_item(idx)
            if isinstance(label, QtWidgets.QWidget):
                is_match = self._update_widget_visibility(label, search_term_list)
                is_match_on_any_child |= is_match
                form_handler.set_row_visible(idx, is_match)

        return is_match_on_any_child

    @staticmethod
    def _is_search_match(text: str, search_term_list: List[str]):
        '''
        Determines whether the given text matches all search terms in the search term list.

        This method checks if each search term in the provided list appears within the
        given text. Both the text and the search terms are compared in a case-insensitive
        manner. The method returns True only if all search terms are found in the text,
        or search_term_list is empty.

        Args:
            text (str): The text to search within.
            search_term_list (List[str]): A list of search terms to match against the text.

        Returns:
            bool: True if all search terms are found in text or if search_term_list is empty.
        '''
        for search_term in search_term_list:
            if not search_term.lower() in text.lower():
                return False
        return True
