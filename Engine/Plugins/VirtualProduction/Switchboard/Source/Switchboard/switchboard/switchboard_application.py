# Copyright Epic Games, Inc. All Rights Reserved.

import base64
import hashlib
import os
import pathlib
import re
import shlex
import subprocess
import sys
import tempfile
import threading
from typing import NamedTuple, Optional, Union

import pythonosc.dispatcher
import pythonosc.osc_server

from .config import CONFIG, SETTINGS
from .switchboard_logging import LOGGER
from . import switchboard_ctypes as sb_ctypes
from . import switchboard_utils as sb_utils


class OscServer:
    def __init__(self):
        self.server_thread = None
        self.server = None

        self.dispatcher = pythonosc.dispatcher.Dispatcher()
        self.dispatcher.set_default_handler(self._default_callback, True)

        # For internal messaging
        self.internal_client = None
        self.server_port = None

    def address(self):
        if not self.server:
            return None

        return self.server.server_address

    def launch(self, address, port):
        # TODO: Allow relaunch of OSC server when address variable changes
        try:
            self.server = pythonosc.osc_server.ThreadingOSCUDPServer(
                (address, port), self.dispatcher)
        except OSError as e:
            if e.errno == 10048:
                LOGGER.error(
                    'OSC Server: Another OSC server is currently using '
                    f'{address}:{port}. Please kill and relaunch')
            elif e.errno == 10049:
                LOGGER.error(f"OSC Server: Couldn't bind {address}:{port}. "
                             'Please check address and relaunch')

            self.server = None
            return False

        # Set the server address and port
        self.server_port = port

        LOGGER.success(
            f'OSC Server: Receiving on {self.server.server_address}')
        self.server_thread = threading.Thread(target=self.server.serve_forever)
        self.server_thread.start()

        return True

    def close(self):
        if not self.server:
            return

        LOGGER.info('OSC Server: Shutting down')
        self.server.shutdown()
        self.server.server_close()
        self.server_thread.join()

    def is_running(self):
        if not self.server_thread:
            return False
        return self.server_thread.is_alive()

    def dispatcher_map(self, command, method):
        self.dispatcher.map(command, method, needs_reply_address=True)

    def _default_callback(self, client_address, command, *args):
        LOGGER.warning(f'Received unhandled OSC message: {command} {args}.')


class CommandLineExtractor:
    '''
    Given command line arguments as either a single string or an array,
    tokenizes and normalizes them (quotes removed), and provides a means to
    extract specific values.
    '''

    # Any single backslash not followed by a "
    WIN_SLASH_RE = re.compile(r"(?<!\\)\\(?![\"\\])")

    def __init__(self, args: Union[str, list[str]]):
        if sys.platform.startswith('win'):
            # Escape path separators (which shlex will subsequently unescape)
            if isinstance(args, str):
                args = self.WIN_SLASH_RE.sub(R'\\\\', args)
            else:
                for arg in args:
                    arg = self.WIN_SLASH_RE.sub(R'\\\\', arg)

        self.tokens = self._extract(args)

    def get_value_for_switch(self, switch: str) -> Optional[str]:
        switch_re = re.compile(f'-{switch}=(.*)', re.IGNORECASE)
        for token in self.tokens:
            if match := switch_re.search(token):
                return match.group(1)

        return None

    def _extract(self, args: Union[str, list[str]]):
        tokens: list[str] = []

        if isinstance(args, str):
            lex = self._shlex(args)
            tokens = list(lex)
        else:
            for arg in args:
                tokens.extend(list(self._shlex(arg)))

        return tokens

    def _shlex(self, instream) -> shlex.shlex:
        lex = shlex.shlex(instream=instream, posix=True)
        lex.whitespace_split = True
        if sys.platform.startswith('win'):
            lex.quotes = '"'  # Single quotes don't affect tokenization
        return lex


class MultiUserApplication:
    def __init__(self):
        self.lock = threading.Lock()
        self._process: Optional[subprocess.Popen] = None

        # Application Options
        self.concert_ignore_cl = False
        self.clear_process_info()

    def exe_path(self):
        return CONFIG.multiuser_server_path()

    def exe_name(self):
        return os.path.split(self.exe_path())[1]

    def get_mu_server_multicast_arg(self):
        multicast = CONFIG.MUSERVER_MULTICAST_ENDPOINT.get_value().strip()
        if multicast:
            return f'-UDPMESSAGING_TRANSPORT_MULTICAST="{multicast}"'
        return ''

    def get_mu_server_endpoint_arg(self):
        endpoint = self.configured_endpoint()
        if endpoint:
            return f'-UDPMESSAGING_TRANSPORT_UNICAST="{endpoint}"'
        return ''

    @property
    def process(self):
        if self._process and (self._process.poll() is None):
            return self._process
        else:
            return self.poll_process()

    def poll_process(self):
        # Aside from this task_name, PollProcess is stateless,
        # and we'd like to pick up on name changes.
        return sb_utils.PollProcess(self.exe_name())

    def launch(self, args: Optional[list[str]] = None):
        if args is None:
            args = []

        with self.lock:
            if self.is_running():
                return False

            if not os.path.exists(self.exe_path()):
                LOGGER.error('Could not find multi-user server at '
                             f'{self.exe_path()}. Has it been built?')
                return

            creationflags = 0

            if sys.platform.startswith('win'):
                args.insert(0, f'"{self.exe_path()}"')
                if not CONFIG.MUSERVER_SLATE_MODE.get_value():
                    creationflags = subprocess.CREATE_NEW_CONSOLE
            else:
                args.insert(0, f'{self.exe_path()}')

            args.extend([
                f'-CONCERTSERVER="{self.configured_server_name()}"',
                f'{CONFIG.MUSERVER_COMMAND_LINE_ARGUMENTS.get_value()}',
                f'-UDPMESSAGING_SHARE_KNOWN_NODES=1',
                f'{self.get_mu_server_endpoint_arg()}',
                f'{self.get_mu_server_multicast_arg()}'])

            if self.concert_ignore_cl:
                args.append('-ConcertIgnore')

            working_dir = CONFIG.MUSERVER_WORKING_DIR.get_value()
            if working_dir:
                args.append(f'-ConcertWorkingDir="{working_dir}"')

            archive_dir = CONFIG.MUSERVER_ARCHIVE_DIR.get_value()
            if archive_dir:
                args.append(f'-ConcertSavedDir="{archive_dir}"')

            if CONFIG.MUSERVER_CLEAN_HISTORY.get_value():
                args.append('-ConcertClean')

            if CONFIG.MUSERVER_FILESHARING_ENABLED.get_value() and CONFIG.MUSERVER_FILESHARING_SRV.get_value():
                args.append('-CONCERTFILESHARINGENABLED=1')
                filesharing = CONFIG.MUSERVER_FILESHARING_SRV.get_value()
                args.append(f'-CONCERTFILESHAREPATH={filesharing}')
                
            args = list(filter(None, args))

            launch_cmd_full = ' '.join(args)
            if sys.platform.startswith('win'):
                # Given a list on Windows, Popen calls subprocess.list2cmdline
                # to combine to a string. However, that also has the effect
                # of turning `-foo="bar baz"`` into `"-foo=\"bar baz\""`,
                # and Unreal's FParse won't accept that (OR `"-foo=bar baz"`);
                # its tokenization is too different from MSVCRT.
                args = launch_cmd_full
        
            LOGGER.info(f'Launching Multi-user Server: {launch_cmd_full}');

            self._process = subprocess.Popen(args, creationflags=creationflags)
            return True

    def extract_process_info(self):
        pid = self.process.pid
        args = self.process.args

        if (not args) or (pid == self._running_pid):
            return

        self.clear_process_info()
        self._running_pid = pid

        extractor = CommandLineExtractor(args)
        self._running_endpoint = extractor.get_value_for_switch(
            'UDPMESSAGING_TRANSPORT_UNICAST') or ''
        self._running_name = extractor.get_value_for_switch(
            'CONCERTSERVER') or ''

    def running_server_name(self):
        return self._running_name

    def running_endpoint(self):
        return self._running_endpoint

    def configured_endpoint(self):
        endpoint_setting = CONFIG.MUSERVER_ENDPOINT.get_value().strip()
        endpoint = ""
        if endpoint_setting:
            addr_setting = SETTINGS.ADDRESS.get_value().strip()
            endpoint = sb_utils.expand_endpoint(endpoint_setting, addr_setting)
        return endpoint

    def configured_server_name(self):
        endpoint = self.configured_endpoint()
        if not endpoint:
            LOGGER.warning(f'No configured endpoint for Multi-user server is defined. This can result in name collisions.')
            endpoint = '127.0.0.1:9030'

        digest = hashlib.md5(endpoint.encode('utf-8')).digest()

        # Use only the first 12 characters which should be enough to prevent clashing and keep the server name relatively short.
        #
        encoded = base64.b64encode(digest).decode('utf-8').strip('==')[:12]

        configured_server_name = CONFIG.MUSERVER_SERVER_NAME.get_value()
        return f'{configured_server_name}_{encoded}'

    def clear_process_info(self):
        self._running_endpoint = ""
        self._running_name = ""
        self._running_pid = None

    def is_running(self):
        if self.process.poll() is None:
            self.extract_process_info()
            return True

        self.clear_process_info()
        return False

    def kill(self):
        self.process.kill()


MultiUserServerInstance: Optional[MultiUserApplication] = None


def get_multi_user_server_instance():
    global MultiUserServerInstance
    if not MultiUserServerInstance:
        MultiUserServerInstance = MultiUserApplication()
    return MultiUserServerInstance


class RsyncServer:
    DEFAULT_PORT = 8730
    INCOMING_LOGS_MODULE = 'device_logs'
    MAX_MONITOR_RECOVERIES = 5
    SOCKET_ERROR_EXIT_CODE = 10

    CONFIG_TEMPLATE = '''\
# chroot not supported on Windows.
use chroot = false

# Default deny all clients (IPv4 and IPv6).
hosts deny = 0.0.0.0/0, ::/0

# Permit known Switchboard devices.
hosts allow = {allowed_addrs}

[{incoming_logs_module}]
    comment = Writable destination to aggregate device logs.
    path = {incoming_logs_path}
    read only = false
    write only = true
    refuse options = delete
'''

    class Client(NamedTuple):
        name: str
        address: str

    def __init__(self):
        self.address = ''
        self.port = 0
        self.incoming_logs_path: Optional[pathlib.Path] = None
        self.config_file = tempfile.NamedTemporaryFile(
            prefix='sb_rsync', suffix='.conf', mode='wt')
        self.log_file = tempfile.NamedTemporaryFile(
            prefix='sb_rsync', suffix='.log', mode='rt')
        self.process: Optional[subprocess.Popen] = None
        self.allowed_clients: dict[object, RsyncServer.Client] = {}

        self._shutdown_event = threading.Event()
        self._monitor_thread: Optional[threading.Thread] = None
        self._monitor_recoveries = 0

        self._cleanup_helper = None
        try:
            self._cleanup_helper = sb_ctypes.CloseSubprocessOnExitHelper()
        except Exception as exc:
            LOGGER.warning('RsyncServer: Error creating cleanup helper')

    def is_running(self) -> bool:
        return (self.process is not None) and (self.process.poll() is None)

    def register_client(self, client: object, name: str, address: str):
        if client in self.allowed_clients:
            raise RuntimeError('Duplicate client key')

        # TODO: Once we have transport security, generate client credentials?
        self.allowed_clients[client] = RsyncServer.Client(name, address)
        self.update_config()

    def unregister_client(self, client: object):
        if client not in self.allowed_clients:
            raise RuntimeError('Unknown client key')

        del self.allowed_clients[client]
        self.update_config()

    def set_incoming_logs_path(self, path: pathlib.Path):
        if path and path.is_dir() and path.is_absolute():
            self.incoming_logs_path = path
        else:
            raise RuntimeError('Must be absolute path to directory')

        self.update_config()

    def make_cygdrive_path(self, native_path: pathlib.Path):
        r''' Given "C:\foo\bar.txt", return "/cygdrive/C/foo/bar.txt" '''
        if native_path.drive.endswith(':'):
            return pathlib.PurePosixPath(
                '/cygdrive/',
                native_path.drive.rstrip(':'),
                native_path.relative_to(native_path.anchor))
        else:  # Could be a UNC path?
            return pathlib.PurePosixPath(native_path.as_posix())

    def update_config(self):
        '''
        Rewrite the rsyncd conf file. (The running server will re-read the file
        upon every incoming connection, so there's no need to relaunch.)
        '''
        if self.incoming_logs_path is not None:
            incoming_logs_path_str = str(
                self.make_cygdrive_path(self.incoming_logs_path))
        else:
            # If blank in generated config, the module will be inaccessible.
            incoming_logs_path_str = ''

        allowed_addrs = {
            client.address for client in self.allowed_clients.values()}

        allowed_addrs.add(self.address)
        allowed_addrs.add(SETTINGS.ADDRESS.get_value())
        allowed_addrs.discard('0.0.0.0')

        config = self.CONFIG_TEMPLATE.format(
            allowed_addrs=', '.join(allowed_addrs),
            incoming_logs_module=self.INCOMING_LOGS_MODULE,
            incoming_logs_path=incoming_logs_path_str,
        )

        self.config_file.seek(0)
        self.config_file.write(config)
        self.config_file.truncate()
        self.config_file.flush()

    def launch(
        self, *, address: str = '0.0.0.0', port: int = DEFAULT_PORT
    ) -> bool:
        if self.is_running():
            LOGGER.warning('RsyncServer.launch: server already running')
            return False

        self.address = address
        self.port = port
        self.update_config()

        if sys.platform.startswith('win'):
            rsync_path = os.path.normpath(os.path.join(
                CONFIG.ENGINE_DIR.get_value(), 'Extras', 'ThirdPartyNotUE',
                'SwitchboardThirdParty', 'cwrsync', 'bin', 'rsync.exe'))
        else:
            rsync_path = 'rsync'

        args = [rsync_path, '--daemon', '--no-detach', f'--address={address}',
                f'--port={port}', f'--config={self.config_file.name}',
                f'--log-file={self.log_file.name}']

        LOGGER.debug(f'RsyncServer.launch: {" ".join(args)}')

        try:
            self.process = subprocess.Popen(
                args,
                stdin=subprocess.DEVNULL,
                stdout=subprocess.PIPE,
                stderr=subprocess.PIPE,
                startupinfo=sb_utils.get_hidden_sp_startupinfo())
        except Exception as exc:
            LOGGER.error('RsyncServer.launch: '
                         f'Failed with exception: {exc}', exc_info=exc)
            return False

        try:
            if self._cleanup_helper:
                self._cleanup_helper.add_process(self.process.pid)
        except Exception as exc:
            LOGGER.warning('RsyncServer.launch: Error registering process for cleanup', exc_info=exc)

        self._shutdown_event.clear()
        self._monitor_thread = threading.Thread(
            target=self._monitor)
        self._monitor_thread.start()

        return True

    def shutdown(self):
        if self.process is None or self._monitor_thread is None:
            LOGGER.warning('RsyncServer.shutdown: server not running')
            return

        LOGGER.debug('RsyncServer.shutdown: requesting shutdown')
        self._shutdown_event.set()
        self._monitor_thread.join(timeout=3)

    def _monitor(self):
        assert self.process
        POLL_INTERVAL = 0.1

        def handle_outputs(stdout: Optional[str], stderr: Optional[str]):
            for line in self.log_file.readlines():
                LOGGER.info(f'RsyncServer.monitor: [log] {line.rstrip()}')

            if stdout:
                for line in stdout.splitlines():
                    LOGGER.info(f'RsyncServer.monitor: [stdout] {line}')

            if stderr:
                for line in stderr.splitlines():
                    LOGGER.error(f'RsyncServer.monitor: [stderr] {line}')

        shutdown_requested = False
        keep_polling = True
        while keep_polling:
            if self._shutdown_event.is_set():
                shutdown_requested = True
                keep_polling = False

                # FIXME? On Win this is kill(), but sending ctrl-c didn't work.
                self.process.terminate()

                try:
                    GRACEFUL_EXIT_TIMEOUT = 2.0
                    self.process.wait(GRACEFUL_EXIT_TIMEOUT)
                except subprocess.TimeoutExpired:
                    pass

            if self.process.poll():
                keep_polling = False

            out: Optional[str] = None
            err: Optional[str] = None
            try:
                out, err = self.process.communicate(timeout=POLL_INTERVAL)
            except subprocess.TimeoutExpired:
                pass

            handle_outputs(out, err)

        if shutdown_requested:
            if self.process.poll() is None:
                LOGGER.warning('RsyncServer.monitor: issuing kill')
                self.process.kill()

            self.process = None

            # Nothing more to do.
            return

        LOGGER.error('RsyncServer.monitor: rsync quit unexpectedly '
                     f'with code {self.process.returncode}')

        # Recover from unexpected exit.
        if self._monitor_recoveries < self.MAX_MONITOR_RECOVERIES:
            LOGGER.info('RsyncServer.monitor: restarting...')

            if self.process.returncode == RsyncServer.SOCKET_ERROR_EXIT_CODE:
                # This exit code most often means we failed to bind the
                # specified endpoint, likely because of a port conflict, so
                # automatically increment the port before we retry.
                self.port += 1

                LOGGER.warning(
                    f'RsyncServer.monitor: incrementing port to {self.port}')

            self._monitor_recoveries += 1
            self.process = None
            self.launch(address=self.address, port=self.port)
        else:
            LOGGER.error('RsyncServer.monitor: max recoveries exceeded '
                         f'({self.MAX_MONITOR_RECOVERIES})')
