# Copyright Epic Games, Inc. All Rights Reserved.

from __future__ import annotations
import ctypes
import sys


if sys.platform.startswith('win'):
    import ctypes.wintypes

    advapi32 = ctypes.WinDLL('Advapi32')
    kernel32 = ctypes.WinDLL('kernel32')

    ERROR_NOT_FOUND = 1168

    def falsey_errcheck(result, func, args):
        if not result:
            last_err = ctypes.get_last_error()
            raise ctypes.WinError(last_err)

        return args

    ####################
    # Credentials

    CRED_PERSIST_LOCAL_MACHINE = 2
    CRED_TYPE_GENERIC = 1

    class CREDENTIALW(ctypes.Structure):
        _fields_ = [
            ('Flags', ctypes.wintypes.DWORD),
            ('Type', ctypes.wintypes.DWORD),
            ('TargetName', ctypes.wintypes.LPWSTR),
            ('Comment', ctypes.wintypes.LPWSTR),
            ('LastWritten', ctypes.wintypes.FILETIME),
            ('CredentialBlobSize', ctypes.wintypes.DWORD),
            ('CredentialBlob', ctypes.wintypes.LPBYTE),
            ('Persist', ctypes.wintypes.DWORD),
            ('AttributeCount', ctypes.wintypes.DWORD),
            ('Attributes', ctypes.c_void_p),
            ('TargetAlias', ctypes.wintypes.LPWSTR),
            ('UserName', ctypes.wintypes.LPWSTR),
        ]

    _CredFree_prototype = ctypes.WINFUNCTYPE(
        None,  # return type

        ctypes.c_void_p,  # Buffer
    )
    CredFree = _CredFree_prototype(
        ('CredFree', advapi32),
        (
            (1, 'Buffer'),
        )
    )

    _CredDeleteW_prototype = ctypes.WINFUNCTYPE(
        ctypes.wintypes.BOOL,  # return type

        ctypes.wintypes.LPCWSTR,  # TargetName
        ctypes.wintypes.DWORD,    # Type
        ctypes.wintypes.DWORD,    # Flags (reserved)
    )
    CredDeleteW = _CredDeleteW_prototype(
        ('CredDeleteW', advapi32),
        (
            (1, 'TargetName'),
            (1, 'Type'),
            (1, 'Flags', 0),
        )
    )

    _CredReadW_prototype = ctypes.WINFUNCTYPE(
        ctypes.wintypes.BOOL,  # return type

        ctypes.wintypes.LPWSTR,  # TargetName
        ctypes.wintypes.DWORD,   # Type
        ctypes.wintypes.DWORD,   # Flags (reserved)
        ctypes.POINTER(ctypes.POINTER(CREDENTIALW)),  # OutCredential

        use_last_error=True,
    )
    CredReadW = _CredReadW_prototype(
        ('CredReadW', advapi32),
        (
            (1, 'TargetName'),
            (1, 'Type'),
            (1, 'Flags', 0),
            (3, 'Credential'),
        )
    )

    _CredWriteW_prototype = ctypes.WINFUNCTYPE(
        ctypes.wintypes.BOOL,  # return type

        ctypes.POINTER(CREDENTIALW),  # Credential
        ctypes.wintypes.DWORD,        # Flags

        use_last_error=True,
    )
    CredWriteW = _CredWriteW_prototype(
        ('CredWriteW', advapi32),
        (
            (1, 'Credential'),
            (1, 'Flags', 0),
        )
    )

    def Cred_errcheck(result, func, args):
        if not result:
            last_err = ctypes.get_last_error()
            if last_err == ERROR_NOT_FOUND:
                raise KeyError()
            else:
                raise ctypes.WinError(last_err)

        return args

    CredReadW.errcheck = Cred_errcheck
    CredWriteW.errcheck = Cred_errcheck

    ####################
    # Job objects

    ULONG_PTR = SIZE_T = ctypes.wintypes.WPARAM

    PROCESS_TERMINATE = 0x0001
    PROCESS_SET_QUOTA = 0x0100
    JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE = 0x00002000
    JobObjectExtendedLimitInformation = 9

    class JOBOBJECT_BASIC_LIMIT_INFORMATION(ctypes.Structure):
        _fields_ = [
            ('PerProcessUserTimeLimit', ctypes.wintypes.LARGE_INTEGER),
            ('PerJobUserTimeLimit', ctypes.wintypes.LARGE_INTEGER),
            ('LimitFlags', ctypes.wintypes.DWORD),
            ('MinimumWorkingSetSize', SIZE_T),
            ('MaximumWorkingSetSize', SIZE_T),
            ('ActiveProcessLimit', ctypes.wintypes.DWORD),
            ('Affinity', ULONG_PTR),
            ('PriorityClass', ctypes.wintypes.DWORD),
            ('SchedulingClass', ctypes.wintypes.DWORD),
        ]

    class IO_COUNTERS(ctypes.Structure):
        _fields_ = [
            ('ReadOperationCount', ctypes.wintypes.ULARGE_INTEGER),
            ('WriteOperationCount', ctypes.wintypes.ULARGE_INTEGER),
            ('OtherOperationCount', ctypes.wintypes.ULARGE_INTEGER),
            ('ReadTransferCount', ctypes.wintypes.ULARGE_INTEGER),
            ('WriteTransferCount', ctypes.wintypes.ULARGE_INTEGER),
            ('OtherTransferCount', ctypes.wintypes.ULARGE_INTEGER),
        ]

    class JOBOBJECT_EXTENDED_LIMIT_INFORMATION(ctypes.Structure):
        _fields_ = [
            ('BasicLimitInformation', JOBOBJECT_BASIC_LIMIT_INFORMATION),
            ('IoInfo', IO_COUNTERS),
            ('ProcessMemoryLimit', SIZE_T),
            ('JobMemoryLimit', SIZE_T),
            ('PeakProcessMemoryUsed', SIZE_T),
            ('PeakJobMemoryUsed', SIZE_T),
        ]

    _CreateJobObjectW_prototype = ctypes.WINFUNCTYPE(
        ctypes.wintypes.HANDLE,  # return type

        ctypes.wintypes.LPVOID,   # JobAttributes (LPSECURITY_ATTRIBUTES*)
        ctypes.wintypes.LPCWSTR,  # Name

        use_last_error=True,
    )
    CreateJobObjectW = _CreateJobObjectW_prototype(
        ('CreateJobObjectW', kernel32),
        (
            (1, 'JobAttributes', 0),
            (1, 'Name'),
        )
    )
    CreateJobObjectW.errcheck = falsey_errcheck

    _QueryInformationJobObject_prototype = ctypes.WINFUNCTYPE(
        ctypes.wintypes.BOOL,  # return type

        ctypes.wintypes.HANDLE,   # hJob
        ctypes.wintypes.INT,      # JobObjectInformationClass (enum JOBOBJECTINFOCLASS)
        ctypes.wintypes.LPVOID,   # lpJobObjectInformation
        ctypes.wintypes.DWORD,    # cbJobObjectInformationLength
        ctypes.wintypes.LPDWORD,  # lpReturnLength

        use_last_error=True,
    )
    QueryInformationJobObject = _QueryInformationJobObject_prototype(
        ('QueryInformationJobObject', kernel32),
        (
            (1, 'hJob'),
            (1, 'JobObjectInformationClass'),
            (3, 'lpJobObjectInformation'),
            (1, 'cbJobObjectInformationLength'),
            (3, 'lpReturnLength'),
        )
    )
    QueryInformationJobObject.errcheck = falsey_errcheck

    _SetInformationJobObject_prototype = ctypes.WINFUNCTYPE(
        ctypes.wintypes.BOOL,  # return type

        ctypes.wintypes.HANDLE,  # hJob
        ctypes.wintypes.INT,     # JobObjectInformationClass (enum JOBOBJECTINFOCLASS)
        ctypes.wintypes.LPVOID,  # lpJobObjectInformation
        ctypes.wintypes.DWORD,   # cbJobObjectInformationLength

        use_last_error=True,
    )
    SetInformationJobObject = _SetInformationJobObject_prototype(
        ('SetInformationJobObject', kernel32),
        (
            (1, 'hJob'),
            (1, 'JobObjectInformationClass'),
            (1, 'lpJobObjectInformation'),
            (1, 'cbJobObjectInformationLength'),
        )
    )
    SetInformationJobObject.errcheck = falsey_errcheck


class CloseSubprocessOnExitHelper:
    ''' Helper to ensure a child process exits if this parent process does. '''

    def __init__(self):
        if sys.platform.startswith('win'):
            # Create job object and configure it to ensure termination
            self._job_object = CreateJobObjectW(None, None)

            ext_limit_info = JOBOBJECT_EXTENDED_LIMIT_INFORMATION()
            QueryInformationJobObject(
                self._job_object,
                JobObjectExtendedLimitInformation,
                ctypes.byref(ext_limit_info),
                ctypes.sizeof(ext_limit_info),
                None,
            )

            ext_limit_info.BasicLimitInformation.LimitFlags |= JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE

            SetInformationJobObject(
                self._job_object,
                JobObjectExtendedLimitInformation,
                ctypes.byref(ext_limit_info),
                ctypes.sizeof(ext_limit_info),
            )

    def add_process(self, pid):
        ''' Bind the process with the specified PID to terminate when this process does. '''

        if sys.platform.startswith('win'):
            # Get native handle to process
            desired_access = ctypes.wintypes.DWORD(PROCESS_TERMINATE | PROCESS_SET_QUOTA)
            inherit_handle = ctypes.wintypes.BOOL(False)
            process_id = ctypes.wintypes.DWORD(pid)
            process_handle = kernel32.OpenProcess(desired_access, inherit_handle, process_id)

            if not process_handle:
                raise ctypes.WinError(ctypes.get_last_error())

            # Assign the process to our job object
            kernel32.AssignProcessToJobObject(self._job_object, process_handle)

            kernel32.CloseHandle(process_handle)
