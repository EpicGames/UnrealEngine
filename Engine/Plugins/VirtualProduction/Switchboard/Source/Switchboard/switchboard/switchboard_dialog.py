# Copyright Epic Games, Inc. All Rights Reserved.

from __future__ import annotations

import datetime
from itertools import count
import logging
import os
import threading
import time
import re
import sys
from typing import Callable, List, Optional, Set, Union

from pathlib import Path

from PySide6 import QtCore
from PySide6 import QtGui
from PySide6 import QtWidgets

from PySide6.QtWidgets import QWidgetAction, QMenu

from switchboard import config
from switchboard import config_osc as osc
from switchboard import p4_utils
from switchboard import recording
from switchboard import resources  # noqa
from switchboard import switchboard_application
from switchboard import switchboard_logging
from switchboard import switchboard_utils
from switchboard import switchboard_widgets as sb_widgets
from switchboard import ugs_utils
from switchboard.add_config_dialog import AddConfigDialog
from switchboard.config import CONFIG, DEFAULT_MAP_TEXT, ENABLE_UGS_SUPPORT, SETTINGS, EngineSyncMethod
from switchboard.device_list_widget import DeviceListWidget
from switchboard.devices.device_base import Device, DeviceStatus
from switchboard.devices.device_manager import DeviceManager
from switchboard.settings_dialog import SettingsDialog
from switchboard.switchboard_logging import ConsoleStream, LOGGER
from switchboard.tools.insights_launcher import InsightsLauncher
from switchboard.tools.listener_launcher import ListenerLauncher
from switchboard.tools.sblhelper_launcher import SBLHelperLauncher
from switchboard.devices.unreal.plugin_unreal import DeviceUnreal
from switchboard.devices.ndisplay.plugin_ndisplay import DevicenDisplay, PackagingClientConfig
from switchboard.devices.unreal.redeploy_dialog import RedeployListenerDialog
from switchboard.util import collect_logs
from switchboard.ui.switchboard_ui import Ui_switchboard_window

import switchboard.debug_utils as debug_utils

ENGINE_PATH = "../../../../.."
RELATIVE_PATH = os.path.dirname(__file__)
EMPTY_SYNC_ENTRY = "-- None --"


class TraceSettings(QtWidgets.QDialog):
    """
    Custom class to prompt user for the trace arguments to use for Unreal Insights trace collection.
    """
    def __init__(self, trace_settings, parent=None):
        super().__init__(parent=parent, f=QtCore.Qt.WindowCloseButtonHint)

        self.setWindowTitle("Trace Settings")

        self.arguments_field = QtWidgets.QLineEdit(self)
        self.arguments_field.setText(trace_settings)

        self.form_layout = QtWidgets.QFormLayout()
        self.form_layout.addRow("Arguments", self.arguments_field)

        layout = QtWidgets.QVBoxLayout()
        layout.insertLayout(0, self.form_layout)

        button_box = QtWidgets.QDialogButtonBox(QtWidgets.QDialogButtonBox.Ok | QtWidgets.QDialogButtonBox.Cancel)
        button_box.accepted.connect(lambda: self.accept())
        button_box.rejected.connect(lambda: self.reject())
        layout.addWidget(button_box)

        self.setLayout(layout)

    def trace_settings(self):
        """
        Return the current trace settings.
        """
        return self.arguments_field.text()


class DeviceAdditionalSettingsUI(QtCore.QObject):
    signal_device_widget_tracing = QtCore.Signal(object)

    enable_insight_trace = False
    insight_trace_args = "log,cpu,gpu,frame,bookmark,concert,messaging"

    def __init__(self, name, parent=None):
        super().__init__(parent)
        self.name = name

    def _update_enable_state(self):
        """
        The button assigned for the settings UI should be "checked" when the user has traces enabled.
        """
        self._button.setChecked(self.enable_insights_menu().isChecked())

    def set_insight_trace_state(self, is_checked):
        """
        External setter for the current tracing state.  This is applied by the device one loading the setting from the global
        configuration.
        """
        if is_checked is not self.enable_insights_menu().isChecked():
            self.enable_insights_menu().setChecked(is_checked)
            self._update_enable_state()

    def set_insight_tracing_args(self, value):
        """
        External setter for tracing arguments. This is applied by the device one loading the setting from the global
        configuration.
        """
        if value != self.insight_trace_args:
            self.insight_trace_args = value
            self._update_enable_state()

    def enable_insights_menu(self):
        return self._tracing_action

    def trace_settings(self):
        return (self.enable_insights_menu().isChecked(), self.insight_trace_args)

    def _set_tracing(self, is_checked):
        self.signal_device_widget_tracing.emit(self)
        self._update_enable_state()

    def _set_trace_settings(self, is_checked):
        self._tracing_settings.setChecked(False)
        settings = TraceSettings(self.insight_trace_args)
        settings.exec()
        self.insight_trace_args = settings.trace_settings()
        self.signal_device_widget_tracing.emit(self)
        self._update_enable_state()

    def _generate_trace_menu_item(self, menu):
        action_item = QtGui.QAction("Enable Unreal Insights Tracing",
                                    menu, checked=DeviceAdditionalSettingsUI.enable_insight_trace, checkable=True)
        action_item.setChecked(DeviceAdditionalSettingsUI.enable_insight_trace)
        action_item.triggered.connect(self._set_tracing)
        self._tracing_action = action_item
        return action_item

    def _generate_trace_settings_item(self, menu):
        action_item = QtGui.QAction("Unreal Insights Trace Settings...",
                                    menu, checked=DeviceAdditionalSettingsUI.enable_insight_trace, checkable=True)
        action_item.setChecked(DeviceAdditionalSettingsUI.enable_insight_trace)
        action_item.triggered.connect(self._set_trace_settings)
        self._tracing_settings = action_item
        return action_item

    def _generate_settings_menu_items(self, menu):
        self.settings_menu.addAction(self._generate_trace_menu_item(menu))
        self.settings_menu.addAction(self._generate_trace_settings_item(menu))

    def get_button(self):
        return self._button

    def assign_button(self, button, parent):
        self._button = button
        self.settings_menu = QtWidgets.QMenu(parent)
        self._generate_settings_menu_items(self.settings_menu)
        self._button.setMenu(self.settings_menu)
        self._button.setStyleSheet("QPushButton::menu-indicator{image:none;}")
        self._button.setDisabled(False)

    def make_button(self, parent):
        """
        Make a new device setting push button.
        """
        button = sb_widgets.ControlQPushButton.create(
                icon_size=QtCore.QSize(21, 21),
                tool_tip='Change device settings',
                hover_focus=False,
                name='settings')

        self.assign_button(button, parent)
        return button


class ProcessMonitor(QtCore.QObject):
    ''' Offloads potentially expensive periodic checks to another thread. '''

    mu_server_status_changed = QtCore.Signal(object)

    class MuServerStatus:
        def __init__(self):
            self.pid: Optional[int] = None
            self.endpoint: Optional[str] = None
            self.name: Optional[str] = None

        def __eq__(self, other):
            if isinstance(other, ProcessMonitor.MuServerStatus):
                return ((self.pid == other.pid) and
                        (self.endpoint == other.endpoint) and
                        (self.name == other.name))
            else:
                return False

    def __init__(self, dialog: SwitchboardDialog):
        super().__init__()

        self._dialog = dialog
        self._exiting = False
        self._last_mu_status = ProcessMonitor.MuServerStatus()

        self._thread = threading.Thread(target=self._run)
        self._thread.start()

    def exit(self):
        self._exiting = True
        self._thread.join()

    def _run(self):
        while not self._exiting:
            self._poll_multiuser_status()
            self._dialog.update_locallistener_menuitem()
            self._dialog.update_localsblhelper_menuitem()
            self._dialog.update_insights_menuitem()
            self._dialog.update_package_game_menuitem()

            time.sleep(1.0)

    def _poll_multiuser_status(self):
        new_status = ProcessMonitor.MuServerStatus()

        server = switchboard_application.get_multi_user_server_instance()
        if server.is_running():
            new_status.pid = server._running_pid
            new_status.endpoint = server.running_endpoint()
            new_status.name = server.running_server_name()

        if new_status != self._last_mu_status:
            self._last_mu_status = new_status
            self.mu_server_status_changed.emit(new_status)


class SwitchboardDialog(QtCore.QObject):

    STYLESHEET_PATH = os.path.join(RELATIVE_PATH, 'ui/switchboard.qss')

    _stylesheet_watcher: Optional[QtCore.QFileSystemWatcher] = None

    @classmethod
    def init_stylesheet_watcher(cls):
        ''' Load Qt stylesheet, and reload it whenever the file is changed. '''
        if not cls._stylesheet_watcher:
            cls._stylesheet_watcher = QtCore.QFileSystemWatcher()
            cls._stylesheet_watcher.addPath(cls.STYLESHEET_PATH)
            cls._stylesheet_watcher.fileChanged.connect(
                lambda: cls.reload_stylesheet())

            cls.reload_stylesheet()

    @classmethod
    def reload_stylesheet(cls):
        with open(cls.STYLESHEET_PATH, "r") as styling:
            stylesheet = styling.read()
            QtWidgets.QApplication.instance().setStyleSheet(stylesheet)

    def __init__(self, script_manager):
        super().__init__()

        self.window = None
        self.script_manager = script_manager

        font_dir = os.path.join(ENGINE_PATH, 'Content/Slate/Fonts')
        font_files = ['Roboto-Regular.ttf', 'Roboto-Bold.ttf',
                      'DroidSansMono.ttf']

        for font_file in font_files:
            font_path = os.path.join(font_dir, font_file)
            QtGui.QFontDatabase.addApplicationFont(font_path)

        self.logger_autoscroll = True
        ConsoleStream.stderr().message_written.connect(self._console_pipe)

        # Create an instance of the UI class
        self.window: Ui_switchboard_window = Ui_switchboard_window()
        self.window.setupUi(self.window)

        # Accept copy-paste, intended for uassets or umaps from Unreal Engine's Content Browser
        QtGui.QShortcut(QtGui.QKeySequence.Paste, self.window).activated.connect(self.on_paste)

        # Avoid other widgets getting focus automaticaly so we can catch the paste shortcut
        self.window.centralwidget.setFocusPolicy(QtCore.Qt.FocusPolicy.ClickFocus)

        # Connect to copy-paste events from handling devices
        DeviceUnreal.static_signals.clipboard_level_added_signal.connect(self.on_clipboard_level_added)

        # Add Tools Menu
        self.add_tools_menu()

        # used to shut down services cleanly on exit
        self.window.installEventFilter(self)
        self.close_event_counter = 0

        self.init_stylesheet_watcher()

        self._periodic_runner = None
        self._shoot = None
        self._sequence = None
        self._slate = None
        self._take = None
        self._project_changelist = None
        self._engine_changelist = None
        self._level = None
        self._multiuser_session_name = None
        self._is_recording = False
        self._recording_start_time: datetime.datetime | None = None
        self._recording_end_time: datetime.datetime | None = None
        self._recording_timer = QtCore.QTimer(self)
        self._recording_timer.timeout.connect(self._on_recording_timer)
        self._update_recording_status()
        self._description = 'description'

        # Recording Manager
        self.recording_manager = recording.RecordingManager(CONFIG.SWITCHBOARD_DIR)
        self.recording_manager.signal_recording_manager_saved.connect(self.recording_manager_saved)

        # DeviceManager
        self.device_manager = DeviceManager()
        self.device_manager.signal_device_added.connect(self.device_added)

        # Convenience UnrealInsights launcher
        self.init_insights_launcher()

        # Convenience local Switchboard Listener launcher
        self.init_listener_launcher()

        # Convenience local Switchboard Listener Helper launcher
        self.init_sblhelper_launcher()

        # Convenience Open Logs Folder menu item
        self.register_open_logs_menuitem()
        self.register_zip_logs_menuitem()

        # Debugging menu item
        if debug_utils.ADD_MENU:
            self.debug_menu = self.window.menu_bar.addMenu("&Debug")
            debug_utils.register_menu_actions(self.debug_menu)

        # Redeploy menu item
        self.register_redeploy_menuitem()

        # Fill DDC submenu
        self.register_fill_ddc_menuitem()

        # Package game
        self.register_package_game_menuitem()

        # Transport Manager
        #self.transport_queue = recording.TransportQueue(CONFIG.SWITCHBOARD_DIR)
        #self.transport_queue.signal_transport_queue_job_started.connect(self.transport_queue_job_started)
        #self.transport_queue.signal_transport_queue_job_finished.connect(self.transport_queue_job_finished)

        # add level picker combo box and refresh button
        self.level_combo_box = sb_widgets.SearchableComboBox(self.window)
        self.level_combo_box.setSizePolicy(QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Fixed)

        self.refresh_levels_button = sb_widgets.ControlQPushButton()
        self.refresh_levels_button.setMaximumSize(22, 22)
        self.refresh_levels_button.setIcon(QtGui.QIcon(':/icons/images/icon_refresh.png'))
        self.refresh_levels_button.setProperty("frameless", True)
        self.refresh_levels_button.setToolTip("Refresh level list")

        self.browse_to_level_button = sb_widgets.ControlQPushButton()
        self.browse_to_level_button.setMaximumSize(18, 22)
        self.browse_to_level_button.setText('...')
        font = self.browse_to_level_button.font()
        font.setBold(True)
        self.browse_to_level_button.setFont(font)
        self.browse_to_level_button.setToolTip("Browse to level")

        self.window.horizontalLayout_7.addWidget(self.level_combo_box)
        self.window.horizontalLayout_7.addWidget(self.browse_to_level_button)
        self.window.horizontalLayout_7.addWidget(self.refresh_levels_button)

        self.shoot = 'Default'
        self.sequence = SETTINGS.CURRENT_SEQUENCE
        self.slate = SETTINGS.CURRENT_SLATE
        self.take = SETTINGS.CURRENT_TAKE

        # Device List Widget
        self.device_list_widget: DeviceListWidget = self.window.device_list_widget
        # When new widgets are added, register the signal/slots
        self.device_list_widget.signal_register_device_widget.connect(self.register_device_widget)
        self.device_list_widget.signal_connect_all_plugin_devices_toggled.connect(self.on_all_plugin_devices_connect_toggled)
        self.device_list_widget.signal_open_all_plugin_devices_toggled.connect(self.on_all_plugin_devices_open_toggled)

        # forward device(widget) removal between ListWidget and DeviceManager
        self.device_list_widget.signal_remove_device.connect(self.device_manager.remove_device_by_hash)
        self.device_manager.signal_device_removed.connect(self.on_device_removed)

        # DeviceManager initialize with from the config
        #
        CONFIG.push_saving_allowed(False)
        try:
            self.device_manager.reset_plugins_settings(CONFIG)
            self.device_manager.add_devices(CONFIG._device_data_from_config)
        finally:
            CONFIG.pop_saving_allowed()

        # add menu for adding new devices
        self.device_add_menu = QtWidgets.QMenu()
        self.device_add_menu.aboutToShow.connect(lambda: self.show_device_add_menu())
        self.window.device_add_tool_button.setMenu(self.device_add_menu)
        self.window.device_add_tool_button.clicked.connect(lambda: self.show_device_add_menu())
        self.device_add_menu.triggered.connect(self.on_triggered_add_device)

        # Start the OSC server
        self.osc_server = switchboard_application.OscServer()
        self.osc_server.launch(SETTINGS.ADDRESS.get_value(), CONFIG.OSC_SERVER_PORT.get_value())

        # Register with OSC server
        self.osc_server.dispatcher_map(osc.TAKE, self.osc_take)
        self.osc_server.dispatcher_map(osc.SLATE, self.osc_slate)
        self.osc_server.dispatcher_map(osc.SLATE_DESCRIPTION, self.osc_slate_description)
        self.osc_server.dispatcher_map(osc.RECORD_START, self.osc_record_start)
        self.osc_server.dispatcher_map(osc.RECORD_STOP, self.osc_record_stop)
        self.osc_server.dispatcher_map(osc.RECORD_CANCEL, self.osc_record_cancel)
        self.osc_server.dispatcher_map(osc.RECORD_START_CONFIRM, self.osc_record_start_confirm)
        self.osc_server.dispatcher_map(osc.RECORD_STOP_CONFIRM, self.osc_record_stop_confirm)
        self.osc_server.dispatcher_map(osc.RECORD_CANCEL_CONFIRM, self.osc_record_cancel_confirm)
        self.osc_server.dispatcher_map(osc.UE4_LAUNCH_CONFIRM, self.osc_ue4_launch_confirm)
        self.osc_server.dispatcher.map(osc.OSC_ADD_SEND_TARGET_CONFIRM, self.osc_add_send_target_confirm, 1, needs_reply_address=True)
        self.osc_server.dispatcher.map(osc.ARSESSION_START_CONFIRM, self.osc_arsession_start_confirm, 1, needs_reply_address=True)
        self.osc_server.dispatcher.map(osc.ARSESSION_STOP_CONFIRM, self.osc_arsession_stop_confirm, 1, needs_reply_address=True)
        self.osc_server.dispatcher_map(osc.BATTERY, self.osc_battery)
        self.osc_server.dispatcher_map(osc.DATA, self.osc_data)

        # Connect UI to methods
        self.window.multiuser_session_lineEdit.editingFinished.connect(self.on_multiuser_session_lineEdit_editingFinished)
        self.window.slate_line_edit.editingFinished.connect(
            lambda: self._set_slate(self.window.slate_line_edit.text()))
        # Connect to our custom take widget signal
        self.window.take_spin_box.valueChanged.connect(self._on_take_value_changed)
        self.window.sequence_line_edit.textChanged.connect(self._set_sequence)
        self.level_combo_box.currentIndexChanged.connect(self._on_selected_level_changed)
        self.refresh_levels_button.clicked.connect(self.refresh_levels_incremental)
        self.browse_to_level_button.clicked.connect(self.browse_to_level_and_add_it)
        self.window.project_cl_combo_box.currentTextChanged.connect(self._set_project_changelist)
        self.window.engine_cl_combo_box.currentIndexChanged.connect(
            lambda _: self._set_engine_changelist(self.window.engine_cl_combo_box.currentText()))
        self.window.engine_cl_combo_box.lineEdit().editingFinished.connect(
            lambda: self._set_engine_changelist(self.window.engine_cl_combo_box.currentText()))
        self.window.logger_level_comboBox.currentTextChanged.connect(self.logger_level_comboBox_currentTextChanged)
        self.window.logger_autoscroll_checkbox.stateChanged.connect(self.logger_autoscroll_stateChanged)
        self.window.clear_log_button.clicked.connect(self.clear_log_button_clicked)
        self.window.logger_wrap_checkbox.stateChanged.connect(self.logger_wrap_stateChanged)
        self.window.record_button.released.connect(self.record_button_released)
        self.window.sync_all_button.clicked.connect(self.sync_all_button_clicked)
        self.window.build_all_button.clicked.connect(self.build_all_button_clicked)
        self.window.sync_and_build_all_button.clicked.connect(self.sync_and_build_all_button_clicked)
        self.window.refresh_project_cl_button.clicked.connect(self.refresh_project_cl_button_clicked)
        self.window.refresh_engine_cl_button.clicked.connect(self.refresh_engine_cl_button_clicked)
        self.window.connect_all_button.clicked.connect(self.connect_all_button_clicked)
        self.window.launch_all_button.clicked.connect(self.launch_all_button_clicked)
        self.window.settings_button.clicked.connect(self.settings_button_clicked)

        self.window.additional_settings = DeviceAdditionalSettingsUI("global")
        self.window.additional_settings.assign_button(self.window.device_settings_button, self.window)

        self.window.use_device_autojoin_setting_checkbox.toggled.connect(self.on_device_autojoin_changed)
        self.refresh_muserver_autojoin()

        self.window.muserver_start_stop_button.clicked.connect(self.on_muserver_start_stop_click)

        self.window.additional_settings.signal_device_widget_tracing.connect(self.on_tracing_settings_changed)
        self.refresh_trace_settings()

        # set up a thread that does periodic maintenace tasks
        self.setup_periodic_tasks_thread()

        # Connect to the session increment number.
        self.window.multiuser_session_inc_button.clicked.connect(self.on_multiuser_session_inc)

        # Stylesheet-related: Object names used for selectors, no focus forcing
        def configure_ctrl_btn(btn: sb_widgets.ControlQPushButton, name: str):
            btn.setObjectName(name)
            btn.hover_focus = False

        configure_ctrl_btn(self.refresh_levels_button, 'refresh')
        configure_ctrl_btn(self.window.sync_all_button, 'sync')
        configure_ctrl_btn(self.window.build_all_button, 'build')
        configure_ctrl_btn(self.window.sync_and_build_all_button, 'sync_and_build')
        configure_ctrl_btn(self.window.refresh_project_cl_button, 'refresh')
        configure_ctrl_btn(self.window.refresh_engine_cl_button, 'refresh')
        configure_ctrl_btn(self.window.launch_all_button, 'open')
        configure_ctrl_btn(self.window.connect_all_button, 'connect')

        # TransportQueue Menu
        #self.window.transport_queue_push_button.setContextMenuPolicy(QtCore.Qt.CustomContextMenu)
        #self.transport_queue_menu = QtWidgets.QMenu(self.window.transport_queue_push_button)
        #self.transport_queue_menu.aboutToShow.connect(self.transport_queue_menu_about_to_show)
        #self.window.transport_queue_push_button.setMenu(self.transport_queue_menu)

        # Log pane
        self.window.logger_autoscroll_checkbox.setCheckState(QtCore.Qt.Checked if self.logger_autoscroll else QtCore.Qt.Unchecked)
        self.window.base_console.horizontalScrollBar().sliderPressed.connect(lambda: self.window.logger_autoscroll_checkbox.setCheckState(QtCore.Qt.Unchecked))
        self.window.base_console.verticalScrollBar().sliderPressed.connect(lambda: self.window.logger_autoscroll_checkbox.setCheckState(QtCore.Qt.Unchecked))
        # entries will be removed from the log window after the number of maximumBlockCount entries has been reached
        self.window.base_console.document().setMaximumBlockCount(1000)

        # Menu items
        self.window.menu_new_config.triggered.connect(self.menu_new_config)
        self.window.menu_save_config_as.triggered.connect(self.menu_save_config_as)
        self.window.menu_delete_config.triggered.connect(self.menu_delete_config)
        self.window.update_settings.triggered.connect(self.menu_update_settings)

        # The "Load Config" menu is populated lazily.
        self.window.menu_load_config.aboutToShow.connect(
            self._on_menu_load_config_about_to_show)

        # Plugin UI
        self.device_manager.plug_into_ui(self.window.menu_bar, self.window.tabs_main)

        if CONFIG.file_path:
            self.toggle_p4_controls(CONFIG.P4_ENABLED.get_value())
            self.refresh_levels(CONFIG.get_levels())
        else:
            self.menu_new_config()

        self.set_config_hooks()

        self.set_multiuser_session_name(f'{CONFIG.MUSERVER_SESSION_NAME.get_value()}')

        # Run the transport queue
        #self.transport_queue_resume()

        self.update_current_config_text()
        self.update_current_address_text()
        SETTINGS.ADDRESS.signal_setting_changed.connect(
            lambda: self.update_current_address_text()
        )
        self.window.current_address_value.editingFinished.connect(self._try_change_address)

        self.refresh_window_title()

        self.script_manager.on_postinit(self)
        self.have_warned_about_muserver = False

        DeviceUnreal.static_signals.ugs_config_updated_signal.connect(
            lambda _: self.p4_refresh_project_cl()
        )

    def on_paste(self):
        ''' Catches pasting of content browser assets.
        The main purpose is to auto-add them to switchboard's caches of assets and levels.
        '''

        clipboard = QtGui.QGuiApplication.clipboard()
        mime = clipboard.mimeData()

        if mime.hasText():
            DeviceUnreal.handle_clipboard_mime_text(mime.text())

    @QtCore.Slot(object)
    def on_clipboard_level_added(self, gamepath):
        ''' Called when a new level gets added. This will typically happen via a copy-paste operation from
        UE's content browser. When this happens, this dialog needs to refresh the dropdown, and we are also
        automatically selecting it in the dropdown, as that is usually what we expect the user to do when they are
        manually adding a new level.
        '''
        levels = CONFIG.get_levels()
        self.refresh_levels(levels)
        self.level = gamepath

    def _try_change_address(self):
        new_value = self.window.current_address_value.text()
        old_address = SETTINGS.ADDRESS.get_value()

        if not SETTINGS.ADDRESS.possible_values.__contains__(new_value):
            SETTINGS.ADDRESS.possible_values.append(new_value)

        SETTINGS.ADDRESS.update_value(new_value)
        SETTINGS.save()
        # Check if osc connection binds before commiting changes
        if self.osc_restart_check(old_address):
            # Print to make it clear to the user that the Address was updated
            if old_address != new_value:
                LOGGER.info(f"Updated address to {new_value}")
        else:
            LOGGER.warning(f"Reverting to the previous address ({old_address})")
            SETTINGS.ADDRESS.update_value(old_address)
            SETTINGS.save()
            self.osc_server.launch(SETTINGS.ADDRESS.get_value(), CONFIG.OSC_SERVER_PORT.get_value())

    def _on_mu_server_status_changed(
            self,
            new_status: ProcessMonitor.MuServerStatus
    ):
        '''
        Update the status of the Multi-user start/stop button to reflect the status of Multi-user server.
        '''
        server = switchboard_application.get_multi_user_server_instance()
        expected_name = server.configured_server_name()
        expected_endpoint = server.configured_endpoint()

        is_running = new_status.pid is not None
        actual_name = new_status.name
        actual_endpoint = new_status.endpoint

        is_valid = (is_running and (actual_endpoint == expected_endpoint) and
                    (actual_name == expected_name))

        is_checked = self.window.muserver_start_stop_button.isChecked()
        if is_running and not is_checked:
            self.window.muserver_start_stop_button.setChecked(True)
        elif not is_running and is_checked:
            self.window.muserver_start_stop_button.setChecked(False)
            self.have_warned_about_muserver = False

        # Re-enable the button after a pending launch completes
        self.window.muserver_start_stop_button.setEnabled(True)

        label = 'Multi-user Server'
        if is_running:
            label += f' ({new_status.name} {new_status.endpoint})'
            if not is_valid:
                label += ' (WARNING: RUNNING SERVER DOES NOT MATCH CONFIGURATION)'
                if not self.have_warned_about_muserver:
                    LOGGER.warning(
                        'The running Multi-user server does not match '
                        'Switchboard configuration. Expected '
                        f'"{expected_name}" with "{expected_endpoint}" but '
                        f'found "{actual_name}" with "{actual_endpoint}". '
                        'Please restart the multi-user server.')

                    self.have_warned_about_muserver = True

        self.window.muserver_label.setText(label)

    def setup_periodic_tasks_thread(self):
        '''
        Sets up a thread for performing maintenance tasks
        '''
        self._periodic_runner = ProcessMonitor(self)
        self._periodic_runner.mu_server_status_changed.connect(
            self._on_mu_server_status_changed)

    def update_locallistener_menuitem(self):
        '''
        Enables/disables the local listener launch menu item depending on whether 
        it is already running or not.
        '''
        self.locallistener_launcher_menuitem.setEnabled(not self.listener_launcher.is_running())

    def update_localsblhelper_menuitem(self):
        '''
        Enables/disables the local listener helper launch menu item depending on whether 
        it is already running or not.
        '''
        self.localsblhelper_launcher_menuitem.setEnabled(not self.sblhelper_launcher.is_running())

    def update_insights_menuitem(self):
        '''
        Enables/disables the UnrealInsights launch menu item depending on whether 
        it is already running or not.
        '''
        self.insights_launcher_menuitem.setEnabled(not self.insights_launcher.is_running())

    def update_package_game_menuitem(self):
        ''' Enables/disables the package game menu depending on whether there are any nDisplay devices or not.
        For discoverability of the feature, we don't check if they are connected. Otherwise the user might be
        clueless as to why the option is disabled.
        '''
        ndisplay_devices = [device for device in self.device_manager.devices() if isinstance(device, DevicenDisplay)]

        do_enable = len(ndisplay_devices) > 0

        self.action_package_development.setEnabled(do_enable)
        self.action_package_shipping.setEnabled(do_enable)

        parent = self.action_package_development.parent()
        while parent is not None and not isinstance(parent, QMenu):
            parent = parent.parent()

        parent.setEnabled(do_enable)

    def on_muserver_start_stop_click(self):
        '''
        Handle the multi-user server button click. If we are running we stop the process. If we are not
        running then we launch the multi-user server.
        '''
        ServerInstance = switchboard_application.get_multi_user_server_instance()
        if ServerInstance.is_running():
            ServerInstance.process.kill()
        else:
            ServerInstance.launch()
            # This will be re-enabled when the button transitions to stop/close
            self.window.muserver_start_stop_button.setEnabled(False)

    def init_insights_launcher(self):
        ''' Initializes insights launcher '''

        self.insights_launcher = InsightsLauncher()

        def launch_insights():
            try:
                self.insights_launcher.launch()
            except Exception as e:
                LOGGER.error(e)

        action = self.register_tools_menu_action("&Insights")
        action.triggered.connect(launch_insights)

        self.insights_launcher_menuitem = action

    def init_listener_launcher(self):
        ''' Initializes switchboard listener launcher '''
        self.listener_launcher = ListenerLauncher()

        def launch_listener():
            try:
                self.listener_launcher.launch()
            except Exception as e:
                LOGGER.error(e)

        action = self.register_tools_menu_action("&Listener")
        action.triggered.connect(launch_listener)

        self.locallistener_launcher_menuitem = action

    def init_sblhelper_launcher(self):
        ''' Initializes switchboard listener helper launcher '''
        self.sblhelper_launcher = SBLHelperLauncher()

        def launch_sblhelper():
            try:
                self.sblhelper_launcher.launch()
            except Exception as e:
                LOGGER.error(e)

        # Gpu Clocker is available in select platforms
        if sys.platform in ('win32','linux'):
            action = self.register_tools_menu_action("&Gpu Clocker")
            action.triggered.connect(launch_sblhelper)
        else:
            action = QWidgetAction(None) # dummy action

        self.localsblhelper_launcher_menuitem = action

    def register_open_logs_menuitem(self):
        ''' Registers convenience "Open Logs Folder" menu item '''
        action = self.register_tools_menu_action("&Open Logs Folder")
        action.triggered.connect(collect_logs.open_logs_folder)

    def register_zip_logs_menuitem(self):
        def save_logs():
            status = collect_logs.execute_zip_logs_workflow(
                CONFIG,
                [device for device in self.device_manager.devices() if isinstance(device, DeviceUnreal)]
            )
            if status:
                collect_logs.open_logs_folder()
        action = self.register_tools_menu_action("&Zip Logs")
        action.triggered.connect(save_logs)

    def register_redeploy_menuitem(self):
        def show_redeploy_dialog():
            dlg = RedeployListenerDialog(DeviceUnreal.active_unreal_devices,
                                         DeviceUnreal.listener_watcher)
            dlg.exec()

        action = self.register_tools_menu_action("Listener &Redeployer")
        action.triggered.connect(show_redeploy_dialog)

    def register_fill_ddc_menuitem(self):
        def fill_ddc(current_level_only):
            unrealdevices = [device for device in self.device_manager.devices() if isinstance(device, DeviceUnreal)]
            for device in unrealdevices:
                if not device.is_disconnected:
                    device.fill_derived_data_cache(current_level_only)

        def fill_ddc_current_level_action():
            fill_ddc(True)

        def fill_ddc_all_levels_action():
            fill_ddc(False)

        current_level_action = self.register_tools_menu_action("Current Level", ["Fill DDC (Prepare Shaders)"])
        current_level_action.triggered.connect(fill_ddc_current_level_action)
        all_levels_action = self.register_tools_menu_action("All Levels", ["Fill DDC (Prepare Shaders)"])
        all_levels_action.triggered.connect(fill_ddc_all_levels_action)

    def register_package_game_menuitem(self) -> None:
        ''' Registers a menu item to package the nDisplay cluster game'''

        def package_nDisplay_game(clientconfig: PackagingClientConfig) -> None:
            ''' Callback to kick off the packaging process '''
            ndisplay_devices = [device for device in self.device_manager.devices()
                                if isinstance(device, DevicenDisplay) and not device.is_disconnected]

            # We need at least 1 device connected
            if not len(ndisplay_devices):
                QtWidgets.QMessageBox.information(
                    self.window,
                    "Unable to start packaging",
                    "Please connect to at least one nDisplay node and try again."
                )
                return

            # Make sure we won't try to send the package command to the same client multiple times.

            unique_addresses = {device.address for device in ndisplay_devices}

            if len(ndisplay_devices) > len(unique_addresses):
                QtWidgets.QMessageBox.information(
                    self.window,
                    "Unable to start packaging",
                    "Please only try to package devices with unique addresses."
                )
                return

            # Validate all the devices before starting to package
            for device in ndisplay_devices:
                try:
                    device.package_game(clientconfig=clientconfig, dryrun=True)
                except Exception as e:
                    QtWidgets.QMessageBox.information(
                        self.window,
                        "Unable to start packaging",
                        f"nDisplay '{device.name}': {str(e)}"
                    )
                    return

            # Ok, we should be good to go.
            for device in ndisplay_devices:
                device.package_game(clientconfig=clientconfig)

        actionname = "Package nDisplay Game"

        actiontooltip = '\n'.join([
            "Packages the project in all the nDisplay nodes currently connected to.",
            f"The archive will reside in the location specified by '{DevicenDisplay.csettings['packaged_game_path'].nice_name}'",
            "The Engine and Project source must be available on the remote machine."
            ])

        self.action_package_development = self.register_tools_menu_action(
            "Development", menunames=[actionname], actiontooltip=actiontooltip
        )
        self.action_package_development.triggered.connect(lambda: package_nDisplay_game(PackagingClientConfig.Development))

        self.action_package_shipping = self.register_tools_menu_action(
            "Shipping", menunames=[actionname], actiontooltip=actiontooltip
        )
        self.action_package_shipping.triggered.connect(lambda: package_nDisplay_game(PackagingClientConfig.Shipping))

    def add_tools_menu(self):
        ''' Adds tools menu to menu bar and populates built-in items '''

        self.tools_menu = self.window.menu_bar.addMenu("&Tools")

    def register_tools_menu_action(
            self,
            actionname: str,
            menunames: List[str] = [],
            actiontooltip: str = '') -> QWidgetAction:
        ''' Registers a QWidgetAction with the tools menu

        Args:
            actionname: Name of the action to be added
            menunames: Submenu(s) where to place the given action

        Returns:
            QWidgetAction: The action that was created.
        '''

        # Find find submenu, and create the ones that don't exist along the way

        current_menu: QMenu = self.tools_menu

        # iterate over menunames given
        for menuname in menunames:

            create_menu = True

            # try to find existing menu
            for current_action in current_menu.actions():

                submenu = current_action.menu()

                if submenu and (menuname == submenu.title()):
                    current_menu = submenu
                    create_menu = False
                    break

            # if we didn't find the submenu, create it
            if create_menu:
                newmenu = QMenu(parent=current_menu)
                newmenu.setTitle(menuname)
                current_menu.addMenu(newmenu)
                current_menu = newmenu

        # add the given action
        action = QWidgetAction(current_menu)
        action.setText(actionname)
        action.setToolTip(actiontooltip)
        current_menu.setToolTipsVisible(True)
        current_menu.addAction(action)

        return action

    def on_device_autojoin_changed(self):
        CONFIG.MUSERVER_AUTO_JOIN.update_value(
            self.window.use_device_autojoin_setting_checkbox.isChecked())

    def set_config_hooks(self):
        CONFIG.P4_PROJECT_PATH.signal_setting_changed.connect(lambda: self.p4_refresh_project_cl())
        CONFIG.P4_ENGINE_PATH.signal_setting_changed.connect(lambda: self.p4_refresh_engine_cl())
        CONFIG.ENGINE_SYNC_METHOD.signal_setting_changed.connect(lambda: self.p4_refresh_engine_cl())
        CONFIG.P4_ENABLED.signal_setting_changed.connect(lambda _, enabled: self.toggle_p4_controls(enabled))
        CONFIG.MAPS_PATH.signal_setting_changed.connect(lambda: self.refresh_levels())
        CONFIG.MAPS_FILTER.signal_setting_changed.connect(lambda: self.refresh_levels())
        CONFIG.INSIGHTS_TRACE_ENABLE.signal_setting_changed.connect(lambda: self.refresh_trace_settings())
        CONFIG.INSIGHTS_TRACE_ARGS.signal_setting_changed.connect(lambda: self.refresh_trace_settings())
        CONFIG.INSIGHTS_STAT_EVENTS.signal_setting_changed.connect(lambda: self.refresh_trace_settings())
        CONFIG.MUSERVER_AUTO_JOIN.signal_setting_changed.connect(lambda: self.refresh_muserver_autojoin())
        CONFIG.MUSERVER_SESSION_NAME.signal_setting_changed.connect(lambda: self.refresh_muserver_session_name())
        CONFIG.PROJECT_NAME.signal_setting_changed.connect(lambda: self.refresh_window_title())

    def refresh_trace_settings(self):
        self.window.additional_settings.set_insight_trace_state(CONFIG.INSIGHTS_TRACE_ENABLE.get_value())
        self.window.additional_settings.set_insight_tracing_args(CONFIG.INSIGHTS_TRACE_ARGS.get_value())

    def refresh_muserver_autojoin(self):
        self.window.use_device_autojoin_setting_checkbox.setChecked(CONFIG.MUSERVER_AUTO_JOIN.get_value())

    def refresh_muserver_session_name(self):
        self.set_multiuser_session_name(CONFIG.MUSERVER_SESSION_NAME.get_value())

    def on_tracing_settings_changed(self):
        settings = self.window.additional_settings
        trace_tuple = settings.trace_settings()
        CONFIG.INSIGHTS_TRACE_ENABLE.update_value(trace_tuple[0])
        CONFIG.INSIGHTS_TRACE_ARGS.update_value(trace_tuple[1])

    def show_device_add_menu(self):
        self.device_add_menu.clear()
        plugins = sorted(self.device_manager.available_device_plugins().keys(), key=str.lower)
        for plugin in plugins:
            icons = self.device_manager.plugin_icons(plugin)
            icon = icons["enabled"] if "enabled" in icons.keys() else QtGui.QIcon()
            self.device_add_menu.addAction(icon, plugin)
        self.window.device_add_tool_button.showMenu()

    def on_triggered_add_device(self, action):
        device_type = action.text()
        dialog = self.device_manager.get_device_add_dialog(device_type)
        dialog.exec()

        if dialog.result() == QtWidgets.QDialog.Accepted:
            for device in dialog.devices_to_remove():
                # this is pretty specific to nDisplay. It will remove all existing nDisplay devices before the devices of a new nDisplay config are added.
                # this offers a simple way to update nDisplay should the config file have been changed.
                self.device_manager.remove_device(device)

            self.device_manager.add_devices({device_type: dialog.devices_to_add()})
            CONFIG.save()

    def on_device_removed(self, device_hash, device_type, device_name, update_config):
        self.device_list_widget.on_device_removed(device_hash, device_type, device_name, update_config)
        CONFIG.on_device_removed(device_hash, device_type, device_name, update_config)

    def eventFilter(self, obj, event: QtCore.QEvent):
        if obj == self.window and event.type() == QtCore.QEvent.Close:
            self.close_event_counter += 1
            if not self.should_allow_exit(self.close_event_counter):
                event.ignore()
                return True
            else:
                self.on_exit()

        return self.window.eventFilter(obj, event)

    def should_allow_exit(self, close_req_id: int) -> bool:
        return all(device.should_allow_exit(close_req_id)
                   for device in self.device_manager.devices())

    def on_exit(self):
        if self._periodic_runner:
            self._periodic_runner.exit()
        self.osc_server.close()
        for device in self.device_manager.devices():
            device.disconnect_listener()
        self.window.removeEventFilter(self)

    def transport_queue_menu_about_to_show(self):
        self.transport_queue_menu.clear()

        action = QtWidgets.QWidgetAction(self.transport_queue_menu)
        action.setDefaultWidget(TransportQueueHeaderActionWidget())
        self.transport_queue_menu.addAction(action)

        for job_name in self.transport_queue.transport_jobs.keys():
            action = QtWidgets.QWidgetAction(self.transport_queue_menu)
            action.setDefaultWidget(TransportQueueActionWidget(job_name))
            self.transport_queue_menu.addAction(action)

    def _on_menu_load_config_about_to_show(self):
        from functools import partial

        self.window.menu_load_config.clear()

        # We'll build up a dictionary of directory paths to the submenu for
        # that directory as we go. Config files in the root configs directory
        # will go at the top level.
        menu_hierarchy = {
            config.CONFIG_MGR.config_dir: self.window.menu_load_config
        }

        def _get_menu_for_path(path):
            # Safe guard to make sure we don't accidentally traverse up past
            # the root configs path.
            if (path != config.CONFIG_MGR.config_dir and
                    config.CONFIG_MGR.config_dir not in path.parents):
                return None

            path_menu = menu_hierarchy.get(path)
            if not path_menu:
                parent_menu = _get_menu_for_path(path.parent)
                if not parent_menu:
                    return None

                path_menu = parent_menu.addMenu(path.name)
                menu_hierarchy[path] = path_menu

            return path_menu

        config_paths = config.CONFIG_MGR.list_config_paths()

        # Take a first pass through the config paths just creating the
        # submenus. This makes sure that all submenus appear before any configs
        # for any given menu level.
        for config_path in config_paths:
            _get_menu_for_path(config_path.parent)

        # Now the dictionary of menus should be populated, so create actions
        # for each config in the appropriate menu.
        for config_path in config_paths:
            menu = _get_menu_for_path(config_path.parent)
            if not menu:
                continue

            config_action = menu.addAction(config_path.stem,
                partial(self.set_current_config, config_path))

            if config_path == SETTINGS.CONFIG:
                config_action.setEnabled(False)

        # Make a special entry for a config not in the normal area
        externalconfig_action = QtGui.QAction("Browse...", self.window.menu_load_config)
        externalconfig_action.triggered.connect(self._on_open_external_config)
        self.window.menu_load_config.addAction(externalconfig_action)

    def _on_open_external_config(self):
        ''' When the user wants to open a config located outside of the designated area
        This is useful when the user keeps the configs somewhere else (e.g. under the project)
        '''
        config_path, _ = QtWidgets.QFileDialog.getOpenFileName(
            self.window,
            'Select config file', str(config.CONFIG_MGR.config_dir),
            f'Config files (*{config.CONFIG_SUFFIX})'
        )

        # Do nothing if the user didn't choose a config path
        if not config_path:
            return

        # ok, let's open it
        self.set_current_config(config_path)

    def set_current_config(self, config_path):

        # Update to the new config
        CONFIG.init_with_file_path(config_path)

        SETTINGS.CONFIG = config_path
        SETTINGS.save()

        # Disable saving while loading
        CONFIG.push_saving_allowed(False)

        try:
            # Remove all devices
            self.device_manager.clear_device_list()
            self.device_list_widget.clear_widgets()

            # Reset plugin settings
            self.device_manager.reset_plugins_settings(CONFIG)

            # Set hooks to this dialog's UI
            self.set_config_hooks()

            # Add new devices
            self.device_manager.add_devices(CONFIG._device_data_from_config)
        finally:
            # Re-enable saving after loading.
            CONFIG.pop_saving_allowed()

        self.p4_refresh_project_cl()
        self.p4_refresh_engine_cl()
        self.refresh_levels(CONFIG.get_levels())
        self.update_current_config_text()
        self.refresh_muserver_autojoin()
        self.refresh_muserver_session_name()
        self.refresh_trace_settings()
        self.refresh_window_title()

    def refresh_window_title(self):
        ''' Updates the window title based on the project name and Unreal Engine version '''

        title = ['Switchboard']

        buildver_data = ugs_utils.get_sb_buildver()
        buildver = [
            (f"{buildver_data.get('MajorVersion', '?')}"
             f".{buildver_data.get('MinorVersion', '?')}"
             f".{buildver_data.get('PatchVersion', '?')}"),
            str(buildver_data.get('Changelist', '(unknown CL)')),
            str(buildver_data.get('BranchName', '(unknown branch)'))
        ]

        title.append(f" (Unreal Engine {'-'.join(buildver)})")

        project_name = CONFIG.PROJECT_NAME.get_value()

        if project_name:
            title.append(f' - {project_name}')

        self.window.setWindowTitle(''.join(title))

    def update_current_config_text(self):
        # Can be none when current file is deleted
        if SETTINGS.CONFIG is not None:
            file_name = os.path.basename(SETTINGS.CONFIG)
            self.window.current_config_file_value.setText(file_name)
        else:
            self.window.current_config_file_value.setText("No config loaded")

    def update_current_address_text(self):
        self.window.current_address_value.setText(SETTINGS.ADDRESS.get_value())

    def create_new_config(self, file_path: Union[str, Path], uproject, engine_dir, p4_settings):
        ''' Creates a new config file

        Args:
            file_path: Path to the new config file
            uproject: Path to the unreal project (.uproject) file
            engine_dir: Path to the Engine/ directory
            p4_settings: Desired Perforce settings
        '''

        CONFIG.init_new_config(
            file_path=file_path,
            uproject=uproject,
            engine_dir=engine_dir,
            p4_settings=p4_settings
        )

        # Disable saving while loading
        CONFIG.push_saving_allowed(False)
        try:
            # Remove all devices
            self.device_manager.clear_device_list()
            self.device_list_widget.clear_widgets()

            # Reset plugin settings
            self.device_manager.reset_plugins_settings(CONFIG)

            # Set hooks to this dialog's UI
            self.set_config_hooks()
        finally:
            # Re-enable saving after loading
            CONFIG.pop_saving_allowed()

        # Update the UI
        self.toggle_p4_controls(CONFIG.P4_ENABLED.get_value())
        self.refresh_levels(CONFIG.get_levels())
        self.update_current_config_text()
        self.refresh_muserver_autojoin()
        self.refresh_muserver_session_name()
        self.refresh_trace_settings()
        self.refresh_window_title()

    def menu_save_config_as(self):
        ''' Copy the current config file and move to it '''

        # get the destination path for the copy

        new_config_path, _ = QtWidgets.QFileDialog.getSaveFileName(
            self.window,
            'Select config file', str(config.CONFIG_MGR.config_dir),
            f'Config files (*{config.CONFIG_SUFFIX})'
        )

        # Do nothing if the user didn't choose a destination config path
        if not new_config_path:
            return

        # If replacing the same config, just save
        if Path(new_config_path).resolve() == Path(SETTINGS.CONFIG).resolve():
            CONFIG.save()
            return

        # Switch to the new config
        CONFIG.save_as(new_config_path)

        # Update the settings
        SETTINGS.CONFIG = new_config_path
        SETTINGS.save()

        # Make sure we reflect the new config in the status bar
        self.update_current_config_text()

        # Reset mu server name
        # * unless it is running to avoid devices failing to connect to the current server
        # * if there are devices running, they won't auto-connect when you launch the renamed server
        mu_server = switchboard_application.get_multi_user_server_instance()
        if not mu_server.is_running():
            CONFIG.MUSERVER_SERVER_NAME.update_value(CONFIG.default_mu_server_name())

    def menu_new_config(self):
        uproject_search_path = os.path.dirname(CONFIG.UPROJECT_PATH.get_value().replace('"',''))

        if not os.path.exists(uproject_search_path):
            uproject_search_path = SETTINGS.LAST_BROWSED_PATH

        dialog = AddConfigDialog(
            uproject_search_path=uproject_search_path,
            previous_engine_dir=CONFIG.ENGINE_DIR.get_value(),
            parent=self.window)
        dialog.exec()

        if dialog.result() == QtWidgets.QDialog.Accepted:

            self.create_new_config(
                file_path=dialog.config_path,
                uproject=dialog.uproject, 
                engine_dir=dialog.engine_dir,
                p4_settings=dialog.p4_settings()
            )

    def menu_delete_config(self):
        """
        Delete the currently loaded config.

        After deleting, it will load the first config found by
        config.list_config_paths(), or it will create a new config.
        """
        # Show the confirmation dialog using a relative path to the config.
        rel_config_path = config.CONFIG_MGR.get_relative_config_path(SETTINGS.CONFIG)
        reply = QtWidgets.QMessageBox.question(
            self.window,
            'Delete Config',
            f'Are you sure you would like to delete config "{rel_config_path}"?',
            QtWidgets.QMessageBox.Yes, QtWidgets.QMessageBox.No)

        if reply == QtWidgets.QMessageBox.Yes:
            # Remove the old config
            try:
                SETTINGS.CONFIG.unlink()
                self.set_current_config(None)
            except FileNotFoundError as e:
                LOGGER.error(f'menu_delete_config: {e}')

            # Grab a new config to lead once this one is deleted
            config_paths = config.CONFIG_MGR.list_config_paths()

            if config_paths:
                self.set_current_config(config_paths[0])
            else:
                # Create a blank config
                self.menu_new_config()

    def menu_update_settings(self):
        """
        Settings window
        """
        # TODO: VALIDATE RECORD PATH
        settings_dialog = SettingsDialog(SETTINGS, CONFIG)

        for plugin_name in sorted(self.device_manager.available_device_plugins().keys(), key=str.lower):

            device_instances = self.device_manager.devices_of_type(plugin_name)

            device_settings = [(device.name, device.device_settings(), device.setting_overrides())
                               for device in device_instances]

            plugin_cls = self.device_manager.available_device_plugins()[plugin_name]

            settings_dialog.add_section_for_plugin(
                plugin_name=plugin_name,
                plugin_cls=plugin_cls,
                plugin_settings=self.device_manager.plugin_settings(plugin_name),
                device_settings=device_settings
            )

        settings_dialog.select_all_tab()

        old_address = SETTINGS.ADDRESS.get_value()

        # avoid saving the config all the time while in the settings dialog
        CONFIG.push_saving_allowed(False)
        try:
            # Show the Settings Dialog
            settings_dialog.ui.exec()
        finally:
            # Restore saving, which should happen at the end of this function
            CONFIG.pop_saving_allowed()

        new_config_path = settings_dialog.config_path()
        if new_config_path != SETTINGS.CONFIG and new_config_path is not None:
            CONFIG.replace(new_config_path)
            SETTINGS.CONFIG = new_config_path
            SETTINGS.save()
        if old_address != SETTINGS.ADDRESS.get_value() or SETTINGS.TRANSPORT_PATH.get_value():
            SETTINGS.save()

        self.osc_restart_check(old_address)

        CONFIG.save()

    def osc_restart_check(self, old_address):
        if old_address != SETTINGS.ADDRESS.get_value():
            self.osc_server.close()
            return self.osc_server.launch(SETTINGS.ADDRESS.get_value(), CONFIG.OSC_SERVER_PORT.get_value())
        else:
            return True

    def clear_log_button_clicked(self):
        ''' Called when the "Clear" button of the log window is clicked'''
        self.window.base_console.clear()

    def sync_all_button_clicked(self):
        if not CONFIG.P4_ENABLED.get_value():
            return

        self._sync_build_all(do_sync=True, do_build=False)

    def build_all_button_clicked(self):
        self._sync_build_all(do_sync=False, do_build=True)

    def sync_and_build_all_button_clicked(self):
        if not CONFIG.P4_ENABLED.get_value():
            return

        self._sync_build_all(do_sync=True, do_build=True)

    def _sync_build_all(self, *, do_sync: bool, do_build: bool):
        actions_str = ' and '.join(filter(
            None, ['sync' if do_sync else '', 'build' if do_build else '']))

        workspace_to_device_map: dict[str, Device] = {}

        for device in self.device_list_widget.devices():
            can_sync = device.widget.can_sync()
            can_build = device.widget.can_build()

            if (do_sync and not can_sync) or (do_build and not can_build):
                continue

            engine_workspace = CONFIG.SOURCE_CONTROL_WORKSPACE.get_value(device.name).casefold()
            project_workspace = CONFIG.PROJECT_WORKSPACE.get_value(device.name).casefold()

            workspaces = [ws for ws in [engine_workspace, project_workspace] if ws]

            if len(workspaces) == 0:
                LOGGER.info(f'Skipping {actions_str} for {device.name} because it has empty workspaces')
                continue

            already_queued = False
            for workspace in workspaces:
                if existing_device := workspace_to_device_map.get(workspace):
                    LOGGER.info(f'Skipping {actions_str} for {device.name} '
                                f'because {existing_device.name} is already '
                                'queued using the same workspace')
                    already_queued = True
                    continue

                workspace_to_device_map[workspace] = device

            if already_queued:
                continue

            if do_sync:
                device.widget.sync_button_clicked()

            if do_build:
                device.widget.build_button_clicked()

    def refresh_project_cl_button_clicked(self):
        self.p4_refresh_project_cl()

    def refresh_engine_cl_button_clicked(self):
        self.p4_refresh_engine_cl()

    def connect_all_button_clicked(self, button_state):
        devices = self.device_manager.devices()
        self.set_device_connection_state(devices, button_state)

    def launch_all_button_clicked(self, button_state):
        devices = self.device_manager.devices()
        if button_state:
            # Opening devices.
            self.set_device_launch_state(devices, button_state)
        else:
            # Closing devices - group by type and use device specific close_all_devices()
            devices_by_type = {}
            for device in devices:
                device_type = type(device)
                if device_type not in devices_by_type:
                    devices_by_type[device_type] = []
                devices_by_type[device_type].append(device)
            
            # Call close_all_devices() for each device type
            for device_class, device_list in devices_by_type.items():
                device_class.close_all_devices(device_list)

    def settings_button_clicked(self, button_state):
        self.menu_update_settings()

    def set_device_launch_state(self, devices, launch_state):
        for device in devices:
            try:
                open_button = device.widget.open_button
                if open_button.isEnabled():
                    if launch_state:
                        device.widget._open()
                    elif open_button.isChecked():
                        device.widget._close()
            except Exception:
                pass

    def set_device_connection_state(self, devices, connection_state):
        for device in devices:
            try:
                if connection_state:
                    device.widget._connect()
                else:
                    device.widget._disconnect()
            except Exception:
                pass

    @QtCore.Slot(object)
    def recording_manager_saved(self, recording):
        """
        When the RecordingManager saves a recording
        """
        pass

    # START HERE
    # TODO
    # If JOB IS ADDED, RESUME
    # If DEVICE CONNECTS, RESUME
    def transport_queue_resume(self):
        # Do not allow transport while recording
        if self._is_recording:
            return

        # Do not transport if the active queue is full
        if self.transport_queue.active_queue_full():
            return

        for _, transport_job in self.transport_queue.transport_jobs.items():
            # Do not transport if the device is disconnected
            device = self.device_manager.device_with_name(transport_job.device_name)
            if device.status < DeviceStatus.OPEN:
                continue

            # Only Transport jobs that are ready
            if transport_job.transport_status != recording.TransportStatus.READY_FOR_TRANSPORT:
                continue

            # Transport the file
            self.transport_queue.run_transport_job(transport_job, device)

            # Bail if active queue is full
            if self.transport_queue.active_queue_full():
                break

    @QtCore.Slot(object)
    def transport_queue_job_finished(self, transport_job):
        """
        When the TransportQueue finished a job
        """
        LOGGER.debug(f'transport_queue_job_finished {transport_job.job_name}')
        '''
        # If the device is connected, set that status as READY_FOR_TRANSPORT
        transport_job.transport_status = recording.TransportStatus.READY_FOR_TRANSPORT
        #transport_queue_job_added
        '''

    @QtCore.Slot(object)
    def transport_queue_job_started(self, transport_job):
        """
        When the TransportQueue is ready to transport a new job
        Grab the device and send it back to the transport queue
        """
        LOGGER.debug('transport_queue_job_started')

    @QtCore.Slot(object)
    def device_added(self, device):
        """
        When a new device is added to the DeviceManger, connect its signals
        """
        device.device_qt_handler.signal_device_connect_failed.connect(self.device_connect_failed, QtCore.Qt.QueuedConnection)
        device.device_qt_handler.signal_device_client_disconnected.connect(self.device_client_disconnected, QtCore.Qt.QueuedConnection)
        device.device_qt_handler.signal_device_project_changelist_changed.connect(self.device_project_changelist_changed, QtCore.Qt.QueuedConnection)
        device.device_qt_handler.signal_device_engine_changelist_changed.connect(self.device_engine_changelist_changed, QtCore.Qt.QueuedConnection)
        device.device_qt_handler.signal_device_built_changelist_changed.connect(self.device_built_changelist_changed, QtCore.Qt.QueuedConnection)
        device.device_qt_handler.signal_device_status_changed.connect(self.device_status_changed, QtCore.Qt.QueuedConnection)
        device.device_qt_handler.signal_device_sync_failed.connect(self.device_sync_failed, QtCore.Qt.QueuedConnection)
        device.device_qt_handler.signal_device_is_recording_device_changed.connect(self.device_is_recording_device_changed, QtCore.Qt.QueuedConnection)
        device.device_qt_handler.signal_device_build_update.connect(self.device_build_update, QtCore.Qt.QueuedConnection)
        device.device_qt_handler.signal_device_sync_update.connect(self.device_sync_update, QtCore.Qt.QueuedConnection)
        device.device_qt_handler.signal_device_package_update.connect(self.device_package_update, QtCore.Qt.QueuedConnection)

        # Add the view
        self.device_list_widget.add_device_widget(device)

    @QtCore.Slot(object)
    def register_device_widget(self, device_widget):
        """
        When a new DeviceWidget is added, connect all the signals
        """
        device_widget.signal_device_widget_connect.connect(self.device_widget_connect)
        device_widget.signal_device_widget_disconnect.connect(self.device_widget_disconnect)
        device_widget.signal_device_widget_open.connect(self.device_widget_open)
        device_widget.signal_device_widget_close.connect(self.device_widget_close)
        device_widget.signal_device_widget_sync.connect(self.device_widget_sync)
        device_widget.signal_device_widget_build.connect(self.device_widget_build)
        if hasattr(device_widget, 'signal_device_widget_fill_ddc'):
            device_widget.signal_device_widget_fill_ddc.connect(self.device_widget_fill_ddc)
        device_widget.signal_device_widget_trigger_start_toggled.connect(self.device_widget_trigger_start_toggled)
        device_widget.signal_device_widget_trigger_stop_toggled.connect(self.device_widget_trigger_stop_toggled)

        # Live Link Face Toggle Display
        try:
            device_widget.signal_device_widget_toggle_display.connect(self.device_widget_toggle_display)
        except:
            pass

        # KiPro Signal Support
        try:
            device_widget.signal_device_widget_play.connect(self.device_widget_play)
            device_widget.signal_device_widget_stop.connect(self.device_widget_stop)
        except:
            pass

        try:
            device = self.device_manager.device_with_hash(device_widget.device_hash)
            device.device_widget_registered(device_widget)
        except:
            LOGGER.error(f'Could not find device with hash {device_widget.device_hash} when registering its widget')

    def device_widget_toggle_display(self, device_widget):
        device = self.device_manager.device_with_hash(device_widget.device_hash)
        device.toggle_display()

    def on_all_plugin_devices_connect_toggled(self, plugin_name, button_state):
        devices = self.device_manager.devices_of_type(plugin_name)
        self.set_device_connection_state(devices, button_state)

    def on_all_plugin_devices_open_toggled(self, plugin_name, button_state):
        devices = self.device_manager.devices_of_type(plugin_name)
        
        # Use device specific close_all_devices() method when Stopping.
        if not button_state and devices:
            device_class = type(devices[0])
            device_class.close_all_devices(devices)
        else:
            self.set_device_launch_state(devices, button_state)

    def multiuser_session_name(self):
        return self._multiuser_session_name

    def on_multiuser_session_lineEdit_editingFinished(self):
        text = self.window.multiuser_session_lineEdit.text()
        self.set_multiuser_session_name(text)

    def _on_take_value_changed(self, value):
        ''' This is called when value actually changes (typed input or button clicks)
        '''
        if value != self._take:
            self._set_take(value)

    def on_multiuser_session_inc(self):
        '''
        Increment the session name by 1, or generate default if empty.
        '''
        current_name = self.multiuser_session_name()
        
        # If the session name is empty or just whitespace, generate a default one
        if not current_name or current_name.strip() == '':
            new_session_name = CONFIG.default_mu_session_name()
            self.set_multiuser_session_name(new_session_name)
            return
        
        # Otherwise, use the existing increment logic
        match = re.search(r'\d+$', current_name)
        basename = current_name
        num_as_int = 1
        padding = 0
        if match is not None:
            num_as_str = match.group()
            basename = str.join(num_as_str, current_name.split(num_as_str)[:-1])
            try:
                num_as_int = int(num_as_str) + 1
                padding = len(num_as_str)
            except ValueError:
                # Do not treat value conversion as an error it will just refer back to default value.
                pass
        new_num = f'{num_as_int}'.zfill(padding)
        self.set_multiuser_session_name(f'{basename}{new_num}')

    def set_multiuser_session_name(self, value):

        # sanitize the session name
        value = value.replace(' ', '_')

        self._multiuser_session_name = value

        if self.window.multiuser_session_lineEdit.text() != value:
            self.window.multiuser_session_lineEdit.setText(value)

        if value != CONFIG.MUSERVER_SESSION_NAME.get_value():
            CONFIG.MUSERVER_SESSION_NAME.update_value(value)

    @property
    def shoot(self):
        return self._shoot

    @shoot.setter
    def shoot(self, value):
        self._shoot = value

    @property
    def sequence(self):
        return self._sequence

    @sequence.setter
    def sequence(self, value):
        self._set_sequence(value)

    def _set_sequence(self, value):
        self._sequence = value
        self.window.sequence_line_edit.setText(value)

        # Reset the take number to 1 if setting the sequence
        self.take = 1

    @property
    def slate(self):
        return self._slate

    @slate.setter
    def slate(self, value):
        self._set_slate(value, reset_take=False)

    def _set_slate(self, value, exclude_addresses=[], reset_take=True):
        """
        Internal setter that allows exclusion of addresses
        """
        # Protect against blank slates
        if value == '':
            return

        if self._slate == value:
            return

        self._slate = value
        SETTINGS.CURRENT_SLATE = value
        SETTINGS.save()

        # Reset the take number to 1 if setting the slate
        if reset_take:
            self.take = 1

        # UI out of date with control means external message
        if self.window.slate_line_edit.text() != self._slate:
            self.window.slate_line_edit.setText(self._slate)

        thread = threading.Thread(target=self._set_slate_all_devices, args=[value], kwargs={'exclude_addresses':exclude_addresses})
        thread.start()

    def _set_slate_all_devices(self, value, exclude_addresses=[]):
        """
        Tell all devices the new slate
        """
        for device in self.device_manager.devices():
            if device.address in exclude_addresses:
                continue

            # Do not send updates to disconnected devices
            if device.is_disconnected:
                continue

            device.set_slate(self._slate)

    @property
    def take(self):
        return self._take

    @take.setter
    def take(self, value):
        self._set_take(value)

    def _set_take(self, value, exclude_addresses=[]):
        """
        Internal setter that allows exclusion of addresses
        """
        requested_take = value

        # TODO: Add feedback in UI
        # Check is that slate/take combo has been used before
        while not self.recording_manager.slate_take_available(self._sequence, self._slate, requested_take):
            requested_take += 1

        if requested_take == value == self._take:
            return

        if requested_take != value:
            LOGGER.warning(f'Slate: "{self._slate}" Take: "{value}" have already been used. Auto incremented up to take: "{requested_take}"')
            # Clear the exclude list since Switchboard changed the incoming value
            exclude_addresses = []

        self._take = requested_take
        SETTINGS.CURRENT_TAKE = value
        SETTINGS.save()

        if self.window.take_spin_box.value() != self._take:
            self.window.take_spin_box.setValue(self._take)

        thread = threading.Thread(target=self._set_take_all_devices, args=[value], kwargs={'exclude_addresses':exclude_addresses})
        thread.start()

    def _set_take_all_devices(self, value, exclude_addresses=[]):
        """
        Tell all devices the new take
        """
        # Tell all devices the new take
        for device in self.device_manager.devices():
            # Do not send updates to disconnected devices
            if device.is_disconnected:
                continue

            if device.address in exclude_addresses:
                continue

            device.set_take(self._take)

    @property
    def description(self):
        return self._description

    @description.setter
    def description(self, value):
        self._description = f'{self.level} {self.slate} {self.take}\nvalue'

    @property
    def level(self):
        return self._level

    @level.setter
    def level(self, value):
        self._set_level(value)

    def _on_selected_level_changed(self, index: int):
        full_map_path = self._get_level_from_combo_box(index)
        self._set_level(full_map_path)

    def _get_level_from_combo_box(self, index: int):
        # Data stores full path
        return self.level_combo_box.itemData(index)

    def _set_level(self, value):
        ''' Called when level dropdown text changes
        '''
        self._level = value

        if CONFIG.CURRENT_LEVEL != value:
            CONFIG.CURRENT_LEVEL = value
            CONFIG.save()

        if self.level_combo_box.currentText() != self._level:
            for index in range(self.level_combo_box.count()):
                if self._get_level_from_combo_box(index) == self._level:
                    self.level_combo_box.blockSignals(True)
                    self.level_combo_box.setCurrentIndex(index)
                    self.level_combo_box.blockSignals(False)
                    break

    @property
    def project_changelist(self):
        return self._project_changelist

    @project_changelist.setter
    def project_changelist(self, value):
        self._set_project_changelist(value)

    def _set_project_changelist(self, value):
        self._project_changelist = value

        if self.window.project_cl_combo_box.currentText() != self._project_changelist:
            self.window.project_cl_combo_box.setCurrentText(self._project_changelist)

        # Check if all of the devices are on the right changelist
        for device in self.device_manager.devices():
            if device.project_changelist:
                device_widget = self.device_list_widget.device_widget_by_hash(device.device_hash)
                device_widget.update_project_changelist(
                    required_cl=self.project_changelist if self.project_changelist != EMPTY_SYNC_ENTRY else None,
                    current_device_cl=device.project_changelist
                )

    @property
    def engine_changelist(self):
        return self._engine_changelist

    @engine_changelist.setter
    def engine_changelist(self, value):
        self._set_engine_changelist(value)

    def _set_engine_changelist(self, changelist_value):
        self._engine_changelist = changelist_value

        if self.window.engine_cl_combo_box.currentText() != self._engine_changelist:
            self.window.engine_cl_combo_box.setCurrentText(self._engine_changelist)

        # Check if all of the devices are on the right changelist
        for device in self.device_manager.devices():
            if device.engine_changelist:
                device_widget = self.device_list_widget.device_widget_by_hash(device.device_hash)
                device_widget.update_engine_changelist(
                    required_cl=self.engine_changelist if self.engine_changelist != EMPTY_SYNC_ENTRY else None, 
                    device=device
                )

    @QtCore.Slot(object)
    def device_widget_connect(self, device_widget):
        device = self.device_manager.device_with_hash(device_widget.device_hash)
        if not device:
            return

        if device.device_type == 'LiveLinkFace':
            device.look_for_device = True
        else:
            device.connect_listener()

    @QtCore.Slot(object)
    def device_connect_failed(self, device):
        device_widget = self.device_list_widget.device_widget_by_hash(device.device_hash)

        if not device_widget:
            return

        device_widget._disconnect()

        LOGGER.warning(f'{device.name}: Could not connect to device')

    @QtCore.Slot(object)
    def device_widget_disconnect(self, device_widget):
        device = self.device_manager.device_with_hash(device_widget.device_hash)
        if not device:
            return

        if device.device_type == 'LiveLinkFace':
            device.look_for_device = False
        else:
            device.disconnect_listener()

    @QtCore.Slot(object)
    def device_client_disconnected(self, device):
        device_widget = self.device_list_widget.device_widget_by_hash(device.device_hash)

        device_widget._disconnect()
        LOGGER.warning(f'{device.name}: Client disconnected')

    @QtCore.Slot(object)
    def device_widget_open(self, device_widget):
        device = self.device_manager.device_with_hash(device_widget.device_hash)

        device.launch(self.level)

    @QtCore.Slot(object)
    def device_widget_close(self, device_widget):
        device = self.device_manager.device_with_hash(device_widget.device_hash)
        device.close(force=True)

    @QtCore.Slot(object)
    def device_widget_sync(self, device_widget):
        if not CONFIG.P4_ENABLED.get_value():
            return
        device = self.device_manager.device_with_hash(device_widget.device_hash)
        project_cl = None if self.project_changelist == EMPTY_SYNC_ENTRY else self.project_changelist
        engine_cl = None if self.engine_changelist == EMPTY_SYNC_ENTRY else self.engine_changelist

        workspace = CONFIG.SOURCE_CONTROL_WORKSPACE.get_value(device.name)
        project_workspace = CONFIG.PROJECT_WORKSPACE.get_value(device.name)

        # If the project workspace is separate and there is a target project_cl, then we need to issue separate
        # sync commands for engine and project.
        if project_cl and project_workspace and (project_workspace != workspace):
            puuid = device.sync(engine_cl=engine_cl, project_cl=None)
            device.sync(engine_cl=None, project_cl=project_cl, puuid_dependency=puuid)
        else:
            # Specify engine and project together because they are in the same workspace
            device.sync(engine_cl, project_cl)

    @QtCore.Slot(object)
    def device_widget_build(self, device_widget):
        device = self.device_manager.device_with_hash(device_widget.device_hash)
        device.build()

    @QtCore.Slot(object)
    def device_widget_fill_ddc(self, device_widget, current_level_only):
        device = self.device_manager.device_with_hash(device_widget.device_hash)
        device.fill_derived_data_cache(current_level_only)

    @QtCore.Slot(object)
    def device_widget_play(self, device_widget):
        device = self.device_manager.device_with_hash(device_widget.device_hash)
        device.play()

    @QtCore.Slot(object)
    def device_widget_stop(self, device_widget):
        device = self.device_manager.device_with_hash(device_widget.device_hash)
        device.stop()

    @QtCore.Slot(object)
    def device_sync_failed(self, device):
        #LOGGER.debug(f'{device.name} device_sync_failed')
        # CHANGE THE SYNC ICON HERE
        pass

    @QtCore.Slot(object)
    def device_build_update(self, device, step, percent):
        device_widget = self.device_list_widget.device_widget_by_hash(device.device_hash)
        device_widget.update_build_status(device, step, percent)

    @QtCore.Slot(object)
    def device_sync_update(self, device, progress):
        device_widget = self.device_list_widget.device_widget_by_hash(device.device_hash)
        device_widget.update_sync_status(device, progress)

    @QtCore.Slot(object)
    def device_package_update(self, device, step, percent):
        device_widget = self.device_list_widget.device_widget_by_hash(device.device_hash)
        device_widget.update_package_status(device, step, percent)

    @QtCore.Slot(object)
    def device_project_changelist_changed(self, device):
        device_widget = self.device_list_widget.device_widget_by_hash(device.device_hash)
        device_widget.update_project_changelist(
            required_cl=self.project_changelist if self.project_changelist != EMPTY_SYNC_ENTRY else None,
            current_device_cl=device.project_changelist
        )

        cl = device.project_changelist
        address = device.address
        for device in self.device_manager.devices():
            if device.address == address and device.project_changelist and device.project_changelist != cl:
                device.project_changelist = cl

    @QtCore.Slot(object)
    def device_engine_changelist_changed(self, device):
        device_widget = self.device_list_widget.device_widget_by_hash(device.device_hash)
        device_widget.update_engine_changelist(
            required_cl=self.engine_changelist if self.engine_changelist != EMPTY_SYNC_ENTRY else None,
            device=device
        )

        cl = device.engine_changelist
        address = device.address
        for device in self.device_manager.devices():
            if device.address == address and device.engine_changelist and device.engine_changelist != cl:
                device.engine_changelist = cl

    @QtCore.Slot(object)
    def device_built_changelist_changed(self, device):
        device_widget = self.device_list_widget.device_widget_by_hash(device.device_hash)
        device_widget.update_build_info(device)

    @QtCore.Slot(object)
    def device_status_changed(self, device, previous_status):
        # Update the device widget
        device.widget.update_status(device.status, previous_status, device)

        devices = self.device_manager.devices_of_type(device.device_type)
        self.device_list_widget.update_category_status(device.category_name, devices)
        self.update_connect_and_open_button_states()

        if previous_status != device.status:
            LOGGER.debug(f'{device.name}: device status change: {device.status.name}')

        if previous_status == DeviceStatus.RECORDING and device.status >= DeviceStatus.OPEN:
            self.device_record_stop_confirm(device)
        elif previous_status == DeviceStatus.READY and device.status >= DeviceStatus.RECORDING:
            self.device_record_start_confirm(device)

        # Send Slate/Take to the device when it connects
        if previous_status <= DeviceStatus.OPEN and device.status >= DeviceStatus.READY:
            device.set_take(self.take)
            device.set_slate(self.slate)

    def update_connect_and_open_button_states(self):
        """ Refresh states of connect-all and start-all buttons. """
        connect_checked, connect_enabled, open_checked, open_enabled = \
            self.device_list_widget.get_connect_and_open_all_button_states()

        self.update_connect_all_button_state(checked=connect_checked,
                                             enabled=connect_enabled)
        self.update_start_all_button_state(checked=open_checked,
                                           enabled=open_enabled)

    def update_connect_all_button_state(self, checked, enabled):
        """ Refresh state of connect-all button. """
        self.window.connect_all_button.setChecked(checked)
        self.window.connect_all_button.setEnabled(enabled)
        if checked:
            self.window.connect_all_button.setToolTip("Disconnect all connected devices")
        else:
            self.window.connect_all_button.setToolTip("Connect all devices")

    def update_start_all_button_state(self, checked, enabled):
        """ Refresh state of start-all button. """
        self.window.launch_all_button.setChecked(checked)
        self.window.launch_all_button.setEnabled(enabled)
        if checked:
            self.window.launch_all_button.setToolTip("Stop all running devices")
        else:
            self.window.launch_all_button.setToolTip("Start all connected devices")

    @QtCore.Slot(object)
    def device_is_recording_device_changed(self, device, is_recording_device):
        """
        When the is_recording_device bool changes, fresh the device status to force the repositioning
        of the device in the UI
        """
        self.device_status_changed(device, DeviceStatus.OPEN)

    def device_record_start_confirm(self, device):
        """
        Callback when the device has started recording
        """
        LOGGER.info(f'{device.name}: Recording started') # {timecode}

    def device_record_stop_confirm(self, device):
        """
        Callback when the device has stopped recording
        """
        LOGGER.info(f'{device.name}: Recording stopped')

        latest_recording = self.recording_manager.latest_recording
        device_recording = device.get_device_recording()

        # Add the device to the latest recording
        self.recording_manager.add_device_to_recording(device_recording, latest_recording)
        '''
        # TransportJob
        # If the device produces transport paths, create a transport job
        paths = device.transport_paths(device_recording)
        if not paths:
            return

        # If the status is not on device, do not create jobs
        if device_recording.status != recording.RecordingStatus.ON_DEVICE:
            return

        device_name = device_recording.device_name
        slate = latest_recording.slate
        take = latest_recording.take
        date = latest_recording.date
        job_name = self.transport_queue.job_name(slate, take, device_name)

        # Create a transport job
        transport_job = recording.TransportJob(job_name, device_name, slate, take, date, paths)
        self.transport_queue.add_transport_job(transport_job)
        '''

    @QtCore.Slot(object)
    def device_widget_trigger_start_toggled(self, device_widget, value):
        device = self.device_manager.device_with_hash(device_widget.device_hash)
        device.trigger_start = value

    @QtCore.Slot(object)
    def device_widget_trigger_stop_toggled(self, device_widget, value):
        device = self.device_manager.device_with_hash(device_widget.device_hash)
        device.trigger_stop = value

    def _console_pipe(self, msg):
        """
        Pipes the emiting message from the QtHandler to the base_console widget.
        Scrolls on each emit signal.
        :param msg: This is a built in event, QT Related, not given.
        """
        if self.window:
            self.window.base_console.appendHtml(msg)

        if self.logger_autoscroll:
            self.logger_scroll_to_end()

    def logger_scroll_to_end(self):
        # This combination keeps the cursor at the bottom left corner in all cases.
        if self.window:
            self.window.base_console.moveCursor(QtGui.QTextCursor.End)
            self.window.base_console.moveCursor(QtGui.QTextCursor.StartOfLine)

    # Allow user to change logging level
    def logger_level_comboBox_currentTextChanged(self):
        value = self.window.logger_level_comboBox.currentText()

        if value == 'Message':
            switchboard_logging.QT_HANDLER.setLevel(switchboard_logging.SBLevels.MESSAGE)
        elif value == 'OSC':
            switchboard_logging.QT_HANDLER.setLevel(switchboard_logging.SBLevels.OSC)
        elif value == 'Debug':
            switchboard_logging.QT_HANDLER.setLevel(logging.DEBUG)
        else:
            switchboard_logging.QT_HANDLER.setLevel(logging.INFO)

    def logger_autoscroll_stateChanged(self, value):
        if QtCore.Qt.CheckState(value) == QtCore.Qt.Checked:
            self.logger_autoscroll = True
            self.logger_scroll_to_end()
        else:
            self.logger_autoscroll = False

    def logger_wrap_stateChanged(self, value):
        if QtCore.Qt.CheckState(value) == QtCore.Qt.Checked:
            self.window.base_console.setLineWrapMode(QtWidgets.QPlainTextEdit.LineWrapMode.WidgetWidth)
        else:
            self.window.base_console.setLineWrapMode(QtWidgets.QPlainTextEdit.LineWrapMode.NoWrap)

        if self.logger_autoscroll:
            self.logger_scroll_to_end()


    class ChangelistRunnable(QtCore.QRunnable):
        ''' Retrieves a list of recent changelists for the given path(s) on the thread pool. '''

        class Signals(QtCore.QObject):
            result = QtCore.Signal(list) # list[tuple[int, str]]

        def __init__(self, paths: list[str], client: str | None = None):
            super().__init__()
            self.paths = paths
            self.client = client
            self.signals = self.Signals()

        def run(self):
            cl_desc_pairs = p4_utils.p4_latest_changelists(self.paths, client=self.client)
            self.signals.result.emit(cl_desc_pairs)

    _engine_cl_runnable: ChangelistRunnable | None = None
    _project_cl_runnable: ChangelistRunnable | None = None

    # Update UI with latest CLs
    def p4_refresh_project_cl(self):
        if not CONFIG.P4_ENABLED.get_value():
            return

        if self._project_cl_runnable:
            return # Already refreshing.

        sync_method = CONFIG.ENGINE_SYNC_METHOD.get_value()

        # If we're syncing 'Precompiled Binaries', then that implies that we should be using UGS:
        use_ugs = ENABLE_UGS_SUPPORT and (sync_method in (EngineSyncMethod.Sync_PCBs.value, EngineSyncMethod.Sync_From_UGS.value))

        # Disable project sync options if not using UGS and there isn't a project p4 path set.
        if CONFIG.P4_PROJECT_PATH.get_value() or use_ugs:
            self.window.project_cl_label.setEnabled(True)
            self.window.project_cl_combo_box.setEnabled(True)
            self.window.refresh_project_cl_button.setEnabled(True)
        else:
            self.window.project_cl_label.setEnabled(False)
            self.window.project_cl_combo_box.setEnabled(False)
            self.window.refresh_project_cl_button.setEnabled(False)
            return

        client = CONFIG.SOURCE_CONTROL_WORKSPACE.get_value()
        project_client = CONFIG.PROJECT_WORKSPACE.get_value()

        if project_client:
            client = project_client

        if use_ugs:
            LOGGER.info("Using UnrealGameSync to refresh project changelists.")
            if changelists := ugs_utils.latest_chagelists(Path(CONFIG.UPROJECT_PATH.get_value()), client=client):
                self._on_project_changelists_refreshed(list([(cl, "") for cl in changelists]))
                return
            else:
                LOGGER.error("UnrealGameSync failed to get the project's latest changelists. "
                             "Falling back to using p4 commands directly.")

        LOGGER.info("Refreshing p4 project changelists")
        paths = [f'{CONFIG.P4_PROJECT_PATH.get_value()}/...']

        if DeviceUnreal.ugs_config:
            if addpaths := DeviceUnreal.ugs_config.try_get(
                'Perforce', 'AdditionalPathsToSync'
            ):
                paths.extend([f'//{client}{x}' for x in addpaths])

        # Indeterminate state, resolved on runnable completion
        self.window.project_cl_combo_box.clear()
        self._set_project_changelist('...')
        self.window.refresh_project_cl_button.setEnabled(False)
        self.window.sync_all_button.setEnabled(False)
        self.window.sync_and_build_all_button.setEnabled(False)

        self._project_cl_runnable = SwitchboardDialog.ChangelistRunnable(paths, client=client)
        self._project_cl_runnable.signals.result.connect(self._on_project_changelists_refreshed)
        QtCore.QThreadPool.globalInstance().start(self._project_cl_runnable)

    @QtCore.Slot()
    def _on_project_changelists_refreshed(self, cl_desc_pairs: list[tuple[int, str]]):
        if self._project_cl_runnable:
            self._project_cl_runnable.signals.result.disconnect(self._on_project_changelists_refreshed)
            self._project_cl_runnable = None

        self.window.project_cl_combo_box.clear()
        self.window.refresh_project_cl_button.setEnabled(True)
        no_pending_refresh = (self._project_cl_runnable is None) and (self._engine_cl_runnable is None)
        self.window.sync_all_button.setEnabled(no_pending_refresh)
        self.window.sync_and_build_all_button.setEnabled(no_pending_refresh)

        for (idx, (cl, desc)) in zip(count(), cl_desc_pairs):
            self.window.project_cl_combo_box.addItem(str(cl))
            self.window.project_cl_combo_box.setItemData(idx, desc, QtCore.Qt.ItemDataRole.ToolTipRole)

        self.window.project_cl_combo_box.setCurrentIndex(0)
        self.window.project_cl_combo_box.addItem(EMPTY_SYNC_ENTRY)
        self.window.project_cl_combo_box.setCurrentText(self._project_changelist)

    def p4_refresh_engine_cl(self):
        if not CONFIG.P4_ENABLED.get_value():
            return

        if self._engine_cl_runnable:
            return # Already refreshing.

        self.window.engine_cl_combo_box.clear()
        # if engine is built from source, refresh the engine cl dropdown
        if CONFIG.ENGINE_SYNC_METHOD.get_value() == EngineSyncMethod.Build_Engine.value:
            LOGGER.info("Refreshing p4 engine changelists")
            self.window.engine_cl_label.setEnabled(True)
            self.window.engine_cl_combo_box.setEnabled(True)
            self.window.engine_cl_combo_box.setToolTip("Select changelist to sync the engine to")
            self.window.refresh_engine_cl_button.setEnabled(True)
            self.window.refresh_engine_cl_button.setToolTip("Click to refresh changelists")

            engine_p4_path = CONFIG.P4_ENGINE_PATH.get_value()
            if engine_p4_path:
                # Indeterminate state, resolved on runnable completion
                self.window.engine_cl_combo_box.clear()
                self._set_engine_changelist('...')
                self.window.refresh_engine_cl_button.setEnabled(False)
                self.window.sync_all_button.setEnabled(False)
                self.window.sync_and_build_all_button.setEnabled(False)

                client = CONFIG.SOURCE_CONTROL_WORKSPACE.get_value()
                self._engine_cl_runnable = SwitchboardDialog.ChangelistRunnable([engine_p4_path+'/...'], client=client)
                self._engine_cl_runnable.signals.result.connect(self._on_engine_changelists_refreshed)
                QtCore.QThreadPool.globalInstance().start(self._engine_cl_runnable)
            else:
                LOGGER.warning('"Build Engine" is enabled in the settings but the engine does not seem to be under perforce control.')
                LOGGER.warning("Please check your perforce settings.")
        else:
            # disable engine cl controls if engine is not built from source
            self.window.engine_cl_label.setEnabled(False)
            self.window.engine_cl_combo_box.setEnabled(False)
            tool_tip = "Engine is not build from source. To use this make sure the Engine is on p4 and 'Build Engine' "
            tool_tip += "is enabled in the Settings."
            self.window.engine_cl_combo_box.setToolTip(tool_tip)
            self.window.refresh_engine_cl_button.setEnabled(False)
            self.window.refresh_engine_cl_button.setToolTip(tool_tip)

    @QtCore.Slot()
    def _on_engine_changelists_refreshed(self, cl_desc_pairs: list[tuple[int, str]]):
        if self._engine_cl_runnable:
            self._engine_cl_runnable.signals.result.disconnect(self._on_engine_changelists_refreshed)
            self._engine_cl_runnable = None

        self.window.engine_cl_combo_box.clear()
        self.window.refresh_engine_cl_button.setEnabled(True)
        no_pending_refresh = (self._project_cl_runnable is None) and (self._engine_cl_runnable is None)
        self.window.sync_all_button.setEnabled(no_pending_refresh)
        self.window.sync_and_build_all_button.setEnabled(no_pending_refresh)

        for (idx, (cl, desc)) in zip(count(), cl_desc_pairs):
            self.window.engine_cl_combo_box.addItem(str(cl))
            self.window.engine_cl_combo_box.setItemData(idx, desc, QtCore.Qt.ItemDataRole.ToolTipRole)

        self.window.engine_cl_combo_box.setCurrentIndex(0)
        self.window.engine_cl_combo_box.addItem(EMPTY_SYNC_ENTRY)
        self.window.engine_cl_combo_box.setCurrentText(self._engine_changelist)

    def refresh_levels(self, levels: Optional[List[str]] = None):

        if levels is None or len(levels) == 0:
            levels = CONFIG.find_levels()

        current_level = CONFIG.CURRENT_LEVEL

        # Disable CONFIG saving because _update_level_list will trigger saves when updating
        # the levels combobox even when the level selection hasn't change
        CONFIG.push_saving_allowed(False)

        # try-finally ensures that pop_saving_allowed happens in the case of exceptionsS
        try:
            self.level_combo_box.clear()
            self._update_level_list(self.level_combo_box, levels)

            if current_level and current_level in levels:
                self.level = current_level
        finally:
            CONFIG.pop_saving_allowed()

        # if the current level changed, save the config
        if self.level != current_level:
            CONFIG.save()

    @classmethod
    def disambiguated_base_names(
        cls,
        path_list: list[str],
        disambiguating_function: Callable[[str], str],
    ) -> tuple[list[str], dict[str, str]]:
        '''
        By default, shorten each path in `path_list` to its basename.
        If a given basename appears more than once, `disambiguating_function`
        is called to generate an alternative label (e.g. "A (/Other/Path/)").
        '''
        name_counts: dict[str, int] = {}
        for path in path_list:
            file_name = os.path.basename(path)
            name_counts[file_name] = name_counts.get(file_name, 0) + 1

        # If unique, reduce to basename. Otherwise, append path to end
        short_name_list: list[str] = []
        short_name_to_path: dict[str, str] = {}
        for path in path_list:
            file_name = os.path.basename(path)
            short_name = (file_name if name_counts[file_name] == 1
                          else disambiguating_function(path))
            short_name_list.append(short_name)
            short_name_to_path[short_name] = path

        return short_name_list, short_name_to_path

    def _update_level_list(self, level_combo_box: sb_widgets.SearchableComboBox, level_path_list: List[str]):
        def compare_file_names(path_a: str, path_b: str):
            return -1 if path_a.lower() < path_b.lower() \
                else 1 if path_a.lower() > path_b.lower() else 0

        def disambiguate_level_path(path: str) -> str:
            file_name = os.path.basename(path)
            path = path.removesuffix(file_name)
            if path == "/":
                return f"{file_name}"
            else:
                return f"{file_name} ({path})"

        short_name_list, short_name_to_path = SwitchboardDialog.disambiguated_base_names(
            level_path_list, disambiguate_level_path)

        from functools import cmp_to_key
        short_name_list = sorted(short_name_list, key=cmp_to_key(compare_file_names))
        level_combo_box.addItems([DEFAULT_MAP_TEXT] + short_name_list)

        if len(level_path_list) == 0:
            return

        # To disambiguate, show the full path name in the drop-down
        level_combo_box.setItemData(0, DEFAULT_MAP_TEXT)
        level_combo_box.setItemData(0, "Default level", QtCore.Qt.ToolTipRole)
        for short_name_index in range(len(short_name_list)):
            short_name = short_name_list[short_name_index]
            level_combo_box.setItemData(short_name_index + 1, short_name_to_path[short_name])
            level_combo_box.setItemData(short_name_index + 1, short_name_to_path[short_name], QtCore.Qt.ToolTipRole)

    def get_current_level_list(self):
        level_combo = self.level_combo_box
        return [self._get_level_from_combo_box(i) for i in range(1, level_combo.count())] # skip DEFAULT_MAP_TEXT

    def browse_to_level_and_add_it(self) -> Optional[Path]:
        """
        Show a file‐open dialog for .umap files, then add & select the chosen level.

        Returns:
            The gamepath to the chosen .umap if one was picked, otherwise None.
        """

        maps_path = os.path.normpath(
            os.path.join(
                CONFIG.get_project_content_dir(),
                CONFIG.MAPS_PATH.get_value())
        )
        
        file_path, _ = QtWidgets.QFileDialog.getOpenFileName(
            parent=None,
            caption="Browse to Level",
            dir=maps_path,
            filter="Unreal Level Files (*.umap)",
            selectedFilter="",
            options=QtWidgets.QFileDialog.Options()
        )

        if not file_path:
            return None

        umap_filepath = Path(file_path)
        
        # Check extension
        if umap_filepath.suffix.lower() != ".umap":
            LOGGER.warning(f"Selected file isn’t a .umap: {umap_filepath}")
            return None

        umap_gamepath = CONFIG.deduce_umap_gamepath_from_filepath(umap_filepath)

        if not umap_gamepath:
            return None

        levels = CONFIG.add_single_level(umap_gamepath)

        # Update the UI

        self.refresh_levels(levels)
        self.level = umap_gamepath  # saves the CONFIG if the level changed.

        return umap_gamepath

    def refresh_levels_incremental(self):
        ''' Wrapper around `refresh_levels` with the following differences:
            - Only resaves config if the selected level was removed (as opposed
              to always generating 2-3 change events/config saves).
            - Logs messages indicating which levels were added/removed.
        '''

        current_level: str = CONFIG.CURRENT_LEVEL
        prev_levels_list: List[str] = self.get_current_level_list()
        updated_levels_list: List[str] = CONFIG.find_levels()

        prev_levels: Set[str] = set(prev_levels_list)
        updated_levels: Set[str] = set(updated_levels_list)
        levels_added = updated_levels - prev_levels
        levels_removed = prev_levels - updated_levels

        if not (levels_added or levels_removed):
            return

        if levels_added:
            LOGGER.info(f'Levels added: {", ".join(levels_added)}')

        if levels_removed:
            LOGGER.info(f'Levels removed: {", ".join(levels_removed)}')

        if current_level in levels_removed:
            LOGGER.warning(f'Selected level "{current_level}" was removed; reverting to default')
            self.level = DEFAULT_MAP_TEXT

        self.refresh_levels(updated_levels_list)

    def toggle_p4_controls(self, enabled):
        self.window.engine_cl_label.setEnabled(enabled)
        self.window.engine_cl_combo_box.setEnabled(enabled)
        self.window.refresh_engine_cl_button.setEnabled(enabled)

        self.window.project_cl_label.setEnabled(enabled)
        self.window.project_cl_combo_box.setEnabled(enabled)
        self.window.refresh_project_cl_button.setEnabled(enabled)

        self.window.sync_all_button.setEnabled(enabled)
        self.window.sync_and_build_all_button.setEnabled(enabled)
        if enabled:
            self.p4_refresh_engine_cl()
            self.p4_refresh_project_cl()

    def osc_take(self, address, command, value):
        device = self._osc_device_from_address(address, command, value=value)

        if not device:
            return

        self._set_take(value, exclude_addresses=[device.address])

    # OSC: Set Slate
    def osc_slate(self, address, command, value):
        device = self._osc_device_from_address(address, command, value=value)
        if not device:
            return

        self._set_slate(value, exclude_addresses=[device.address], reset_take=False)

    # OSC: Set Description UPDATE THIS TO MAKE IT WORK
    def osc_slate_description(self, address, command, value):
        self.description = value

    def record_button_released(self):
        """
        User press record button
        """
        if self._is_recording:
            LOGGER.debug('Record stop button pressed')
            self._record_stop(1)
            self.window.take_spin_box.setValue(self.window.take_spin_box.value() + 1)
        else:
            LOGGER.debug('Record start button pressed')
            self._record_start(self.slate, self.take, self.description)

    def _osc_device_from_address(self, address, command, value=''):
        device: Device | None = None

        for check_device in self.device_manager.devices():
            if (check_device.address == address[0]) and (check_device.osc_client is not None):
                device = check_device
                break

        if not device:
            LOGGER.warning(f'{address} is not registered with an OSC device in Switchboard')
            return None

        # Do not receive osc from disconnected devices
        if device.is_disconnected:
            LOGGER.warning(f'{device.name}: is sending OSC commands but is not connected. Ignoring "{command} {value}"')
            return None

        LOGGER.osc(f'OSC Server: Received "{command} {value}" from {device.name} ({device.address})')
        return device

    # OSC: Start a recording
    def osc_record_start(self, address, command, slate, take, description):
        '''
        OSC message Recieved /RecordStart
        '''
        device = self._osc_device_from_address(address, command, value=[slate, take, description])
        if not device:
            return

        # There is a bug that causes a slate of None. If this occurs, use the stored slate in control
        # Try to track down this bug in sequencer
        if not slate or slate == 'None':
            LOGGER.critical(f'Slate is None, using {self.slate}')
        else:
            self.slate = slate

        self.take = take
        self.description = description

        self._record_start(self.slate, self.take, self.description, exclude_address=device.address)

    def _on_recording_timer(self):
        self._update_recording_status()

    def _fmt_rec_duration(self, timedelta: datetime.timedelta) -> str:
        mm, ss = divmod(int(timedelta.total_seconds() + 0.5), 60)
        hh, mm = divmod(mm, 60)
        result = ''
        if hh:
            result += f'{hh:02}' + (' hours, ' if hh != 1 else ' hour, ')
        if mm or hh:
            result += f'{mm:02}' + (' mins, ' if mm != 1 else ' min, ')
        result += f'{ss:02}'  + (' secs ' if ss != 1 else ' sec ')
        return result

    def _update_recording_status(self):
        self.window.recording_status.setStyleSheet('')

        if not self._recording_start_time:
            self.window.recording_status.setText('Not Recording ')
        elif self._recording_end_time:
            recording_duration = self._recording_end_time - self._recording_start_time
            self.window.recording_status.setText('Last Recording: ' + self._fmt_rec_duration(recording_duration))
        else:
            recording_duration = datetime.datetime.now() - self._recording_start_time
            self.window.recording_status.setText('Recording Duration: ' + self._fmt_rec_duration(recording_duration))
            self.window.recording_status.setStyleSheet('color: #e52d00;')

    def _record_start(self, slate, take, description, exclude_address=None):
        LOGGER.success(f'Record Start: "{self.slate}" {self.take}')

        # Update the UI button
        pixmap = QtGui.QPixmap(":/icons/images/record_start.png")
        self.window.record_button.setIcon(QtGui.QIcon(pixmap))

        # Pause the TransportQueue
        #self.transport_queue.pause()

        # Start a new recording
        self._is_recording = True
        self._recording_start_time = datetime.datetime.now()
        self._recording_end_time = None
        self._recording_timer.start(1000)
        self._update_recording_status()

        # TODO: Lock SLATE/TAKE/SESSION/CL

        # Return a Recording object
        new_recording = recording.Recording()
        new_recording.project = CONFIG.PROJECT_NAME.get_value()
        new_recording.shoot = self.shoot
        new_recording.sequence = self.sequence
        new_recording.slate = self.slate
        new_recording.take = self.take
        new_recording.description = self.description
        new_recording.date = switchboard_utils.date_to_string(self._recording_start_time.date())
        new_recording.map = self.level
        new_recording.multiuser_session = self.multiuser_session_name()
        new_recording.changelist = self.project_changelist

        self.recording_manager.add_recording(new_recording)

        # Sends the message to all recording devices
        devices = self.device_manager.devices()
        for device in devices:
            # Do not send a start record message to whichever device sent it
            if exclude_address and exclude_address == device.address:
                continue

            # Do not send updates to disconnected devices
            if device.is_disconnected:
                continue

            LOGGER.debug(f'Record Start {device.name}')
            device.record_start(slate, take, description)

    def _record_stop(self, exclude_address=None):
        LOGGER.success(f'Record Stop: "{self.slate}" {self.take}')

        pixmap = QtGui.QPixmap(":/icons/images/record_stop.png")
        self.window.record_button.setIcon(QtGui.QIcon(pixmap))

        # Resume the transport queue
        #self.transport_queue.resume()

        # End the recording
        self._is_recording = False
        self._recording_end_time = datetime.datetime.now()
        self._recording_timer.stop()
        self._update_recording_status()

        # Pull the latest recording down
        new_recording = self.recording_manager.latest_recording
        # Start the auto_save timer
        self.recording_manager.auto_save(new_recording)

        # TODO: If no UE4 auto incriment 

        # TODO: Unlock SLATE/TAKE/SESSION/CL

        devices = self.device_manager.devices()

        # Sends the message to all recording devices
        for device in devices:
            # Do not send a start record message to whichever device sent it
            if exclude_address and exclude_address == device.address:
                continue

            # Do not send updates to disconnected devices
            if device.is_disconnected:
                continue

            device.record_stop()

    def _record_cancel(self, exclude_address=None):
        self._record_stop(exclude_address=exclude_address)

        # Increment Take
        #new_recording = self.recording_manager.latest_recording
        #self.take = new_recording.take + 1

    def osc_record_start_confirm(self, address, command, timecode):
        device = self._osc_device_from_address(address, command, value=timecode)
        if not device:
            return

        device.record_start_confirm(timecode)

    def osc_record_stop(self, address, command):
        device = self._osc_device_from_address(address, command)
        if not device:
            return

        self._record_stop(exclude_address=device.address)

    def osc_record_stop_confirm(self, address, command, timecode, *paths):
        device = self._osc_device_from_address(address, command, value=timecode)
        if not device:
            return

        if not paths:
            paths = None
        device.record_stop_confirm(timecode, paths=paths)

    def osc_record_cancel(self, address, command):
        """
        This is called when record has been pressed and stopped before the countdown in take recorder
        has finished
        """
        device = self._osc_device_from_address(address, command)
        if not device:
            return

        self._record_cancel(exclude_address=device.address)

    def osc_record_cancel_confirm(self, address, command, timecode):
        pass
        #device = self._osc_device_from_address(address, command, value=timecode)
        #if not device:
        #    return

        #self.record_cancel_confirm(device, timecode)

    def osc_ue4_launch_confirm(self, address, command):
        device = self._osc_device_from_address(address, command)
        if not device:
            return

        # If the device is already ready, bail
        if device.status == DeviceStatus.READY:
            return

        # Set the device status to ready
        device.status = DeviceStatus.READY

    def osc_add_send_target_confirm(self, address, command, value):
        device = self.device_manager.device_with_address(address[0])
        if not device:
            return

        device.osc_add_send_target_confirm()

    def osc_arsession_stop_confirm(self, address, command, value):
        LOGGER.debug(f'osc_arsession_stop_confirm {value}')

    def osc_arsession_start_confirm(self, address, command, value):
        device = self._osc_device_from_address(address, command, value=value)
        if not device:
            return

        device.connect_listener()

    def osc_battery(self, address, command, value):
        # The Battery command is used to handshake with LiveLinkFace. Don't reject it if it's not connected 
        device = self.device_manager.device_with_address(address[0])

        if not device:
            return

        # Update the device
        device.battery = value

        # Update the UI
        device_widget = self.device_list_widget.device_widget_by_hash(device.device_hash)
        device_widget.set_battery(value)

    def osc_data(self, address, command, value):
        device = self._osc_device_from_address(address, command)
        if not device:
            return


class TransportQueueHeaderActionWidget(QtWidgets.QWidget):
    def __init__(self, parent=None):
        super().__init__(parent)

        self.layout = QtWidgets.QHBoxLayout()
        self.layout.setContentsMargins(0, 0, 0, 0)
        self.setLayout(self.layout)

        def __label(label_text):
            label = QtWidgets.QLabel()
            label.setText(label_text)
            label.setObjectName('widget_header')
            return label

        self.name_label = __label('Transport Queue')

        self.layout.addWidget(self.name_label)

    def paintEvent(self, event):
        opt = QtWidgets.QStyleOption()
        opt.initFrom(self)
        painter = QtGui.QPainter(self)
        self.style().drawPrimitive(QtWidgets.QStyle.PE_Widget, opt, painter, self)


class TransportQueueActionWidget(QtWidgets.QWidget):
    def __init__(self, name, parent=None):
        super().__init__(parent)

        self.layout = QtWidgets.QHBoxLayout()
        self.layout.setContentsMargins(20, 2, 20, 2)
        self.layout.setSpacing(2)
        self.setLayout(self.layout)

        # Job Label
        label = QtWidgets.QLabel(name)
        self.layout.addWidget(label)

        spacer = QtWidgets.QSpacerItem(0, 20, QtWidgets.QSizePolicy.Expanding, QtWidgets.QSizePolicy.Minimum)
        self.layout.addItem(spacer)

        # Remove button
        button = sb_widgets.ControlQPushButton()
        sb_widgets.set_qt_property(button, 'no_background', True)

        icon = QtGui.QIcon()
        pixmap = QtGui.QPixmap(f":/icons/images/icon_close_disabled.png")
        icon.addPixmap(pixmap, QtGui.QIcon.Normal, QtGui.QIcon.Off)
        pixmap = QtGui.QPixmap(f":/icons/images/icon_close.png")
        icon.addPixmap(pixmap, QtGui.QIcon.Normal, QtGui.QIcon.On)
        pixmap = QtGui.QPixmap(f":/icons/images/icon_close_hover.png")
        icon.addPixmap(pixmap, QtGui.QIcon.Active, QtGui.QIcon.Off)

        button.setIcon(icon)
        button.setIconSize(pixmap.rect().size()*0.75)
        button.setMinimumSize(QtCore.QSize(20, 20))

        button.setCheckable(False)
        #button.clicked.connect(self.device_button_clicked)
        self.layout.addWidget(button)

    def test(self):
        LOGGER.debug('BOOM!')

    def paintEvent(self, event):
        opt = QtWidgets.QStyleOption()
        opt.initFrom(self)
        painter = QtGui.QPainter(self)
        self.style().drawPrimitive(QtWidgets.QStyle.PE_Widget, opt, painter, self)
