# Copyright Epic Games, Inc. All Rights Reserved.

import datetime
from enum import IntEnum
from io import StringIO
import logging
import os
import re
import sys
import tempfile
from typing import cast, Optional

from PySide6 import QtCore


DEFAULT_LOGGER_NAME = 'switchboard'
LOGGING_FMT = '%(asctime)s [%(levelname)-8s] <%(name)s>: %(message)s'


class SBLevels(IntEnum):
    MESSAGE = logging.DEBUG - 2
    OSC = logging.DEBUG - 1
    SUCCESS = logging.INFO + 2


class QtHandler(logging.Handler):
    LEADING_SPACES_RE = re.compile(r'^( +)', re.MULTILINE)

    def __init__(self):
        super().__init__()
        self.formatter = logging.Formatter()

    def emit(self, record: logging.LogRecord):

        # Filter spammy debug messages from UI (but they can still be found in the log file)
        if record.name in ('quic', 'asyncio') and record.levelno < logging.WARNING:
            return
            
        html_record = self.format(record)
        ConsoleStream.stdout().write(html_record)
        ConsoleStream.stderr().write(html_record)

    def format(self, record: logging.LogRecord):
        match record.levelno:
            case logging.DEBUG:
                initial = 'D'
                color = '#66D9EF'
            case logging.INFO:
                initial = 'I'
                color = 'white'
            case logging.WARNING:
                initial = 'W'
                color = 'yellow'  # E6DB74
            case logging.CRITICAL:
                initial = 'C'
                color = '#FD971F'
            case logging.ERROR:
                initial = 'D'
                color = '#F92672'
            case SBLevels.OSC:
                initial = 'O'
                color = '#4F86C6'
            case SBLevels.MESSAGE:
                initial = 'M'
                color = '#7b92ad'
            case SBLevels.SUCCESS:
                initial = 'S'
                color = '#A6E22E'
            case _:
                initial = '?'
                color = '#ff00ff'

        time_str = datetime.datetime.now().strftime('%H:%M:%S')
        msg = self.formatter.format(record) if self.formatter else record.getMessage()

        # HTML escapes
        msg = re.sub(self.LEADING_SPACES_RE, lambda m: '&nbsp;' * len(m.group(1)), msg)
        msg = msg.replace('<', '&lt;')
        msg = msg.replace('\n', '<br>')

        logger_name = f" &lt;{record.name}>" if record.name != DEFAULT_LOGGER_NAME else ""

        return f"""
        <span style="margin: 0px; display: block">
            <font color="grey">
                [{time_str}] [{initial}]{logger_name}:
            </font>
            <font color="{color}">{msg}</font>
        </span>
        """


class ConsoleStream(QtCore.QObject):
    _stdout = None
    _stderr = None
    message_written = QtCore.Signal(str)

    def write(self, message):
        if not self.signalsBlocked():
            self.message_written.emit(message)

    @staticmethod
    def stdout():
        if not ConsoleStream._stdout:
            ConsoleStream._stdout = ConsoleStream()
            sys.stdout = ConsoleStream._stdout
        return ConsoleStream._stdout

    @staticmethod
    def stderr():
        if not ConsoleStream._stderr:
            ConsoleStream._stderr = ConsoleStream()
            sys.stdout = ConsoleStream._stderr
        return ConsoleStream._stderr

    def flush(self):
        pass


class SBLogger(logging.getLoggerClass()):
    """ Adds convenience functions for our custom verbosity levels. """

    def __init__(self, name: str, level: int = logging.NOTSET):
        super().__init__(name, level)

        self.file_handler: Optional[logging.FileHandler] = None
        self.log_path: Optional[str] = None

    def success(self, message, *args, **kwargs):
        self.log(SBLevels.SUCCESS, message, *args, **kwargs)

    def osc(self, message, *args, **kwargs):
        self.log(SBLevels.OSC, message, *args, **kwargs)

    def message(self, message, *args, **kwargs):
        self.log(SBLevels.MESSAGE, message, *args, **kwargs)

    @staticmethod
    def make_log_path(timestamp: datetime.datetime | None = None) -> str:
        DEFAULT_LOG_DIR = os.path.join(tempfile.gettempdir(), 'switchboard')
        if not os.path.isdir(DEFAULT_LOG_DIR):
            os.makedirs(DEFAULT_LOG_DIR)
        if not timestamp:
            timestamp = datetime.datetime.now()
        datetime_suffix = timestamp.strftime('%Y-%m-%d_%H-%M-%S')
        return os.path.join(DEFAULT_LOG_DIR, f"switchboard_{datetime_suffix}.log")


logging.basicConfig(level=logging.DEBUG, format=LOGGING_FMT)
logging.addLevelName(SBLevels.OSC, 'OSC')
logging.addLevelName(SBLevels.SUCCESS, 'SUCCESS')
logging.addLevelName(SBLevels.MESSAGE, 'MESSAGE')

logging.setLoggerClass(SBLogger)
LOGGER = cast(SBLogger, logging.getLogger(DEFAULT_LOGGER_NAME))

# Add our handlers to the root
ROOT_LOGGER = logging.getLogger()

QT_HANDLER = QtHandler()
ROOT_LOGGER.addHandler(QT_HANDLER) 

FILE_HANDLER = logging.FileHandler(SBLogger.make_log_path())
FILE_HANDLER.setFormatter(logging.Formatter(fmt=LOGGING_FMT))
ROOT_LOGGER.addHandler(FILE_HANDLER)

# Note: QUIC and asyncio debug message filtering is handled in QtHandler.emit()
# This allows debug messages to still reach the file log while suppressing UI spam
