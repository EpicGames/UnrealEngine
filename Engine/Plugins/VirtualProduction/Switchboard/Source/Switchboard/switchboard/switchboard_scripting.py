# Copyright Epic Games, Inc. All Rights Reserved.

import sys
import importlib
import inspect

from pathlib import Path

from switchboard.switchboard_logging import LOGGER
from switchboard.switchboard_dialog import SwitchboardDialog


class SwitchboardScriptBase:
    ''' Base class for script files that can be passed in the command line using -script and -scriptargs'''

    scriptargs = ''

    def __init__(self, scriptargs):
        self.scriptargs = scriptargs

    def on_preinit(self):
        ''' Called before SB UI initializes'''
        pass

    def on_postinit(self, app: SwitchboardDialog):
        ''' Called after SB UI initializes'''
        pass

    def on_exit(self):
        ''' Called right before the application exits'''
        pass


class ScriptManager:

    scripts = []

    def add_script_from_path(self, scriptpath, scriptargs):

        # file existence sanity check
        path = Path(scriptpath)

        if not path.is_file():
            raise FileNotFoundError

        # insert its folder to sys path to import it
        script_directory = str(path.parents[0].absolute())
        sys.path.insert(0, script_directory)

        # import the module by name
        scriptmod = importlib.import_module(path.stem)

        # find SwitchboardScriptBase subclasses. 
        # They will all share the same scriptargs
        for name, scriptclass in inspect.getmembers(scriptmod):
            if inspect.isclass(scriptclass) and scriptclass in SwitchboardScriptBase.__subclasses__():

                try:
                    script = scriptclass(scriptargs)
                except Exception as e:
                    LOGGER.error(f"Could not initialize script '{name}': {e}")
                    raise

                self.scripts.append(script)

    def on_preinit(self):

        for script in self.scripts:
            try:
                script.on_preinit()
            except Exception as e:
                LOGGER.error(e)

    def on_postinit(self, app):

        for script in self.scripts:
            try:
                script.on_postinit(app)
            except Exception as e:
                LOGGER.error(e)

    def on_exit(self):

        for script in self.scripts:
            try:
                script.on_exit()
            except Exception as e:
                LOGGER.error(e)
