# Copyright Epic Games, Inc. All Rights Reserved.

import csv
import datetime
from enum import Enum
import io
import os
import subprocess
import threading
from typing import Optional

# NOTE: This file should not depend on Qt, directly or indirectly.


class PriorityModifier(Enum):
    ''' Corresponds to `int32 PriorityModifier` parameter to `FPlatformProcess::CreateProc`. '''
    Low = -2
    Below_Normal = -1
    Normal = 0
    Above_Normal = 1
    High = 2


def get_hidden_sp_startupinfo():
    ''' Returns subprocess.startupinfo and avoids extra cmd line window in Windows. '''
    startupinfo = None

    if hasattr(subprocess, 'STARTUPINFO') and hasattr(subprocess, 'STARTF_USESHOWWINDOW'):
        startupinfo = subprocess.STARTUPINFO()
        startupinfo.dwFlags |= subprocess.STARTF_USESHOWWINDOW
        startupinfo.wShowWindow = subprocess.SW_HIDE

    return startupinfo


class RepeatFunction(object):
    """
    Repeate a function every interval until timeout is reached or the function returns True
    """
    def __init__(self, interval, timeout, function, *args, **kwargs):
        self.interval = interval
        self.timeout = datetime.timedelta(seconds=timeout)
        self.function = function
        self.args = args
        self.kwargs = kwargs

        self.results_function = None

        self.finish_function = None
        self.finish_args = None
        self.finish_kwargs = None

        self.timeout_function = None
        self.timeout_args = None
        self.timeout_kwargs = None

        self.start_time = None
        self._started = False

        self._stop = False

    def add_finish_callback(self, function, *args, **kwargs):
        # When finish is succesfull
        self.finish_function = function
        self.finish_args = args
        self.finish_kwargs = kwargs

    def add_timeout_callback(self, function, *args, **kwargs):
        self.timeout_function = function
        self.timeout_args = args
        self.timeout_kwargs = kwargs

    def start(self, results_function=None):
        self.start_time = datetime.datetime.now()
        self.results_function = results_function

        self._run()

    def stop(self):
        self._stop = True

    def _run(self):
        if self._stop:
            return

        time_delta = (datetime.datetime.now() - self.start_time)

        if time_delta > self.timeout and self._started:
            if self.timeout_function:
                self.timeout_function(*self.timeout_args, **self.timeout_kwargs)
            return

        # Be sure the function runs atleast 1 time
        self._started = True

        # Run the function
        results = self.function(*self.args, **self.kwargs)

        if results and self.results_function:
            if self.results_function(results):
                if self.finish_function:
                    self.finish_function(*self.finish_args, **self.finish_kwargs)
                return

        threading.Timer(self.interval, self._run).start()


class PollProcess(object):
    '''
    Have the same signature as a popen object as a backup if a ue4 process is already running
    on the machine
    '''
    def __init__(self, task_name: str):
        self.task_name = task_name

    @property
    def args(self) -> str:
        get_commandline_cmd = f"wmic process where caption=\"{self.task_name}\" get commandline /format:csv"
        try:
            output = subprocess.check_output(get_commandline_cmd, startupinfo=get_hidden_sp_startupinfo()).decode()
            dictobj = next(csv.DictReader(io.StringIO(output.strip())))
            return dictobj['CommandLine'] if dictobj else ''
        except:
            return ""

    @property
    def pid(self) -> Optional[int]:
        pid_cmd = f"tasklist /FI \"IMAGENAME eq {self.task_name}\" /FO csv"
        try:
            output = subprocess.check_output(pid_cmd, startupinfo=get_hidden_sp_startupinfo()).decode()
            dictobj = next(csv.DictReader(io.StringIO(output)))
            if dictobj:
                return int(dictobj['PID'])
        except:
            return None

    def poll(self):
        # 'list' output format because default 'table' truncates imagename to 25 characters
        tasklist_cmd = f"tasklist /FI \"IMAGENAME eq {self.task_name}\" /FO list"
        try: # Figure out what is happening here OSError: [WinError 6] The handle is invalid
            tasklist_output = subprocess.check_output(tasklist_cmd, startupinfo=get_hidden_sp_startupinfo()).decode()

            #p = re.compile(f"{self.task_name} (.*?) Console")
            # Some process do not always list the .exe in the name
            if os.path.splitext(self.task_name)[0].lower() in tasklist_output.lower():
                return None
            return True
        except:
            return True

    def kill(self):
        try:
            subprocess.check_output(f"taskkill.exe /F /IM {self.task_name}", startupinfo=get_hidden_sp_startupinfo())
        except:
            pass


def download_file(url, local_filename):
    import requests
    with requests.get(url, stream=True) as r:
        r.raise_for_status()
        with open(local_filename, 'wb') as f:
            for chunk in r.iter_content(chunk_size=8192):
                if chunk:
                    f.write(chunk)
    return local_filename


def capture_name(slate, take):
    return f'{slate}_T{take}'


def date_to_string(date):
    return date.strftime("%y%m%d")


def remove_prefix(str, prefix):
    if str.startswith(prefix):
        return str[len(prefix):]
    return str


def expand_endpoint(
        endpoint: str, default_addr: str = '0.0.0.0',
        default_port: int = 0) -> str:
    '''
    Given an endpoint where either address or port was omitted, use
    the provided defaults.
    '''
    addr_str, _, port_str = endpoint.partition(':')

    if not addr_str:
        addr_str = default_addr

    if not port_str:
        port_str = str(default_port)

    return f'{addr_str}:{port_str}'
