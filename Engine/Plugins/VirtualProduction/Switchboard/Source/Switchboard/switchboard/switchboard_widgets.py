# Copyright Epic Games, Inc. All Rights Reserved.
import re
import socket
import time
from typing import Dict, Optional

from PySide6 import QtCore, QtGui, QtWidgets

from switchboard.switchboard_logging import LOGGER


DEVICE_LIST_WIDGET_HEIGHT = 54
DEVICE_HEADER_LIST_WIDGET_HEIGHT = 40
DEVICE_WIDGET_HIDE_ADDRESS_WIDTH = 500


class NonScrollableComboBox(QtWidgets.QComboBox):
    onHoverScrollBox = QtCore.Signal()

    def __init__(self, parent=None):
        super().__init__(parent=parent)
        self.installEventFilter(self)

    def enterEvent(self, event):
        self.onHoverScrollBox.emit()
        super().enterEvent(event)

    def eventFilter(self, obj, event):
        if obj == self and event.type() == QtCore.QEvent.Wheel:
            event.ignore()
            return True
        return False


# Each entry has a checkbox, the first element is a lineedit that will show all selected entries
class MultiSelectionComboBox(QtWidgets.QComboBox):
    separator = ' | '
    signal_selection_changed = QtCore.Signal(list)

    def __init__(self, parent=None):
        super().__init__(parent=parent)
        self.model().itemChanged.connect(self.on_stateChanged)

        # Only "editable" to use the edit control for display; read-only, selection disallowed.
        self.setEditable(True)
        self.lineEdit().setReadOnly(True)
        self.lineEdit().selectionChanged.connect(lambda: self.lineEdit().setSelection(0, 0))

        # Hook mouse button events on the line edit to open/close the combo.
        self.lineEdit().installEventFilter(self)

        super().addItem("")
        item = self.model().item(0, 0)
        item.setEnabled(False)

        # the combo calls show/hidePopup internally, to avoid messing up the state we only allow showing/hiding the popup
        # if it happened inside a time intervall that could have reasonably been triggered by a user.
        # this is obviously a workaround but there seems to be no way to get better behavior w/o implementing a full-blown combobox.
        self.last_time_popup_was_triggered = time.time()
        self.popup_toggle_min_interval = 0.2
        self.popup_is_showing = False

    def add_items(self, selected_entries, all_entries):
        for entry in all_entries:
            self.addItem(str(entry), entry)
            item = self.model().item(self.count()-1, 0)
            item.setFlags(QtCore.Qt.ItemIsUserCheckable | QtCore.Qt.ItemIsEnabled)
            state = QtCore.Qt.Checked if entry in selected_entries else QtCore.Qt.Unchecked
            item.setCheckState(state)

        invalid_entries = [entry for entry in selected_entries if entry not in all_entries]
        for entry in invalid_entries:
            self.addItem(str(entry), entry)
            item = self.model().item(self.count()-1, 0)
            item.setFlags(QtCore.Qt.ItemIsUserCheckable | QtCore.Qt.ItemIsEnabled)
            item.setCheckState(QtCore.Qt.Checked)

            brush = item.foreground()
            brush.setColor(QtCore.Qt.red)
            item.setForeground(brush)

    def eventFilter(self, obj, event):
        if obj == self.lineEdit() and event.type() == QtCore.QEvent.MouseButtonPress:
            # if the lineedit is clicked we want the combo to open/close
            if self.popup_is_showing:
                self.hidePopup()
            else:
                self.showPopup()
            event.accept()
            return True
        elif obj == self and event.type() == QtCore.QEvent.Wheel:
            event.ignore()
            return True
        return False

    def wheelEvent(self, event):
        event.ignore()
        return True

    def showPopup(self):
        now = time.time()
        diff = abs(now - self.last_time_popup_was_triggered)
        if diff > self.popup_toggle_min_interval:
            super().showPopup()
            self.popup_is_showing = True
            self.last_time_popup_was_triggered = now

    def hidePopup(self):
        now = time.time()
        diff = abs(now - self.last_time_popup_was_triggered)
        if diff > self.popup_toggle_min_interval:
            super().hidePopup()
            self.popup_is_showing = False
            self.last_time_popup_was_triggered = now

    def on_stateChanged(self, item):
        # Without this, display can be incorrect when no items are checked [UE-112543].
        self.setCurrentIndex(0)

        selected_values = []
        selected_strings = []
        for i in range(self.count()):
            item = self.model().item(i, 0)
            if item.checkState() == QtCore.Qt.Checked:
                selected_values.append(self.itemData(i))
                selected_strings.append(self.itemText(i))

        self.setEditText(self.separator.join(selected_strings))
        self.signal_selection_changed.emit(selected_values)


# A combo box that has a static icon and opens a list of options when pressed, e.g. view options in settings
class DropDownMenuComboBox(QtWidgets.QComboBox):
    on_select_option = QtCore.Signal(str)

    def __init__(self, icon: QtGui.QIcon = None, icon_size: int = 25, parent=None):
        super().__init__(parent=parent)

        drop_down_arrow_size = 15
        if icon is not None:
            self.addItem(icon, "")
            self.setFixedWidth(icon_size + drop_down_arrow_size)
        else:
            self.addItem("")
            self.setFixedWidth(drop_down_arrow_size)

        self.model().setData(self.model().index(0, 0), QtCore.QSize(100, 100), QtCore.Qt.SizeHintRole)
        self.view().setRowHidden(0, True)
        self.currentIndexChanged.connect(self._on_index_changed)

    def _on_index_changed(self, index: int):
        selected_item = self.itemText(index)
        self.on_select_option.emit(selected_item)
        # Always make sure that the icon is shown
        self.setCurrentIndex(0)

    def showPopup(self):
        self.view().setMinimumWidth(self.view().sizeHintForColumn(0))
        super().showPopup()


class ControlQPushButton(QtWidgets.QPushButton):
    def __init__(self, parent = None, *, hover_focus: bool = True):
        super().__init__(parent)

        # Avoid "click -> focus -> disable -> focus arbitrary nearby widget"
        self.setFocusPolicy(QtCore.Qt.TabFocus)

        # Deprecated behavior: force focus on mouse enter for icon changes.
        # Defaults to True for backward compatibility, but should be removed.
        # New code should use stylesheet :hover selectors instead.
        self.hover_focus = hover_focus

    def enterEvent(self, event):
        super().enterEvent(event)
        if self.hover_focus:
            self.setFocus()

    def leaveEvent(self, event):
        super().leaveEvent(event)
        if self.hover_focus:
            self.clearFocus()

    @classmethod
    def create(
        cls, icon_off=None,
        icon_on=None,
        icon_hover_on=None, icon_hover=None,
        icon_disabled_on=None, icon_disabled=None,
        icon_size=None,
        checkable=True, checked=False,
        tool_tip=None,
        *,
        hover_focus: bool = True,
        name: Optional[str] = None
    ):
        button = ControlQPushButton(hover_focus=hover_focus)

        if name:
            button.setObjectName(name)

        set_qt_property(button, 'no_background', True)

        icon = QtGui.QIcon()
        pixmap: Optional[QtGui.QPixmap] = None

        if icon_off:
            pixmap = QtGui.QPixmap(icon_off)
            icon.addPixmap(pixmap, QtGui.QIcon.Normal, QtGui.QIcon.Off)

        if icon_on:
            pixmap = QtGui.QPixmap(icon_on)
            icon.addPixmap(pixmap, QtGui.QIcon.Normal, QtGui.QIcon.On)

        if icon_hover:
            pixmap = QtGui.QPixmap(icon_hover)
            icon.addPixmap(pixmap, QtGui.QIcon.Active, QtGui.QIcon.Off)

        if icon_hover_on:
            pixmap = QtGui.QPixmap(icon_hover_on)
            icon.addPixmap(pixmap, QtGui.QIcon.Active, QtGui.QIcon.On)

        if icon_disabled:
            pixmap = QtGui.QPixmap(icon_disabled)
            icon.addPixmap(pixmap, QtGui.QIcon.Disabled, QtGui.QIcon.Off)

        if icon_disabled_on:
            pixmap = QtGui.QPixmap(icon_disabled)
            icon.addPixmap(pixmap, QtGui.QIcon.Disabled, QtGui.QIcon.On)

        button.setIcon(icon)

        if pixmap and not icon_size:
            icon_size = pixmap.rect().size()

        if icon_size:
            button.setIconSize(icon_size)

        button.setMinimumSize(25, 35)

        if tool_tip:
            button.setToolTip(tool_tip)

        if checkable:
            button.setCheckable(checkable)
            button.setChecked(checked)

        return button


class FramelessQLineEdit(QtWidgets.QLineEdit):
    def __init__(self, parent=None):
        super().__init__(parent)

        if not self.isReadOnly():
            set_qt_property(self, "frameless", True)

        self.current_text = None
        self.is_valid = True

    def enterEvent(self, event):
        super().enterEvent(event)

        if not self.isReadOnly():
            set_qt_property(self, "frameless", False)

    def leaveEvent(self, event):
        super().leaveEvent(event)

        if self.hasFocus():
            return

        if not self.isReadOnly():
            set_qt_property(self, "frameless", True)

    def focusInEvent(self, e):
        super().focusInEvent(e)

        # Store the current value
        self.current_text = self.text()

        if not self.isReadOnly():
            set_qt_property(self, "frameless", False)

    def focusOutEvent(self, e):
        super().focusOutEvent(e)

        if not self.isReadOnly():
            set_qt_property(self, "frameless", True)

        if not self.is_valid:
            self.setText(self.current_text)
            self.is_valid = True

            set_qt_property(self, "error", False)

    def keyPressEvent(self, e):
        super().keyPressEvent(e)

        if e.key() == QtCore.Qt.Key_Return or e.key() == QtCore.Qt.Key_Enter:
            if self.is_valid:
                self.clearFocus()
        elif e.key() == QtCore.Qt.Key_Escape:
            self.setText(self.current_text)
            self.clearFocus()


class CollapsibleGroupBox(QtWidgets.QGroupBox):
    def __init__(self):
        super().__init__()

        self.setCheckable(True)
        self.setStyleSheet(
            'QGroupBox::indicator:checked:hover {image: url(:icons/images/tree_arrow_expanded_hovered.png);}'
            'QGroupBox::indicator:checked {image: url(:icons/images/tree_arrow_expanded.png);}'
            'QGroupBox::indicator:unchecked:hover {image: url(:icons/images/tree_arrow_collapsed_hovered.png);}'
            'QGroupBox::indicator:unchecked {image: url(:icons/images/tree_arrow_collapsed.png);}'
        )

        self.toggled.connect(self._on_set_expanded)
        self.setChecked(True)

    @property
    def is_expanded(self):
        return self.isChecked()

    def set_expanded(self, value: bool):
        if value and not self.is_expanded or not value and self.is_expanded:
            self._on_set_expanded(value)
            self.blockSignals(True)
            self.setChecked(value)
            self.blockSignals(False)

    def _on_set_expanded(self, should_expand: bool):
        if should_expand:
            self.setMaximumHeight(self._original_maximum_height)
        else:
            self._original_maximum_height = self.maximumHeight()
            # Just font height does not suffice ... need to investigate how to get a better value programmatically
            safety_margin = 6
            self.setMaximumHeight(self.fontMetrics().height() + safety_margin)


class SearchableComboBox(QtWidgets.QComboBox):

    class CustomQList(QtWidgets.QListWidget):
        def __init__(self, parent):
            super().__init__(parent)


    def __init__(self, parent):
        super().__init__(parent)

        self.HEIGHT = 15

        self.item_list = self.CustomQList(self)
        self.item_list.setViewportMargins(0, self.HEIGHT, 0, 0)
        self.setView(self.item_list)
        self.setModel(self.item_list.model())
        self.resize(300,self.size().height())
        self.search_line = QtWidgets.QLineEdit(self.view())
        self.search_line.setVisible(False)
        self.search_line.setPlaceholderText("Search")

        self.search_line.textChanged.connect(self.text_changed)
        self.wheel_only_on_focus = True

        # Maps item text to its index for fast lookup during search filtering
        self._item_index_map: Optional[Dict[str, int]] = None

    def showPopup(self):
        super().showPopup()

        MARGIN = 10
        XPOS = MARGIN/2
        YPOS = 2

        self.search_line.setGeometry(
            XPOS, YPOS, self.view().width() - MARGIN*2.5, self.HEIGHT)
        self.search_line.setVisible(True)

        self.CB_item_text = [self.itemText(i) for i in range(self.count())]

        # Reset index mapping when popup shows in case items changed
        self._item_index_map = None

    def hidePopup(self):
        super().hidePopup()

        self.search_line.setVisible(False)
        self.search_line.clear()

    def text_changed(self, text: str):

        # Show all and early exit if text is empty
        if not text:
            for i in range(len(self.CB_item_text)):
                self.view().setRowHidden(i, False)
            return
        
        text_lower = text.lower()
        
        # Build index mapping if was reset
        if self._item_index_map is None:
            self._item_index_map = {item: i for i, item in enumerate(self.CB_item_text)}
        
        # Track visibility changes to minimize UI updates
        visibility_changes = []
        
        for item in self.CB_item_text:
            index = self._item_index_map.get(item)

            # This shouldn't happen but let's handle it just in case.
            if index is None:
                LOGGER.warning(f"SearchableComboBox: Item '{item}' unexpectedly not found in index map, rebuilding map")
                self._item_index_map = {item: i for i, item in enumerate(self.CB_item_text)}
                index = self._item_index_map[item]
            
            should_show = text_lower in item.lower()
            current_hidden = self.view().isRowHidden(index)
            
            if should_show and current_hidden:
                visibility_changes.append((index, False))
            elif not should_show and not current_hidden:
                visibility_changes.append((index, True))
        
        # Apply visibility changes in batch
        for index, hidden in visibility_changes:
            self.view().setRowHidden(index, hidden)

    def wheelEvent(self, event) -> None:

        if self.wheel_only_on_focus:
            if self.hasFocus():
                super().wheelEvent(event)
        else:
            super().wheelEvent(event)


class HostnameValidator(QtGui.QValidator):
    INVALID_CHARS_RE = re.compile(r"[^A-Z.\d-]", re.IGNORECASE)
    VALID_SEGMENT_RE = re.compile(r"(?!-)[A-Z\d-]{1,63}(?<!-)$",
                                  re.IGNORECASE)

    def fixup(self, input: str):
        return self.INVALID_CHARS_RE.sub('', input)

    def validate(self, input: str, pos: int):
        input_len = len(input)
        if input_len == 0 or input_len > 255:
            return QtGui.QValidator.Intermediate

        if input[-1] == '.':
            input = input[:-1]  # trim FQDN terminator

        for segment in input.split('.'):
            if not self.VALID_SEGMENT_RE.match(segment):
                return QtGui.QValidator.Intermediate

        return QtGui.QValidator.Acceptable


class AddressComboBox(NonScrollableComboBox):
    '''
    An editable combo box with visible indication of hostname resolvability.
    '''

    class ResolverRunnable(QtCore.QRunnable):
        ''' Performs a name resolution query on the thread pool. '''

        class Signals(QtCore.QObject):
            result = QtCore.Signal(list)

        def __init__(self, name: str):
            super().__init__()
            self.name = name
            self.signals = self.Signals()

        def run(self):
            addrs = list[str]()
            try:
                (_, _, addrs) = socket.gethostbyname_ex(self.name)
            except socket.gaierror:
                pass
            self.signals.result.emit(addrs)

    def __init__(self, parent=None):
        super().__init__(parent=parent)

        self.runnable = AddressComboBox.ResolverRunnable('')
        self.resolving = False
        self.debounce = QtCore.QTimer()
        self.debounce.setSingleShot(True)
        self.debounce.timeout.connect(self.on_text_settled)

        self.setEditable(True)
        self.lineEdit().setValidator(HostnameValidator())
        self.lineEdit().textChanged.connect(self.on_text_changed)
        self.lineEdit().editingFinished.connect(self.on_editing_finished)

    def start_resolve(self):
        if self.resolving:
            return False

        name = self.lineEdit().text()
        self.runnable = AddressComboBox.ResolverRunnable(name)
        self.runnable.signals.result.connect(self.on_resolver_result)
        QtCore.QThreadPool.globalInstance().start(self.runnable)
        self.resolving = True
        return True

    def on_text_changed(self, new_text: str):
        if not self.lineEdit().hasAcceptableInput():
            set_qt_property(self, 'validation', 'invalid')
            self.debounce.stop()
            return

        set_qt_property(self, 'validation', 'intermediate')

        # Don't issue the query until the user stops typing for a second
        SETTLE_WAIT_MILLISEC = 1000
        self.debounce.start(SETTLE_WAIT_MILLISEC)

    def on_editing_finished(self):
        self.debounce.stop()

        if not self.lineEdit().hasAcceptableInput():
            set_qt_property(self, 'validation', 'invalid')
            return

        set_qt_property(self, 'validation', 'intermediate')
        self.start_resolve()

    def on_text_settled(self):
        self.start_resolve()

    def on_resolver_result(self, addrs: list[str]):
        self.resolving = False

        current_text = self.lineEdit().text()
        last_resolved = self.runnable.name

        if current_text != last_resolved:
            # Resolver result was stale, issue a new one
            self.start_resolve()
            return

        if len(addrs) == 0:
            set_qt_property(self, 'validation', 'invalid')
        else:
            set_qt_property(self, 'validation', 'succeeded')


def set_qt_property(
    widget: QtWidgets.QWidget, prop, value, *,
    update_box_model: bool = True,
    recursive_refresh: bool = False
):
    '''
    Set a dynamic property on the specified widget, and also recalculate its
    styling to take the property change into account.

    Args:
        widget: The widget for which to trigger a style refresh.
        prop: The name of the dynamic property to set.
        value: The new value of the dynamic property `prop`.
        update_box_model: Whether to trigger the more expensive update steps
            required for style changes that impact the box model of the widget.
        recursive: Whether to also refresh the styling of all child widgets.
    '''
    widget.setProperty(prop, value)
    refresh_qt_styles(widget, update_box_model=update_box_model,
                      recursive=recursive_refresh)


def refresh_qt_styles(
    widget: QtWidgets.QWidget, *,
    update_box_model: bool = True,
    recursive: bool = False
):
    '''
    Recalculate a Qt widget's styling (and optionally its children's styling).

    Args:
        widget: The widget for which to trigger a style refresh.
        update_box_model: Whether to perform the more expensive update steps
            required for style changes that impact the box model of the widget.
        recursive: Whether to also refresh the styling of all child widgets.
    '''
    widget.style().unpolish(widget)
    widget.style().polish(widget)

    if update_box_model:
        style_change_event = QtCore.QEvent(QtCore.QEvent.StyleChange)
        QtWidgets.QApplication.sendEvent(widget, style_change_event)
        widget.update()
        widget.updateGeometry()

    if recursive:
        for child in widget.findChildren(QtWidgets.QWidget):
            refresh_qt_styles(child, recursive=True,
                              update_box_model=update_box_model)
