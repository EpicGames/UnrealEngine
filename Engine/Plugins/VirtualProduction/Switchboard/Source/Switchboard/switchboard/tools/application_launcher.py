# Copyright Epic Games, Inc. All Rights Reserved.

import os
import subprocess
import threading
from typing import Optional

from switchboard import switchboard_utils as sb_utils
from switchboard.switchboard_logging import LOGGER


class ApplicationLauncher(object):
    ''' Manages launching external applications '''

    def __init__(self, name: str):
        self.lock = threading.Lock()
        self.process: Optional[subprocess.Popen] = None
        self.name = name
        self.requires_elevation = False

    def exe_path(self):
        ''' Returns the expected executable path. Extension not included '''
        return ""

    def exe_name(self):
        return os.path.split(self.exe_path())[1]

    def launch(
        self,
        args: Optional[list[str]] = None,
        allow_duplicate: bool = False
    ):
        ''' Launches this application with the given arguments '''
        if args is None:
            args = []

        args.insert(0, self.exe_path())
        cmdline = ' '.join(args)

        with self.lock:
            if (not allow_duplicate) and self.is_running():
                return False

            if not os.path.exists(self.exe_path()):
                LOGGER.error(
                    f"Could not find '{self.name}' at '{self.exe_path()}'. "
                    "Has it been built?")
                return False

            LOGGER.debug(f"Launching '{cmdline}' ...")

            shell = True if self.requires_elevation else False
            creationflags = 0
            if hasattr(subprocess, 'CREATE_NEW_CONSOLE'):
                creationflags = subprocess.CREATE_NEW_CONSOLE
            subprocess.Popen(cmdline, shell=shell, creationflags=creationflags)

        return True

    def poll_process(self):
        return sb_utils.PollProcess(self.exe_name())

    def is_running(self):
        if self.process and (self.process.poll() is None):
            return True
        elif self.poll_process().poll() is None:
            return True

        return False

    def terminate(self, bypolling=False):
        if not bypolling and self.process:
            self.process.terminate()
        else:
            self.poll_process().kill()
