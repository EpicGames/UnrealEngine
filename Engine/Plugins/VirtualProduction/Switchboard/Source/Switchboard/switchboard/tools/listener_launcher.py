# Copyright Epic Games, Inc. All Rights Reserved.

from switchboard.config import CONFIG
from .application_launcher import ApplicationLauncher

class ListenerLauncher(ApplicationLauncher):
    ''' Manages launching Switchboard Listener '''

    def __init__(self, name:str = 'SwitchboardListener'):
        super().__init__(name)

    #~Begin ApplicationLauncher interface
    def exe_path(self):
        return CONFIG.listener_path()
    #~End ApplicationLauncher interface
