# Copyright Epic Games, Inc. All Rights Reserved.

from switchboard.config import CONFIG
from .application_launcher import ApplicationLauncher

class SBLHelperLauncher(ApplicationLauncher):
    ''' Manages launching Switchboard Listener Helper'''

    def __init__(self, name:str = 'SwitchboardListenerHelper'):
        super().__init__(name)

        # The SBL Helper executable requires elevation in order to be able to lock/unlock the gpu clocks.
        self.requires_elevation = True

    #~Begin ApplicationLauncher interface
    def exe_path(self):
        return CONFIG.sblhelper_path()
    #~End ApplicationLauncher interface
