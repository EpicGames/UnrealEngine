# Copyright Epic Games, Inc. All Rights Reserved.

from __future__ import annotations

from enum import auto, Enum
import glob
import json
import logging
from pathlib import Path, PurePath, PurePosixPath
import os
import time
from typing import Any, Optional, TypeAlias, Union


UnrealPath: TypeAlias = PurePosixPath

PROJECT_CONTENT_MOUNT_NAME = 'Game'
PROJECT_CONTENT_MOUNT_POINT = f'/{PROJECT_CONTENT_MOUNT_NAME}'


class UnrealPluginType(Enum):
    ''' Mirrors `EPluginType` from IPluginManager.h. '''
    Engine = auto()
    Enterprise = auto()
    Project = auto()
    External = auto()
    Mod = auto()


class UnrealPluginManager:
    '''
    Enumerates the engine and project plugin directories, as well as any paths
    listed in the .uproject "AdditionalPluginDirectories" field (so-called
    `UnrealPluginType.External` plugins) to discover .uplugin descriptors.

    Also has functionality to map local file paths to Unreal content mount
    points (`file_to_content_path`), and vice versa (`content_to_file_path`).

    Any `Project` or `External` plugins which are not marked disabled are
    returned by the `enabled_project_plugins` property accessor, and by default
    content within those plugins is surfaced elsewhere in Switchboard.
    '''

    def __init__(self):
        self._engine_plugin_paths: list[Path] = []
        self._project_plugin_paths: list[Path] = []
        self._external_plugin_paths: list[Path] = []

        self._engine_dir: Optional[Path] = None
        self._uproject_path: Optional[Path] = None

        # Information derived from .uproject
        self._addl_plugin_dirs: list[PurePath] = []
        self._disabled_plugins: set[str] = set()

        # Plugin name -> UnrealPlugin
        self._plugin_map: dict[str, UnrealPlugin] = {}

        # Reverse lookup from Content dir to owning plugin
        self._content_dir_map: dict[Path, UnrealPlugin] = {}

        # Cached for quick reference; also includes "external" plugins
        self._enabled_project_plugins: list[UnrealPlugin] = []

    @property
    def engine_dir(self) -> Optional[Path]:
        return self._engine_dir

    @property
    def uproject_path(self) -> Optional[Path]:
        return self._uproject_path

    def set_engine_dir(self, in_dir: Optional[Path]):
        if self._engine_dir == in_dir:
            return

        self._engine_plugin_paths.clear()
        self._engine_dir = in_dir
        if in_dir:
            self._engine_plugin_paths = self.find_plugins_in_dir(
                in_dir / 'Plugins')

        self._refresh_plugin_map()

    def set_uproject_path(self, in_path: Optional[Path]):
        if self.uproject_path == in_path:
            return

        self._project_plugin_paths.clear()
        self._external_plugin_paths.clear()

        self._addl_plugin_dirs.clear()
        self._disabled_plugins.clear()
        self._enabled_project_plugins.clear()

        self._uproject_path = in_path
        if not self._uproject_path:
            return

        try:
            with open(self._uproject_path, encoding='utf-8') as f:
                uproj: dict[str, Any] = json.load(f)
        except Exception as exc:
            logging.error('Error parsing .uproject JSON for %s',
                          self._uproject_path, exc_info=exc)
            self._uproject_path = None
            return

        proj_dir = self._uproject_path.parent
        proj_plugin_path = proj_dir / 'Plugins'
        self._project_plugin_paths = self.find_plugins_in_dir(proj_plugin_path)

        # Parse .uproject additional plugin paths
        self._addl_plugin_dirs = [
            PurePath(x) for x in
            uproj.get('AdditionalPluginDirectories', [])
        ]

        for addl_dir in self._addl_plugin_dirs:
            addl_plugins = self.find_plugins_in_dir(proj_dir / addl_dir)
            self._external_plugin_paths.extend(addl_plugins)

        # Parse .uproject plugin references to identify disabled plugins
        proj_plugin_refs: list[dict[str, Any]] = uproj.get('Plugins', [])
        for ref in proj_plugin_refs:
            ref_name: Optional[str] = ref.get('Name')
            if ref_name is None:
                logging.error('Error parsing %s: '
                              'Plugin references must have a "Name" field',
                              in_path)
                continue

            ref_enabled: Optional[bool] = ref.get('Enabled')
            if ref_enabled is None:
                logging.error('Error parsing %s: '
                              'Plugin references must have an "Enabled" field',
                              in_path)
                continue

            if not ref_enabled:
                self._disabled_plugins.add(ref_name)

        # Update plugin map
        self._refresh_plugin_map()

        # Cache enabled project + external plugins
        for plugin_path in (self._project_plugin_paths +
                            self._external_plugin_paths):
            plugin_name = plugin_path.stem

            if plugin_name in self._disabled_plugins:
                continue

            if plugin := self.get_plugin_by_name(plugin_name):
                self._enabled_project_plugins.append(plugin)
            else:
                logging.warning("Couldn't find enabled project plugin %s",
                                plugin_path)

    def content_to_file_path(
        self,
        path: Union[UnrealPath, str]
    ) -> Optional[Path]:
        '''
        Given an Unreal content path beginning with e.g. `/Game/...` or
        `/PluginName/...` map it to the corresponding local file or directory.
        '''

        if isinstance(path, str):
            path = UnrealPath(path)

        if not path.is_absolute() or len(path.parts) < 2:
            logging.error('Not an absolute path: %s', path)
            return None

        mount_name = path.parts[1]
        rest = path.relative_to(f'/{mount_name}/')

        if rest.name:
            # Trim object/subobject portion if present
            if (dot_idx := rest.name.find('.')) != -1:
                rest = rest.parent / rest.name[:dot_idx]

        if mount_name == PROJECT_CONTENT_MOUNT_NAME:
            assert self._uproject_path
            return self._uproject_path.parent / 'Content' / rest
        else:
            plugin = self.get_plugin_by_name(mount_name)
            if plugin is None:
                logging.warning('Unknown mount point for path %s', path)
                return None

            return plugin.plugin_content_path / rest

    def file_to_content_path(
        self,
        path: Union[Path, str]
    ) -> Optional[UnrealPath]:
        '''
        Given a file or directory path, map it to the corresponding Unreal
        content path (relative to the project or plugin content mount point).
        '''

        if isinstance(path, str):
            path = Path(path)

        assert self._uproject_path
        project_content_path = self._uproject_path.parent / 'Content'

        if path.is_relative_to(project_content_path):
            rest = path.relative_to(project_content_path)

            # Strip file extension if present
            if rest.suffix:
                rest = rest.with_suffix('')
                
            return UnrealPath(PROJECT_CONTENT_MOUNT_POINT) / rest

        for parent in path.parents:
            if plugin := self.get_plugin_by_content_dir(parent):
                rest = path.relative_to(parent)

                # Strip file extension if present
                if rest.suffix:
                    rest = rest.with_suffix('')

                return plugin.mounted_path / rest

        logging.warning('Failed to resolve content path "%s"', path)
        return None

    def get_plugin_by_name(self, name: str) -> Optional[UnrealPlugin]:
        return self._plugin_map.get(name)

    def get_plugin_by_content_dir(
            self,
            in_dir: Path
    ) -> Optional[UnrealPlugin]:
        return self._content_dir_map.get(in_dir)

    def find_plugins_in_dir(self, in_path: Path) -> list[Path]:
        logging.debug('Discovering plugins in path "%s"', in_path)

        start_time = time.perf_counter()

        # We don't expect to find plugins in the following folders, so we can prune those subtrees.
        skip_dirs = {'content'} 

        found_plugins: list[Path] = []
        suffix = '.uplugin'

        for dirpath, dirs, files in os.walk(in_path, topdown=True):
            # prune subtrees whose name is in skip_dirs
            dirs[:] = [d for d in dirs if d.lower() not in skip_dirs]

            for fname in files:
                if fname.lower().endswith(suffix):
                    found_plugins.append(Path(dirpath) / fname)

                     # Plugins don't nest; don't descend this tree
                    dirs.clear()
                    break

        end_time = time.perf_counter()

        logging.debug('Discovered %i plugins in %.2f seconds',
                      len(found_plugins), end_time - start_time)

        return found_plugins

    @property
    def enabled_project_plugins(self) -> list[UnrealPlugin]:
        return self._enabled_project_plugins

    @property
    def addl_plugin_dirs(self) -> list[PurePath]:
        ''' Returns the relative path fragments from the uproject JSON. '''
        return self._addl_plugin_dirs

    def _refresh_plugin_map(self):
        self._plugin_map.clear()
        self._content_dir_map.clear()

        # Plugin name collisions in later paths supercede earlier ones.
        plugin_lists: list[tuple[list[Path], UnrealPluginType]] = [
            (self._engine_plugin_paths, UnrealPluginType.Engine),
            (self._external_plugin_paths, UnrealPluginType.External),
            (self._project_plugin_paths, UnrealPluginType.Project),
        ]

        for plugin_list, plugin_type in plugin_lists:
            for plugin_path in plugin_list:
                plugin_name = plugin_path.stem

                # TODO?: Precedence logic in FPluginManager::CreatePluginObject
                if existing := self._plugin_map.get(plugin_name):
                    logging.warning('Plugin at "%s" superceded by "%s"',
                                    existing.uplugin_file_path, plugin_path)

                plugin = UnrealPlugin(plugin_path, plugin_type=plugin_type)

                self._plugin_map[plugin_name] = plugin

        self._content_dir_map = {x.plugin_content_path: x
                                 for x in self._plugin_map.values()}


class UnrealPlugin(object):
    '''
    A simple object that encapsulates properties of an Unreal Engine plugin
    based on the file system path and name of the plugin's .uplugin file.
    '''

    @classmethod
    def from_plugin_path(
        cls,
        plugin_path: Union[Path, str]
    ) -> Optional[UnrealPlugin]:
        '''
        Get an UnrealPlugin object based on the given file system path.

        The provided path can be either a file path to an Unreal Engine
        ".uplugin" file or a path to the plugin directory that contains
        a ".uplugin" file. In the latter case, the directory will be
        searched for the ".uplugin" file.
        '''
        if not isinstance(plugin_path, Path):
            plugin_path = Path(plugin_path)

        if not plugin_path.exists():
            logging.warning(
                'Cannot find uplugin file or plugin directory at '
                f'path: {plugin_path}')
            return None

        if (plugin_path.is_file() and
                plugin_path.suffix.casefold() == '.uplugin'):
            return cls(plugin_path)

        if not plugin_path.is_dir():
            logging.warning(f'Plugin path is not a directory: {plugin_path}')
            return None

        uplugin_file_paths = list(plugin_path.glob('*.uplugin'))
        if not uplugin_file_paths:
            logging.warning(
                f'No .uplugin files found in plugin directory: {plugin_path}')
            return None

        if len(uplugin_file_paths) > 1:
            msg = (
                'Multiple .uplugin files found in plugin directory: '
                f'{plugin_path}\n    ')
            msg += '\n    '.join([str(p) for p in uplugin_file_paths])
            logging.warning(msg)
            return None

        uplugin_file_path = Path(uplugin_file_paths[0])

        return cls(uplugin_file_path)

    @classmethod
    def from_path_filters(
        cls,
        uproject_path: Union[Path, str],
        filter_patterns: list[str]
    ) -> list[UnrealPlugin]:
        '''
        Get a list of Unreal Engine plugins matching the given path-based
        filter patterns.

        The provided path patterns can be absolute or relative. For relative
        path patterns, patterns resembling a directory name (i.e. no path
        separators) will be assumed to refer to plugin directories inside the
        Unreal Engine project (in its "Plugins" subdirectory). Otherwise,
        relative path patterns are assumed to be relative project directory
        (the directory containing the .uproject file).
        '''
        if not isinstance(uproject_path, Path):
            uproject_path = Path(uproject_path)

        plugin_paths = set()

        for filter_pattern in filter_patterns:
            path_pattern = Path(filter_pattern)

            if path_pattern.is_absolute():
                plugin_filter = path_pattern
            else:
                path_pattern_parts = path_pattern.parts
                if len(path_pattern_parts) < 1:
                    continue

                if len(path_pattern_parts) == 1:
                    # When the path pattern is relative and it looks like a
                    # directory name (i.e. no path separators), assume we're
                    # matching against plugin directories inside the project.
                    plugin_filter = (uproject_path.parent /
                                     'Plugins' / path_pattern)
                else:
                    # Otherwise, assume that the path pattern is relative to
                    # the project directory.
                    plugin_filter = uproject_path.parent / path_pattern

            for plugin_path in glob.glob(str(plugin_filter)):
                plugin_path = Path(plugin_path).resolve()
                plugin_paths.add(plugin_path)

        unreal_plugins: list[UnrealPlugin] = []
        for plugin_path in sorted(list(plugin_paths)):
            unreal_plugin = cls.from_plugin_path(plugin_path)
            if unreal_plugin:
                unreal_plugins.append(unreal_plugin)

        return unreal_plugins

    def __init__(
        self,
        uplugin_file_path: Union[Path, str],
        plugin_type: UnrealPluginType = UnrealPluginType.Project,
    ):
        if not isinstance(uplugin_file_path, Path):
            uplugin_file_path = Path(uplugin_file_path)

        self._uplugin_file_path = uplugin_file_path
        self._plugin_type = plugin_type

    def __repr__(self) -> str:
        return f'{self.__class__.__name__}("{self._uplugin_file_path}")'

    @property
    def uplugin_file_path(self) -> Path:
        '''
        The file system path to the plugin's .uplugin file.
        '''
        return self._uplugin_file_path

    @property
    def name(self) -> str:
        '''
        The name of the plugin.
        '''
        return self._uplugin_file_path.stem

    @property
    def plugin_dir(self) -> Path:
        '''
        The file system path to the root directory of the plugin.
        '''
        return self._uplugin_file_path.parents[0]

    @property
    def plugin_content_path(self) -> Path:
        '''
        The file system path to the plugin's "Content" directory.

        This is the directory that gets mounted in Unreal Engine.
        '''
        return self.plugin_dir / 'Content'

    @property
    def mounted_path(self) -> UnrealPath:
        '''
        The root content path of the plugin when its "Content" directory is
        mounted in Unreal Engine.
        '''
        # Note that UE uses the file name of the uplugin file to produce the
        # mounted path in engine, and not the name of the plugin directory or
        # any data inside the uplugin file.
        return UnrealPath(f'/{self.name}')

    @property
    def plugin_type(self) -> UnrealPluginType:
        ''' The type of this plugin. '''
        return self._plugin_type
