# Copyright Epic Games, Inc. All Rights Reserved.

import csv
import io
import json
import logging
import os
import pathlib
import re
import shutil
import subprocess
import sys
from typing import Callable, Optional, TextIO, Union
import uuid

from . import p4_utils, switchboard_utils as sb_utils

try:
    import P4
    P4PYTHON_AVAILABLE = True
except ImportError:
    P4PYTHON_AVAILABLE = False


_CACHED_SB_BUILDVER: dict[str, int | str] | None = None

def get_sb_buildver(*, force_refresh=False):
    ''' Return the JSON dict of Build.version in the Switchboard source tree. '''
    global _CACHED_SB_BUILDVER

    if _CACHED_SB_BUILDVER is None or force_refresh:
        # The objective is to query the UE version associated with Switchboard,
        # which is not guaranteed to be the same as the engine_dir setting.
        # We will rely on the location of this application, and get the associated
        # Build.version file to get the version. An alternative to this could be to
        # query it from the UnrealEditor executable (not SBL due to possible mis-redeployment).
        sbmod_dir = pathlib.Path(os.path.dirname(sys.modules['switchboard'].__file__))

        for parent in sbmod_dir.parents:
            if parent.name == 'Engine':
                buildver_path = parent / 'Build' / 'Build.version'
                try:
                    with open(buildver_path, 'r', encoding='utf-8') as f:
                        _CACHED_SB_BUILDVER = json.load(f)
                except Exception as e:
                    logging.warning(f'Error while reading {buildver_path}: {e}', exc_info=True)

                break

    if _CACHED_SB_BUILDVER is None:
        _CACHED_SB_BUILDVER = {}

    return _CACHED_SB_BUILDVER


def find_bin(
    dir_to_look_under: Optional[pathlib.Path] = None
) -> Optional[pathlib.Path]:
    ugs_dir = dir_to_look_under

    if not ugs_dir:
        env_ugs_path = shutil.which('ugs')
        if env_ugs_path:
            ugs_dir = pathlib.Path(env_ugs_path).parent
        elif sys.platform.startswith('win'):
            ugs_dir = os.path.join('${LOCALAPPDATA}', 'UnrealGameSync',
                                   'Latest')
        # elif:
            # for other platforms we don't have a default install location (the
            # dll is usually installed from perforce, which could go anywhere)

    INSTALL_PATH_SUGGESTION = (
        'Ensure that UGS is installed on the target device. If it is already '
        'installed, then either make sure the `ugs` command is globally '
        'available, or specify the directory explicitly.'
    )

    if not ugs_dir:
        logging.error('Could not determine where to find the UnrealGameSync '
                      'library (ugs.dll). ' + INSTALL_PATH_SUGGESTION)
        return None

    ugs_bin_path = os.path.join(os.path.expandvars(ugs_dir), 'ugs.dll')
    if not os.path.exists(ugs_bin_path):
        logging.error(f"Failed to find '{ugs_bin_path}'. "
                      + INSTALL_PATH_SUGGESTION)
        return None

    return pathlib.Path(ugs_bin_path)


def setup_dependencies(
    ue_engine_dir: pathlib.Path
) -> int:
    logging.debug('Verifying UnrealGameSync dependencies.')

    if sys.platform.startswith('win'):
        script_name = 'GetDotnetPath.bat'
        dotnet_setup_script = os.path.join(ue_engine_dir, 'Build',
                                           'BatchFiles', script_name)
        dotnet_setup_args = [dotnet_setup_script]
    else:
        if sys.platform.startswith('darwin'):
            platform_dirname = 'Mac'
        else:
            platform_dirname = 'Linux'

        platform_scripts_dir = os.path.join(ue_engine_dir, 'Build',
                                            'BatchFiles', platform_dirname)

        script_name = 'SetupEnvironment.sh'
        dotnet_setup_script = os.path.join(platform_scripts_dir, script_name)
        dotnet_setup_args = [dotnet_setup_script, '-dotnet',
                             platform_scripts_dir]

    try:
        with subprocess.Popen(dotnet_setup_args, stdin=subprocess.DEVNULL,
                              stdout=subprocess.PIPE,
                              startupinfo=sb_utils.get_hidden_sp_startupinfo()
                              ) as proc:
            for line in proc.stdout:
                logging.debug(f'{script_name}> {line.decode().rstrip()}')
    except Exception as exc:
        logging.error('setup_ugs_dependencies(): exception running Popen: '
                      f'{dotnet_setup_args}', exc_info=exc)
        return -1

    if proc.returncode != 0:
        logging.error('Unable to find a install of Dotnet SDK. Please ensure '
                      'you have it installed and that `dotnet` is a globally '
                      'available command.')

    return proc.returncode


def _find_engine_dir(
    uproj_path: pathlib.Path
) -> Optional[pathlib.Path]:
    iter_dir = uproj_path.parent
    while not iter_dir.samefile(iter_dir.anchor):
        candidate = iter_dir / 'Engine'
        if candidate.is_dir():
            return candidate
        iter_dir = iter_dir.parent

    # TODO: Consider looking for Engine dir as ancestor of this script?
    return None


def _get_active_ugs_context(
    ugs_dll_path: pathlib.Path,
    cwd: Optional[pathlib.Path] = None
) -> Optional[dict[str, str]]:
    # Assumes UGS dependencies have been already setup (prevents us from
    # redundantly running `setup_dependencies()`)

    ugs_state = {}
    status_args = ['dotnet', str(ugs_dll_path), 'status']
    try:
        with subprocess.Popen(status_args, cwd=cwd, stdin=subprocess.DEVNULL,
                              stdout=subprocess.PIPE,
                              startupinfo=sb_utils.get_hidden_sp_startupinfo()
                              ) as status_proc:
            for status_line in status_proc.stdout:
                status_line_str = status_line.decode().rstrip()
                logging.debug(f'ugs> {status_line_str}')
                if 'Project:' in status_line_str:
                    match = re.match(r"Project:\s*//(\S+?)/(\S+)",
                                     status_line_str)
                    if match:
                        ugs_state['client'] = match.group(1)
                        ugs_state['project'] = match.group(2)
                if 'User:' in status_line_str:
                    match = re.match(r"User:\s*(\S+)", status_line_str)
                    if match:
                        ugs_state['user'] = match.group(1)
    except Exception as exc:
        logging.error('_get_ugs_state(): exception running Popen: '
                      f'{status_args}', exc_info=exc)
        return None

    if not ugs_state:
        return None

    return ugs_state


def _set_active_ugs_context(
    ugs_dll_path: pathlib.Path,
    ugs_settings: dict[str, str],
    cwd: Optional[pathlib.Path] = None
) -> int:
    # Assumes UGS dependencies have been already setup (prevents us from
    # redundantly running `setup_dependencies()`)

    init_args = ['dotnet', str(ugs_dll_path), 'init']
    try:
        for key, val in ugs_settings.items():
            init_args.append(f'-{key}={val}')

        with subprocess.Popen(init_args, cwd=cwd, stdin=subprocess.DEVNULL,
                              stdout=subprocess.PIPE,
                              startupinfo=sb_utils.get_hidden_sp_startupinfo()
                              ) as init_proc:
            for init_line in init_proc.stdout:
                logging.info(f'ugs> {init_line.decode().rstrip()}')
    except Exception as exc:
        logging.error('_set_active_ugs_context(): exception running Popen: '
                      f'{init_args}', exc_info=exc)
        return -1

    return 0


def run(
    ugs_args: Union[str, list[str]],
    uproj_path: pathlib.Path,
    ugs_bin_dir: Optional[pathlib.Path] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
    output_handling_fn: Optional[Callable[[str], int]] = None,
) -> int:
    # Even though we specify this should be a list, handle the simplified case
    # of users passing a single word command
    if isinstance(ugs_args, str):
        ugs_args = ugs_args.split()
    args_str = ' '.join(ugs_args)

    ugs_bin_path = find_bin(ugs_bin_dir)
    if not ugs_bin_path:
        logging.error("Failed to find 'ugs.dll'. Aborting UnrealGameSync "
                      f"command: `ugs {args_str}`")
        return -1

    ue_engine_dir = _find_engine_dir(uproj_path)
    if not ue_engine_dir:
        logging.warning("Failed to locate the Unreal '/Engine/' directory. "
                        "As a result we are unable to validate UGS "
                        "dependencies. Ignoring and continuing on as if they "
                        "were setup.")
    # Make sure the needed dependencies (dotnet, etc.) are installed upfront
    elif setup_dependencies(ue_engine_dir) != 0:
        logging.warning('Failed to validate UnrealGameSync dependencies. '
                        'Ignoring and continuing on as if they were setup.')

    cwd = uproj_path.parent

    logging.info("Capturing UnrealGameSync's current state.")
    ugs_state_to_restore = _get_active_ugs_context(ugs_bin_path, cwd=cwd)
    if not ugs_state_to_restore:
        logging.warning('Failed to capture the current state of '
                        'UnrealGameSync. Will not be able to restore it after '
                        'completing our operation.')

    logging.info('Setting UnrealGameSync context.')
    # UGS expects a project path relative to the repo's root
    if ue_engine_dir:
        sanitized_proj_path = os.path.relpath(uproj_path, ue_engine_dir.parent)
    else:
        sanitized_proj_path = str(uproj_path)
    # and only accepts paths with delimited by forward-slashes
    sanitized_proj_path = sanitized_proj_path.replace('\\', '/')
    op_context_params = {'project': sanitized_proj_path}
    if user:
        op_context_params['user'] = user
    if client:
        op_context_params['client'] = client

    init_result = _set_active_ugs_context(ugs_bin_path, op_context_params,
                                          cwd=cwd)
    if init_result != 0:
        logging.error("Failed to initialize UnrealGameSync. Aborting "
                      f"UnrealGameSync command: `ugs {args_str}`")
        return init_result

    logging.info(f'Executing UnrealGameSync command: `ugs {args_str}`.')
    ugs_cmd = ['dotnet', ugs_bin_path] + ugs_args

    try:
        with subprocess.Popen(ugs_cmd, cwd=cwd, stdin=subprocess.DEVNULL,
                              stdout=subprocess.PIPE,
                              startupinfo=sb_utils.get_hidden_sp_startupinfo()
                              ) as ugs_proc:
            for line in ugs_proc.stdout:
                line_str = f'{line.decode().rstrip()}'

                invalid_arg_match = re.match(r"Invalid argument:\s*(\S+)",
                                             line_str)
                if invalid_arg_match:
                    logging.error(f'ugs> {line_str}')
                    logging.error('The current version of UnrealGameSync does '
                                  'not support the '
                                  f'`{invalid_arg_match.group(1)}`'
                                  'argument. Please ensure UGS is up to date.')
                    ugs_proc.terminate()
                elif output_handling_fn:
                    if output_handling_fn(line_str) != 0:
                        ugs_proc.terminate()
                else:
                    logging.info(f'ugs> {line_str}')
    except Exception as exc:
        logging.error(f'ugs.run(): exception running Popen: {ugs_cmd}',
                      exc_info=exc)
        return -1

    if ugs_state_to_restore:
        logging.info('Restoring previous UnrealGameSync state.')
        if _set_active_ugs_context(ugs_bin_path,
                                   ugs_state_to_restore, cwd=cwd) != 0:
            logging.warning("Failed to restore UnrealGameSync's state.")

    return ugs_proc.returncode


def sync(
    uproj_path: pathlib.Path,
    sync_cl: Optional[int] = None,
    sync_pcbs: Optional[bool] = False,
    ugs_bin_dir: Optional[pathlib.Path] = None,
    user: Optional[str] = None,
    client: Optional[str] = None
) -> int:

    sync_args = ['sync']
    if sync_cl:
        sync_args.append(f'{sync_cl}')
    else:
        sync_args.append('latest')

    if sync_pcbs:
        sync_args.append('-binaries')

    return run(
        sync_args,
        uproj_path,
        ugs_bin_dir=ugs_bin_dir,
        user=user,
        client=client
    )


def latest_chagelists(
    uproj_path: pathlib.Path,
    ugs_bin_dir: Optional[pathlib.Path] = None,
    user: Optional[str] = None,
    client: Optional[str] = None
) -> Optional[list[int]]:

    changelist_list = []

    def changes_output_handler(output_str: str,
                               changes_out=changelist_list) -> int:
        cl_desc_match = re.match(r"^\s*(\d+).*", output_str)
        if cl_desc_match:
            changes_out.append(cl_desc_match.group(1))
        return 0

    run_result = run(
        'changes',
        uproj_path,
        ugs_bin_dir=ugs_bin_dir,
        user=user,
        client=client,
        output_handling_fn=changes_output_handler
    )

    if run_result != 0:
        return None

    return changelist_list


def get_depot_config_paths(
    engine_dir: Optional[Union[str, pathlib.Path]],
    project_dir: Optional[Union[str, pathlib.Path]]
) -> list[str]:
    # See GetDepotConfigPaths in UnrealGameSyncShared/Utility.cs
    engine_dir = str(engine_dir) if engine_dir else None
    project_dir = str(project_dir) if project_dir else None

    config_paths: list[str] = []

    def add_platform_dirs(base: str, filename: str):
        config_paths.extend([
            f'{base}/{filename}',
            f'{base}/*/{filename}'
        ])

    def add_platform_extensions(base: str, rel: str, filename: str):
        add_platform_dirs(f'{base}{rel}', filename)
        add_platform_dirs(f'{base}/Platforms/*{rel}', filename)
        add_platform_dirs(f'{base}/Restricted/*{rel}', filename)

    if engine_dir:
        add_platform_extensions(engine_dir, '/Programs/UnrealGameSync',
                                'UnrealGameSync.ini')

    if project_dir:
        add_platform_extensions(project_dir, '/Build',
                                'UnrealGameSync.ini')

    return config_paths


class IniParser:
    '''
    Modeled after ConfigParser, but handles Unreal duplicate `+Key=ValToAppend`
    '''

    class Section:
        def __init__(self, name: str):
            self.name = name
            self.pairs: dict[str, str] = {}

        def set_value(self, key: str, value: str):
            self.pairs[key] = value

        def append_value(self, key: str, value: str):
            if key in self.pairs:
                self.pairs[key] += f'\n{value}'
            else:
                self.pairs[key] = value

    def __init__(self):
        self.sections: dict[str, IniParser.Section] = {}

    def read_string(self, string: str, source: str):
        sfile = io.StringIO(string)
        self.read_file(sfile, source)

    def read_file(self, file: TextIO, source: Optional[str] = None):
        if source is None:
            try:
                source = file.name
            except AttributeError:
                source = '<???>'

        self._read(file, source)

    def try_get(self, section_name: str, key: str) -> Optional[list[str]]:
        value = self.try_get_raw(section_name, key)
        return value.splitlines() if value is not None else None

    def try_get_raw(self, section_name: str, key: str) -> Optional[str]:
        ''' Multiple/appended values will be returned separated by '\\n' '''
        section = self.sections.get(section_name)
        if section is None:
            return None
        value = section.pairs.get(key)
        if value is None:
            return None
        return value

    def _read(self, file: TextIO, source: str):
        # See Parse() in UnrealGameSyncShared/ConfigFile.cs
        current_section: Optional[IniParser.Section] = None
        for line_num, line in enumerate(file, start=1):
            line = line.rstrip('\r\n')
            strip_line = line.strip()

            if len(strip_line) == 0:
                continue

            if strip_line.startswith(';'):
                continue

            if strip_line.startswith('[') and strip_line.endswith(']'):
                section_name = strip_line[1:-1]
                current_section = self.sections.get(section_name)
                if current_section is None:
                    new_section = IniParser.Section(section_name)
                    self.sections[section_name] = new_section
                    current_section = new_section
                continue

            if current_section is None:
                logging.warning(f'{source}:{line_num}: error: '
                                'current_section is None')
                logging.warning(f'\t{line}')
                continue

            equals_idx = strip_line.find('=')
            if equals_idx != -1:
                value = line[equals_idx+1:].lstrip()
                if strip_line.startswith('+'):
                    key = strip_line[1:equals_idx].strip()
                    current_section.append_value(key, value)
                else:
                    key = strip_line[0:equals_idx].rstrip()
                    current_section.set_value(key, value)
            else:
                logging.warning(f'{source}:{line_num}: error: missing =')
                logging.warning(f'\t{line}')

    def dump(self):
        ''' Write parsed state to `logging.info`. '''
        for name, section in self.sections.items():
            logging.info(f'[{name}]')
            for key, value in section.pairs.items():
                value_lines = value.splitlines()
                if len(value_lines) == 1:
                    logging.info(f'{key}={value}')
                else:
                    logging.info(f'{key}=')
                    for line in value_lines:
                        logging.info(f'\t{line}')


def parse_depot_ugs_configs(
    engine_dir: Optional[Union[str, pathlib.Path]],
    project_dir: Optional[Union[str, pathlib.Path]],
    *,
    port: Optional[str] = None,
    user: Optional[str] = None,
    client: Optional[str] = None,
) -> IniParser:
    '''
    Given local or depot engine and/or project paths, search the Perforce depot
    for UnrealGameSync.ini configs at known locations, coalesce their contents,
    and return an `IniParser` populated with the results.
    '''
    parser = IniParser()

    p4kwargs = {'port': port, 'user': user, 'client': client}

    # Might append depot wildcards to host paths and end up with mixed syntax
    # here, but the fstat separates them into normalized depot/host paths.
    mixed_config_paths = get_depot_config_paths(engine_dir, project_dir)
    if not mixed_config_paths:
        logging.debug('No paths to check for UnrealGameSync configuration')
        return parser

    logging.debug('Parsing UnrealGameSync configuration from depot')

    config_fstats = p4_utils.p4_fstat(mixed_config_paths, **p4kwargs)

    config_contents_map: dict[str, Optional[str]] = {}

    # First pass: get contents of locally modified files, or depot paths
    # of files to `p4 print` in the second pass otherwise
    for record in config_fstats:
        action: Optional[bytes] = record.get(b'action')
        head_action: Optional[bytes] = record.get(b'headAction')
        if record[b'code'] == b'stat':
            if (b'clientFile' in record) and (b'action' in record):
                # If the client file exists and is modified, use local ver
                client_path = pathlib.Path(record[b'clientFile'].decode())
                if client_path.exists() and action == b'edit':
                    local_contents = client_path.read_text(encoding='utf-8')
                    config_contents_map[str(client_path)] = local_contents
                    continue
                elif action == b'delete':
                    continue

            if head_action == b'delete':
                continue

            if b'depotFile' in record:
                depot_file = record[b'depotFile'].decode()
                config_contents_map[depot_file] = None
                continue

        logging.warning(f'Unhandled fstat record: {record}')

    # Second pass: p4 prints of depot configs @ head
    depot_print_paths = [k for k, v in config_contents_map.items()
                         if v is None]  # configs we didn't read locally

    config_prints = p4_utils.p4_print(depot_print_paths, **p4kwargs)

    for depot_path, record in zip(depot_print_paths, config_prints):
        if record.is_valid:
            config_contents_map[depot_path] = record.text
        elif b'delete' in record.stat.get(b'action', b''):
            del config_contents_map[depot_path]
        else:
            logging.warning(f'Invalid P4PrintResult for {depot_path}')

    for path, contents in config_contents_map.items():
        if contents is None:
            logging.warning(f'No contents for {path}')
            continue

        parser.read_string(contents, path)

    return parser


class SyncFilters:
    '''
    Determines whether given paths are excluded, based on a list of rules
    containing Perforce-style wildcards.

    These rules can be extracted from UnrealGameSync.ini files (see
    `read_categories_from_ini_parser()`), or added via calls directly to
    `SyncFilters.map.insert()`.

    To test a path against the filters, use `SyncFilters.includes_path()`.

    Examples:
        - `-/ExcludedRoot/...`
        - `-*.excludedfileextension`
        - `-.../SubdirectoryExcludedAnywhere/...`
    '''

    @classmethod
    def supported(cls):
        '''
        If this method returns False, only basic parsing of configured
        categories is available. Other important methods, such as
        `includes_path` and `exclude_category` will raise an exception.
        '''
        return P4PYTHON_AVAILABLE

    class CsvDialect(csv.excel):
        skipinitialspace = True

    class Category:
        def __init__(self, catid: uuid.UUID, name: str, paths: list[str]):
            self.catid = catid
            self.name = name
            self.paths = paths

            self.hidden = False
            self.requires: Optional[uuid.UUID] = None

    def __init__(self):
        self.categories: dict[uuid.UUID, SyncFilters.Category] = {}
        self.excluded_categories: set[uuid.UUID] = set()

        if P4PYTHON_AVAILABLE:
            self.map = P4.Map()
            self._reset_map()

    def _reset_map(self):
        self.map.clear()
        self.map.insert('...')

    def read_categories_from_ini_parser(self, config: IniParser):
        categories_raw = config.try_get_raw('Options', 'SyncCategory')
        if categories_raw:
            sfile = io.StringIO(categories_raw)
            self._read(sfile)

    def exclude_category(self, catid: uuid.UUID):
        if not P4PYTHON_AVAILABLE:
            raise RuntimeError('This method currently requires P4PYTHON')

        if catid in self.excluded_categories:
            return  # already excluded

        category = self.categories.get(catid)
        if category is None:
            logging.warning(f'exclude_category: no such category {catid}')
            return

        self._map_exclusion(category)

    def includes_path(self, path: str) -> bool:
        if not P4PYTHON_AVAILABLE:
            raise RuntimeError('This method currently requires P4PYTHON')

        return self.map.includes(path)

    def _read(self, file: TextIO):
        category_lines = [line.strip('()\r\n') for line in file]
        category_reader = csv.reader(category_lines, SyncFilters.CsvDialect)
        for category_fields in category_reader:
            category_dict: dict[str, str] = {}
            for field in category_fields:
                key, _, value = field.partition('=')
                value = value.strip('"')
                category_dict[key] = value

            catid = uuid.UUID(hex=category_dict['UniqueId'])
            name = category_dict['Name']
            paths = category_dict['Paths'].split(';')
            paths = [path.lstrip() for path in paths]

            hidden = category_dict.get('Hidden', '').lower() == 'true'
            requires = category_dict.get('Requires')
            requires = uuid.UUID(requires) if requires else None

            if catid not in self.categories:
                category = self.create_category(catid, name, paths)
                category.hidden = hidden
                category.requires = requires
            else:
                # This seems weird, but... some configs need it
                self.categories[catid].paths.extend(paths)

    def create_category(self, catid: uuid.UUID, name: str, paths: list[str]):
        category = SyncFilters.Category(catid, name, paths)
        self.categories[catid] = category
        return category

    DIR_REGEX = re.compile(r'(\.\.\.|/|\\)')

    def _map_exclusion(self, category: Category):
        self.excluded_categories.add(category.catid)
        for path in category.paths:
            self.map.insert(f'-{path}')

            if not self.DIR_REGEX.search(path):
                # unqualified file pattern, so also match nested instances
                self.map.insert(f'-.../{path}')
