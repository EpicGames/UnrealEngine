# Copyright Epic Games, Inc. All Rights Reserved.

from PySide6 import QtCore
from PySide6 import QtWidgets


class HorizontalTabBar(QtWidgets.QTabBar):
    def paintEvent(self, event):
        painter = QtWidgets.QStylePainter(self)
        option = QtWidgets.QStyleOptionTab()
        for index in range(self.count()):
            self.initStyleOption(option, index)
            painter.drawControl(QtWidgets.QStyle.CE_TabBarTabShape, option)
            painter.drawText(self.tabRect(index),
                             QtCore.Qt.AlignCenter | QtCore.Qt.TextDontClip,
                             self.tabText(index))

    def tabSizeHint(self, index):
        size = QtWidgets.QTabBar.tabSizeHint(self, index)
        if size.width() < size.height():
            size.transpose()
        size.setHeight(30)
        return size


class HorizontalTabWidget(QtWidgets.QTabWidget):
    def __init__(self, parent=None):
        QtWidgets.QTabWidget.__init__(self, parent)
        self.setTabBar(HorizontalTabBar())