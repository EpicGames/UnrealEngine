<RCC>
  <qresource prefix="images">
    <file>images/epic_games_logo.png</file>
  </qresource>
  <qresource prefix="css">
    <file>switchboard.qss</file>
  </qresource>
  <qresource prefix="icons">
    <file>images/alert-triangle-64.svg</file>
    <file>images/connected.png</file>
    <file>images/check_box.png</file>
    <file>images/check_box_checked.png</file>
    <file>images/check_box_checked_hovered.png</file>
    <file>images/check_box_hovered.png</file>
    <file>images/disconnected.png</file>
    <file>images/down_arrow.png</file>
    <file>images/empty_set_12x.png</file>
    <file>images/icon_add.png</file>
    <file>images/icon_add_disabled.png</file>
    <file>images/icon_add_hover.png</file>
    <file>images/icon_autojoin.png</file>
    <file>images/icon_autojoin_disabled.png</file>
    <file>images/icon_autojoin_hover.png</file>
    <file>images/icon_build.png</file>
    <file>images/icon_build_disabled.png</file>
    <file>images/icon_build_hover.png</file>
    <file>images/icon_close.png</file>
    <file>images/icon_close_disabled.png</file>
    <file>images/icon_close_hover.png</file>
    <file>images/icon_connect.png</file>
    <file>images/icon_connect_disabled.png</file>
    <file>images/icon_connect_hover.png</file>
    <file>images/icon_connected.png</file>
    <file>images/icon_connected_disabled.png</file>
    <file>images/icon_connected_hover.png</file>
    <file>images/icon_download.png</file>
    <file>images/icon_download_disabled.png</file>
    <file>images/icon_edit.png</file>
    <file>images/icon_edit_disabled.png</file>
    <file>images/icon_edit_hover.png</file>
    <file>images/icon_livelinkface_10.png</file>
    <file>images/icon_livelinkface_20.png</file>
    <file>images/icon_livelinkface_30.png</file>
    <file>images/icon_livelinkface_40.png</file>
    <file>images/icon_livelinkface_50.png</file>
    <file>images/icon_livelinkface_60.png</file>
    <file>images/icon_livelinkface_70.png</file>
    <file>images/icon_livelinkface_80.png</file>
    <file>images/icon_livelinkface_90.png</file>
    <file>images/icon_livelinkface_100.png</file>
    <file>images/icon_open.png</file>
    <file>images/icon_open_disabled.png</file>
    <file>images/icon_open_hover.png</file>
    <file>images/icon_play.png</file>
    <file>images/icon_play_disabled.png</file>
    <file>images/icon_play_hover.png</file>
    <file>images/icon_radar.png</file>
    <file>images/icon_radar_disabled.png</file>
    <file>images/icon_radar_hover.png</file>
    <file>images/icon_record.png</file>
    <file>images/icon_record_disabled.png</file>
    <file>images/icon_record_hover.png</file>
    <file>images/icon_refresh.png</file>
    <file>images/icon_refresh_disabled.png</file>
    <file>images/icon_refresh_error.png</file>
    <file>images/icon_refresh_hover.png</file>
    <file>images/icon_remove.png</file>
    <file>images/icon_remove_disabled.png</file>
    <file>images/icon_remove_hover.png</file>
    <file>images/icon_send_signal.png</file>
    <file>images/icon_send_signal_hover.png</file>
    <file>images/icon_send_signal_off.png</file>
    <file>images/icon_settings.png</file>
    <file>images/icon_settings_disabled.png</file>
    <file>images/icon_settings_hover.png</file>
    <file>images/icon_stop.png</file>
    <file>images/icon_stop_disabled.png</file>
    <file>images/icon_stop_hover.png</file>
    <file>images/icon_sync.png</file>
    <file>images/icon_sync_disabled.png</file>
    <file>images/icon_sync_hover.png</file>
    <file>images/icon_sync_and_build.png</file>
    <file>images/icon_sync_and_build_disabled.png</file>
    <file>images/icon_sync_and_build_hover.png</file>
    <file>images/icon_trigger_start.png</file>
    <file>images/icon_trigger_start_disabled.png</file>
    <file>images/icon_trigger_start_hover.png</file>
    <file>images/icon_trigger_stop.png</file>
    <file>images/icon_trigger_stop_disabled.png</file>
    <file>images/icon_trigger_stop_hover.png</file>
    <file>images/mocap.png</file>
    <file>images/PlusSymbol_12x.png</file>
    <file>images/record_start.png</file>
    <file>images/record_start_hover.png</file>
    <file>images/record_stop.png</file>
    <file>images/reset_to_default.png</file>
    <file>images/status_blank.png</file>
    <file>images/status_blank_disabled.png</file>
    <file>images/status_cyan.png</file>
    <file>images/status_green.png</file>
    <file>images/status_orange.png</file>
    <file>images/status_red.png</file>
    <file>images/switchboard.png</file>
    <file>images/sync_icon.png</file>
    <file>images/star_yellow.png</file>
    <file>images/star_yellow_off.png</file>
    <file>images/Settings_40x.png</file>
    <file>images/tree_arrow_collapsed.png</file>
    <file>images/tree_arrow_collapsed_hovered.png</file>
    <file>images/tree_arrow_expanded.png</file>
    <file>images/tree_arrow_expanded_hovered.png</file>
    <file>images/view_button.png</file>
  </qresource>
</RCC>
