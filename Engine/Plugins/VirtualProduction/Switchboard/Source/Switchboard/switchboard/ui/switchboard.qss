* {
    font-family: Roboto;
    color: #d8d8d8;
}

*[disconnected=true] {
    color: #545454;
}

*[not_synched=true] {
    color: #e52d00;
    font-weight: bold;
}

*[not_built=true] {
    color: #E2B900;
    font-weight: bold;
}

*[error=true] {
    color: #e52d00;
    font-weight: bold;
}

*[input_error=true] {
    background-color: #520000;
}

*[override=true] {
    background-color: #8f610e;
}

[validation~="intermediate"] {
    background-color: #525200;
}

[validation~="invalid"] {
    background-color: #520000;
}

QTextBrowser
{
    font-family:DroidSansMono;
}

QToolTip
{
     border-style: none;
     background-color: #323232;
     padding: 1px;
     opacity: 255; /* This is non-standard; should be 0.0-1.0 or 0%-100%, but Qt seems to use 0-255. */
}

QWidget
{
    background-color: #323232;
}

QMenuBar::item
{
    background-color: transparent;
}

QMenuBar::item:selected
{
    background: transparent;
    border: 1px solid #47a2ff;
}

QMenu
{
    background-color: #3d3d3d;
    border: 1px solid #242424;
}

QMenu::item
{
    padding: 2px 20px 2px 20px;
}

QMenu::item:selected
{
    background-color: #47a2ff;
}

QWidget:disabled
{
    color: #555555;
}

QWidget:focus
{
}

QLineEdit
{
    background-color: #242424;
    padding: 2px;
    border-style: none;
    border-radius: 0;
}

QLineEdit#device_name
{
    font-size: 14pt;
    font-weight: bold;
}

QLineEdit#device_address
{
    font-size: 10pt;
}

QPushButton
{
    background-color: QLinearGradient( x1: 0, y1: 0, x2: 0, y2: 1, stop: 0 #565656, stop: 0.1 #525252, stop: 0.5 #4e4e4e, stop: 0.9 #4a4a4a, stop: 1 #464646);
    border-width: 0px;
    border-color: #1e1e1e;
    border-style: solid;
    border-radius: 4;
    padding: 3px;
    padding-left: 5px;
    padding-right: 5px;
}

QPushButton:focus:hover
{
    outline: none;
}

QPushButton[no_background="true"]
{
    background-color: rgba(0, 0, 0, 0);
    border-style: none;
    padding: 0px;
}

QPushButton[not_synched=true]
{
    background-color: #e52d00;
}

QPushButton[not_built=true]
{
    background-color: #E2B900;
}

QPushButton:hover[no_background="true"]
{
    border-style: none;
    border-width: 2px;
    border-color: #242424;
}

QPushButton[no_border="true"]
{
    border-width: 0px;
    border-color: #1e1e1e;
    border-style: none;
    border-radius: 0;
    padding: 2px;
}

QPushButton:hover[record="true"]
{
    border-style: none;
}

QPushButton:pressed {
    background-color: #242424;
}

*[frameless="true"]
{
    background-color: rgba(0, 0, 0, 0);
    border-style: none;
}

QLabel
{ 
    background:transparent;
    border-style: none;
}

QLabel#changelist
{
    font-size: 10pt;
}

QLabel#widget_header
{
    font-weight: bold;
    qproperty-alignment: AlignCenter;
}

QLabel[warn="true"]
{ 
    color: #ff5603;
}

QLabel[footer="true"]
{   
    background-color: grey;
    border: 0px none white;
}

QComboBox
{
    background-color: QLinearGradient( x1: 0, y1: 0, x2: 0, y2: 1, stop: 0 #565656, stop: 0.1 #525252, stop: 0.5 #4e4e4e, stop: 0.9 #4a4a4a, stop: 1 #464646);
    border-style: none;
    border-radius: 5;
}

QComboBox:hover,QPushButton:hover
{
    /*border: 2px solid QLinearGradient( x1: 0, y1: 0, x2: 0, y2: 1, stop: 0 #ffa02f, stop: 1 #d7801a);*/
}


QComboBox:on
{
    padding-top: 3px;
    padding-left: 4px;
    background-color: QLinearGradient( x1: 0, y1: 0, x2: 0, y2: 1, stop: 0 #2d2d2d, stop: 0.1 #2b2b2b, stop: 0.5 #292929, stop: 0.9 #282828, stop: 1 #252525);
}

QComboBox QAbstractItemView
{
    background-color: #3d3d3d;
    border: 1px solid 242424;
}

QComboBox::drop-down
{
     subcontrol-origin: padding;
     subcontrol-position: top right;
     width: 15px;

     border-left-width: 0px;
     border-left-color: darkgray;
     border-left-style: solid;
     border-top-right-radius: 3px;
     border-bottom-right-radius: 3px;
}

QComboBox::down-arrow
{
     image: url(':/icons/images/down_arrow.png');
}

QSplitter::handle
{
    background-color: rgba(24, 24, 24, 50);
}

QTextEdit
{
    border-style: none;
    selection-background-color: #47a2ff;
}

QScrollArea
{
    border-style: none;
}

QScrollBar:horizontal {
    background-color: #323232;
    height: 15px;
    margin: 0 16px 0 16px;
    border-style: none;
}

QScrollBar::handle:horizontal
{   
    background-color: #3d3d3d;
    min-height: 15px;
    border-style: none;
}

QScrollBar::add-line:horizontal {
    border-style: none;
    background-color: #3d3d3d;
    width: 14px;
    subcontrol-position: right;
    subcontrol-origin: margin;
}

QScrollBar::sub-line:horizontal {
    border-style: none;
    background-color: #3d3d3d;
    width: 14px;
    subcontrol-position: left;
    subcontrol-origin: margin;
}

QScrollBar::right-arrow:horizontal, QScrollBar::left-arrow:horizontal
{
      border: 0px solid black;
      width: 4px;
      height: 4px;
      background-color: #d8d8d8;
}

QScrollBar::add-page:horizontal, QScrollBar::sub-page:horizontal
{
    background: none;
}

QScrollBar:vertical
{
    background-color: #323232;
    width: 15px;
    margin: 16px 0 16px 0;
    border-style: none;
}

QScrollBar::handle:vertical
{
    background-color: #3d3d3d;
    min-height: 15px;
    border-style: none;
}

QScrollBar::add-line:vertical
{
    border-style: none;
    background-color: #3d3d3d;
    height: 14px;
    subcontrol-position: bottom;
    subcontrol-origin: margin;
}

QScrollBar::sub-line:vertical
{
    border-style: none;
    background-color: #3d3d3d;
    height: 14px;
    subcontrol-position: top;
    subcontrol-origin: margin;
}

QScrollBar::up-arrow:vertical, QScrollBar::down-arrow:vertical
{
      border: 0px solid black;
      width: 4px;
      height: 4px;
      background-color: #d8d8d8;
}

QScrollBar::add-page:vertical, QScrollBar::sub-page:vertical
{
      background: none;
}

QTextEdit
{
    background-color: #242424;
}

QPlainTextEdit
{
    background-color: #242424;
}

QMainWindow::separator
{
    background-color: QLinearGradient(x1:0, y1:0, x2:0, y2:1, stop:0 #161616, stop: 0.5 #151515, stop: 0.6 #212121, stop:1 #343434);
    color: white;
    padding-left: 4px;
    border: 1px solid #4c4c4c;
    spacing: 3px; /* spacing between items in the tool bar */
}

QMainWindow::separator:hover
{

    background-color: QLinearGradient(x1:0, y1:0, x2:0, y2:1, stop:0 #d7801a, stop:0.5 #b56c17 stop:1 #ffa02f);
    color: white;
    padding-left: 4px;
    border: 1px solid #6c6c6c;
    spacing: 3px; /* spacing between items in the tool bar */
}

QToolBar::handle
{
     spacing: 3px; /* spacing between items in the tool bar */
     background: url(:/images/handle.png);
}

QToolButton
{
    background-color: QLinearGradient( x1: 0, y1: 0, x2: 0, y2: 1, stop: 0 #565656, stop: 0.1 #525252, stop: 0.5 #4e4e4e, stop: 0.9 #4a4a4a, stop: 1 #464646);
    border-style: none;
    border-radius: 5;
    spacing: 10px;
    padding-right: 12px;
}

QToolButton::menu-button
{
    color: white;
    background-color: QLinearGradient( x1: 0, y1: 0, x2: 0, y2: 1, stop: 0 #565656, stop: 0.1 #525252, stop: 0.5 #4e4e4e, stop: 0.9 #4a4a4a, stop: 1 #464646);
    border-style: none;
    border-radius: 5;
    padding-left: 10px;
    padding-right: 12px;
}

QToolButton::menu-arrow
{
    image: url(':/icons/images/down_arrow.png');
}

QMenu::separator
{
    height: 2px;
    background-color: QLinearGradient(x1:0, y1:0, x2:0, y2:1, stop:0 #161616, stop: 0.5 #151515, stop: 0.6 #212121, stop:1 #343434);
    color: white;
    padding-left: 4px;
    margin-left: 10px;
    margin-right: 5px;
}

QProgressBar
{
    border: 2px solid grey;
    border-radius: 5px;
    text-align: center;
}

QProgressBar::chunk
{
    background-color: #d7801a;
    width: 2.15px;
    margin: 0.5px;
}

QCheckBox::indicator:disabled, QRadioButton::indicator:disabled
{
    border: 1px solid #444;
}

QCheckBox::indicator:unchecked {
    image: url(:icons/images/check_box.png);
}

QCheckBox::indicator:unchecked:hover {
    image: url(:icons/images/check_box_hovered.png);
}

QCheckBox::indicator:checked {
    image: url(:icons/images/check_box_checked.png);
}

QCheckBox::indicator:checked:hover {
    image: url(:icons/images/check_box_checked_hovered.png);
}

QListWidget
{
    border-style: none;
    border-radius: 6;
    background-color: transparent;
    margin: 2;
}
/*
QListWidget::item
{
    border-style: none;
    border-radius: 10;
    background-color: #3d3d3d;

    border-width: 0px;
    padding: 0px;
    margin: 2;
}
*/

QListWidget::item
{
    border-style: none;
    background-color: transparent;
    padding: 0px;
    margin: 0;
}

QListWidget::item:selected
{
    background-color: #520000;
}

QSpinBox {
    padding-right: 15px; /* make room for the arrows */
    border-style: none;
    background-color: #242424;
}

QGroupBox {
    border-style: none;
    background-color: #3d3d3d;
    font-weight: bold;
    padding:2;
    padding-top:15;
    padding-left:6;
}

QGroupBox::title {
    subcontrol-origin: margin;
    margin: 5;
}

QGroupBox QGroupBox {
    background-color: #323232;
}

QGroupBox[stylevariant="device_override"] {
    background-color: #2c2c2c;
}

QHeaderView::section { 
    background-color:#323232; 
}

DeviceWidgetHeader
{
    border-style: none;
    background-color: #242424;
    font-weight: bold;
    border-width: 0px;
    padding: 0px;
    margin: 0;
    margin-top: 5;
    margin-bottom: 5;
}

DeviceWidgetItem
{
    border-style: none;
    border-radius: 10;
    background-color: #3d3d3d;

    border-width: 0px;
    padding: 0px;
    margin: 2;
   
}

DeviceWidgetItem[recording=true]
{
    background-color: #520000;
}

TransportQueueHeaderActionWidget
{
    border-style: none;
    background-color: #242424;
    font-weight: bold;
    border-width: 0px;
    padding: 0px;
    margin: 0;
    margin-top: 0;
    margin-bottom: 0;
}

QTabWidget::pane 
{
    border-top: 2px solid #242424;
    background-color: #242424;
}

QTabBar::tab 
{
    background-color: #242424;
    border: 2px solid #242424;
    border-bottom-color: #242424;
    border-top-left-radius: 4px;
    border-top-right-radius: 4px;
    min-width: 8ex;
    min-height: 3ex;
    padding: 4px;
}

QTabBar::tab:!selected
{
    margin-top: 2px; /* make it look slightly smaller */
    background-color: #2A2A2A;
    border: 2px solid #2A2A2A;
}

RedeployListenerDialog QPushButton:disabled {
    background-color: #242424;
}

ControlQPushButton#settings {
    icon: url(':/icons/images/icon_settings_disabled.png') off
        , url(':/icons/images/icon_settings.png') on;
}

ControlQPushButton#settings:hover {
    icon: url(':/icons/images/icon_settings_hover.png') on
        , url(':/icons/images/icon_settings_hover.png') off;
}

ControlQPushButton#autojoin {
    icon: url(':/icons/images/icon_autojoin_disabled.png') off
        , url(':/icons/images/icon_autojoin_disabled.png') disabled off
        , url(':/icons/images/icon_autojoin_disabled.png') disabled on
        , url(':/icons/images/icon_autojoin.png') on;
}

ControlQPushButton#autojoin:hover {
    icon: url(':/icons/images/icon_autojoin_hover.png') on
        , url(':/icons/images/icon_autojoin_disabled.png') disabled off
        , url(':/icons/images/icon_autojoin_disabled.png') disabled on
        , url(':/icons/images/icon_autojoin_hover.png') off;
}

ControlQPushButton#open {
    icon: url(':/icons/images/icon_open.png') off
        , url(':/icons/images/icon_open_disabled.png') disabled off
        , url(':/icons/images/icon_close.png') on
        , url(':/icons/images/icon_close_disabled.png') disabled on;
}

ControlQPushButton#open:hover {
    icon: url(':/icons/images/icon_open_hover.png') off
        , url(':/icons/images/icon_close_hover.png') on;
}


[widget_classes~="download"] ControlQPushButton#open {
    icon: url(':/icons/images/icon_download.png')
        , url(':/icons/images/icon_download_disabled.png') disabled;
}


ControlQPushButton#connect {
    icon: url(':/icons/images/icon_connect.png') off
        , url(':/icons/images/icon_connect_disabled.png') disabled off
        , url(':/icons/images/icon_connected.png') on
        , url(':/icons/images/icon_connected_disabled.png') disabled on;
}

ControlQPushButton#connect:hover {
    icon: url(':/icons/images/icon_connect_hover.png') off
        , url(':/icons/images/icon_connected_hover.png') on;
}


ControlQPushButton#sync {
    icon: url(':/icons/images/icon_sync.png')
        , url(':/icons/images/icon_sync_disabled.png') disabled;
}

ControlQPushButton#sync:hover {
    icon: url(':/icons/images/icon_sync_hover.png');
}


ControlQPushButton#build {
    icon: url(':/icons/images/icon_build.png')
        , url(':/icons/images/icon_build_disabled.png') disabled;
}

ControlQPushButton#build:hover {
    icon: url(':/icons/images/icon_build_hover.png');
}


ControlQPushButton#sync_and_build {
    icon: url(':/icons/images/icon_sync_and_build.png')
        , url(':/icons/images/icon_sync_and_build_disabled.png') disabled;
}

ControlQPushButton#sync_and_build:hover {
    icon: url(':/icons/images/icon_sync_and_build_hover.png');
}


ControlQPushButton#refresh {
    icon: url(':/icons/images/icon_refresh.png')
        , url(':/icons/images/icon_refresh_disabled.png') disabled;
}

ControlQPushButton#refresh:hover {
    icon: url(':/icons/images/icon_refresh_hover.png');
}

SearchableComboBox QLineEdit {
    border-radius: 5px;
}

SearchableComboBox::CustomQList QListWidget {
    outline: 0;
    border: none;
}

SearchableComboBox::CustomQList QListWidget::item {
    color:#D8D8D8;
}

SearchableComboBox::CustomQList QListWidget::item:selected {
    background:#0078D7;
}
