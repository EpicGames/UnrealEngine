<?xml version="1.0" encoding="UTF-8"?>
<ui version="4.0">
 <class>switchboard_window</class>
 <widget class="QMainWindow" name="switchboard_window">
  <property name="geometry">
   <rect>
    <x>0</x>
    <y>0</y>
    <width>1280</width>
    <height>952</height>
   </rect>
  </property>
  <property name="minimumSize">
   <size>
    <width>900</width>
    <height>500</height>
   </size>
  </property>
  <property name="windowTitle">
   <string>Switchboard</string>
  </property>
  <property name="windowIcon">
   <iconset resource="resources.qrc">
    <normaloff>:/icons/images/switchboard.png</normaloff>:/icons/images/switchboard.png</iconset>
  </property>
  <widget class="QWidget" name="centralwidget">
   <property name="sizePolicy">
    <sizepolicy hsizetype="Preferred" vsizetype="Preferred">
     <horstretch>0</horstretch>
     <verstretch>0</verstretch>
    </sizepolicy>
   </property>
   <layout class="QVBoxLayout" name="verticalLayout_3">
    <property name="bottomMargin">
     <number>2</number>
    </property>
    <item>
     <layout class="QHBoxLayout" name="horizontalLayout">
      <property name="spacing">
       <number>15</number>
      </property>
      <item>
       <widget class="QLabel" name="label_4">
        <property name="minimumSize">
         <size>
          <width>50</width>
          <height>0</height>
         </size>
        </property>
        <property name="font">
         <font>
          <family>Roboto</family>
          <pointsize>9</pointsize>
          <weight>75</weight>
          <bold>true</bold>
         </font>
        </property>
        <property name="text">
         <string>Sequence</string>
        </property>
        <property name="alignment">
         <set>Qt::AlignRight|Qt::AlignTrailing|Qt::AlignVCenter</set>
        </property>
       </widget>
      </item>
      <item>
       <widget class="FramelessQLineEdit" name="sequence_line_edit">
        <property name="sizePolicy">
         <sizepolicy hsizetype="MinimumExpanding" vsizetype="Fixed">
          <horstretch>0</horstretch>
          <verstretch>0</verstretch>
         </sizepolicy>
        </property>
        <property name="minimumSize">
         <size>
          <width>300</width>
          <height>0</height>
         </size>
        </property>
        <property name="maximumSize">
         <size>
          <width>16777215</width>
          <height>16777215</height>
         </size>
        </property>
        <property name="font">
         <font>
          <pointsize>20</pointsize>
         </font>
        </property>
        <property name="text">
         <string>Default</string>
        </property>
        <property name="recording" stdset="0">
         <bool>true</bool>
        </property>
       </widget>
      </item>
      <item>
       <widget class="QLabel" name="slate_label">
        <property name="enabled">
         <bool>true</bool>
        </property>
        <property name="sizePolicy">
         <sizepolicy hsizetype="Fixed" vsizetype="Preferred">
          <horstretch>0</horstretch>
          <verstretch>0</verstretch>
         </sizepolicy>
        </property>
        <property name="font">
         <font>
          <family>Roboto</family>
          <pointsize>9</pointsize>
          <weight>75</weight>
          <bold>true</bold>
         </font>
        </property>
        <property name="text">
         <string>Slate</string>
        </property>
        <property name="alignment">
         <set>Qt::AlignCenter</set>
        </property>
        <property name="wordWrap">
         <bool>true</bool>
        </property>
       </widget>
      </item>
      <item>
       <widget class="FramelessQLineEdit" name="slate_line_edit">
        <property name="sizePolicy">
         <sizepolicy hsizetype="MinimumExpanding" vsizetype="Fixed">
          <horstretch>0</horstretch>
          <verstretch>0</verstretch>
         </sizepolicy>
        </property>
        <property name="minimumSize">
         <size>
          <width>0</width>
          <height>0</height>
         </size>
        </property>
        <property name="maximumSize">
         <size>
          <width>16777215</width>
          <height>16777215</height>
         </size>
        </property>
        <property name="font">
         <font>
          <family>Roboto</family>
          <pointsize>20</pointsize>
         </font>
        </property>
        <property name="text">
         <string>SLATE</string>
        </property>
        <property name="alignment">
         <set>Qt::AlignLeading|Qt::AlignLeft|Qt::AlignVCenter</set>
        </property>
        <property name="placeholderText">
         <string/>
        </property>
       </widget>
      </item>
      <item>
       <widget class="QLabel" name="take_label">
        <property name="enabled">
         <bool>true</bool>
        </property>
        <property name="sizePolicy">
         <sizepolicy hsizetype="Fixed" vsizetype="Preferred">
          <horstretch>0</horstretch>
          <verstretch>0</verstretch>
         </sizepolicy>
        </property>
        <property name="font">
         <font>
          <family>Roboto</family>
          <pointsize>9</pointsize>
          <weight>75</weight>
          <bold>true</bold>
         </font>
        </property>
        <property name="text">
         <string>Take</string>
        </property>
        <property name="alignment">
         <set>Qt::AlignCenter</set>
        </property>
        <property name="wordWrap">
         <bool>true</bool>
        </property>
       </widget>
      </item>
      <item>
       <widget class="TakeSpinBox" name="take_spin_box">
        <property name="sizePolicy">
         <sizepolicy hsizetype="Fixed" vsizetype="Fixed">
          <horstretch>0</horstretch>
          <verstretch>0</verstretch>
         </sizepolicy>
        </property>
        <property name="minimumSize">
         <size>
          <width>80</width>
          <height>0</height>
         </size>
        </property>
        <property name="maximumSize">
         <size>
          <width>16777215</width>
          <height>16777215</height>
         </size>
        </property>
        <property name="font">
         <font>
          <family>Roboto</family>
          <pointsize>20</pointsize>
         </font>
        </property>
        <property name="alignment">
         <set>Qt::AlignCenter</set>
        </property>
        <property name="minimum">
         <number>1</number>
        </property>
        <property name="maximum">
         <number>999</number>
        </property>
       </widget>
      </item>
      <item>
       <widget class="QPushButton" name="record_button">
        <property name="enabled">
         <bool>true</bool>
        </property>
        <property name="sizePolicy">
         <sizepolicy hsizetype="Fixed" vsizetype="Fixed">
          <horstretch>0</horstretch>
          <verstretch>0</verstretch>
         </sizepolicy>
        </property>
        <property name="minimumSize">
         <size>
          <width>75</width>
          <height>75</height>
         </size>
        </property>
        <property name="maximumSize">
         <size>
          <width>75</width>
          <height>75</height>
         </size>
        </property>
        <property name="baseSize">
         <size>
          <width>0</width>
          <height>0</height>
         </size>
        </property>
        <property name="text">
         <string/>
        </property>
        <property name="icon">
         <iconset>
          <normalon>:/icons/images/record_stop.png</normalon>
         </iconset>
        </property>
        <property name="iconSize">
         <size>
          <width>60</width>
          <height>60</height>
         </size>
        </property>
        <property name="checkable">
         <bool>true</bool>
        </property>
        <property name="checked">
         <bool>false</bool>
        </property>
        <property name="no_background" stdset="0">
         <bool>true</bool>
        </property>
        <property name="no_border" stdset="0">
         <bool>true</bool>
        </property>
       </widget>
      </item>
     </layout>
    </item>
    <item>
     <widget class="Line" name="line">
      <property name="orientation">
       <enum>Qt::Horizontal</enum>
      </property>
     </widget>
    </item>
    <item>
     <layout class="QHBoxLayout" name="horizontalLayout_4">
      <item>
       <widget class="QWidget" name="device_widget" native="true">
        <property name="maximumSize">
         <size>
          <width>600</width>
          <height>16777215</height>
         </size>
        </property>
        <layout class="QVBoxLayout" name="verticalLayout">
         <property name="spacing">
          <number>6</number>
         </property>
         <property name="topMargin">
          <number>0</number>
         </property>
         <property name="bottomMargin">
          <number>0</number>
         </property>
         <item>
          <layout class="QHBoxLayout" name="horizontalLayout_7">
           <item>
            <widget class="QPushButton" name="settings_button">
             <property name="minimumSize">
              <size>
               <width>0</width>
               <height>0</height>
              </size>
             </property>
             <property name="styleSheet">
              <string notr="true">QPushButton { background-color: transparent; border: 0px }</string>
             </property>
             <property name="text">
              <string/>
             </property>
             <property name="icon">
              <iconset>
               <normalon>:/icons/images/Settings_40x.png</normalon>
              </iconset>
             </property>
            </widget>
           </item>
           <item>
            <widget class="QToolButton" name="device_add_tool_button">
             <property name="text">
              <string>Add Device</string>
             </property>
             <property name="popupMode">
              <enum>QToolButton::MenuButtonPopup</enum>
             </property>
             <property name="toolButtonStyle">
              <enum>Qt::ToolButtonTextOnly</enum>
             </property>
             <property name="arrowType">
              <enum>Qt::DownArrow</enum>
             </property>
            </widget>
           </item>
           <item>
            <spacer name="horizontalSpacer_2">
             <property name="orientation">
              <enum>Qt::Horizontal</enum>
             </property>
             <property name="sizeType">
              <enum>QSizePolicy::Minimum</enum>
             </property>
             <property name="sizeHint" stdset="0">
              <size>
               <width>50</width>
               <height>20</height>
              </size>
             </property>
            </spacer>
           </item>
           <item>
            <widget class="QLabel" name="label_2">
             <property name="minimumSize">
              <size>
               <width>0</width>
               <height>0</height>
              </size>
             </property>
             <property name="font">
              <font>
               <family>Roboto</family>
               <weight>75</weight>
               <bold>true</bold>
              </font>
             </property>
             <property name="text">
              <string>Level: </string>
             </property>
             <property name="alignment">
              <set>Qt::AlignRight|Qt::AlignTrailing|Qt::AlignVCenter</set>
             </property>
            </widget>
           </item>
          </layout>
         </item>
         <item>
          <layout class="QHBoxLayout" name="horizontalLayout_3">
           <property name="spacing">
            <number>2</number>
           </property>
           <property name="topMargin">
            <number>6</number>
           </property>
           <property name="rightMargin">
            <number>0</number>
           </property>
           <property name="bottomMargin">
            <number>0</number>
           </property>
           <item>
            <layout class="QVBoxLayout" name="verticalLayout_6">
             <item>
              <layout class="QHBoxLayout" name="horizontalLayout_10">
               <item>
                <widget class="QLabel" name="engine_cl_label">
                 <property name="minimumSize">
                  <size>
                   <width>60</width>
                   <height>0</height>
                  </size>
                 </property>
                 <property name="font">
                  <font>
                   <family>Roboto</family>
                   <weight>50</weight>
                   <bold>false</bold>
                  </font>
                 </property>
                 <property name="text">
                  <string>Engine CL</string>
                 </property>
                 <property name="alignment">
                  <set>Qt::AlignLeading|Qt::AlignLeft|Qt::AlignVCenter</set>
                 </property>
                </widget>
               </item>
               <item>
                <widget class="QComboBox" name="engine_cl_combo_box">
                 <property name="minimumSize">
                  <size>
                   <width>80</width>
                   <height>0</height>
                  </size>
                 </property>
                 <property name="editable">
                  <bool>true</bool>
                 </property>
                 <property name="currentText">
                  <string>-- None --</string>
                 </property>
                 <property name="maxVisibleItems">
                  <number>11</number>
                 </property>
                </widget>
               </item>
               <item>
                <widget class="ControlQPushButton" name="refresh_engine_cl_button">
                 <property name="maximumSize">
                  <size>
                   <width>22</width>
                   <height>22</height>
                  </size>
                 </property>
                 <property name="text">
                  <string/>
                 </property>
                 <property name="icon">
                  <iconset resource="resources.qrc">
                   <normaloff>:/icons/images/icon_refresh.png</normaloff>
                   <disabledoff>:/icons/images/icon_refresh_disabled.png</disabledoff>
                   <activeoff>:/icons/images/icon_refresh_hover.png</activeoff>:/icons/images/icon_refresh.png</iconset>
                 </property>
                 <property name="iconSize">
                  <size>
                   <width>18</width>
                   <height>18</height>
                  </size>
                 </property>
                 <property name="frameless" stdset="0">
                  <bool>true</bool>
                 </property>
                </widget>
               </item>
              </layout>
             </item>
             <item>
              <layout class="QHBoxLayout" name="horizontalLayout_9">
               <item>
                <widget class="QLabel" name="project_cl_label">
                 <property name="minimumSize">
                  <size>
                   <width>60</width>
                   <height>0</height>
                  </size>
                 </property>
                 <property name="font">
                  <font>
                   <family>Roboto</family>
                   <weight>50</weight>
                   <bold>false</bold>
                  </font>
                 </property>
                 <property name="text">
                  <string>Project CL</string>
                 </property>
                 <property name="alignment">
                  <set>Qt::AlignLeading|Qt::AlignLeft|Qt::AlignVCenter</set>
                 </property>
                </widget>
               </item>
               <item>
                <widget class="QComboBox" name="project_cl_combo_box">
                 <property name="sizePolicy">
                  <sizepolicy hsizetype="Preferred" vsizetype="Fixed">
                   <horstretch>0</horstretch>
                   <verstretch>0</verstretch>
                  </sizepolicy>
                 </property>
                 <property name="minimumSize">
                  <size>
                   <width>80</width>
                   <height>0</height>
                  </size>
                 </property>
                 <property name="maximumSize">
                  <size>
                   <width>16777215</width>
                   <height>16777215</height>
                  </size>
                 </property>
                 <property name="font">
                  <font>
                   <family>Roboto</family>
                  </font>
                 </property>
                 <property name="layoutDirection">
                  <enum>Qt::LeftToRight</enum>
                 </property>
                 <property name="editable">
                  <bool>true</bool>
                 </property>
                 <property name="currentText">
                  <string>5435842</string>
                 </property>
                 <property name="maxVisibleItems">
                  <number>11</number>
                 </property>
                 <item>
                  <property name="text">
                   <string>5435842</string>
                  </property>
                 </item>
                </widget>
               </item>
               <item>
                <widget class="ControlQPushButton" name="refresh_project_cl_button">
                 <property name="sizePolicy">
                  <sizepolicy hsizetype="Preferred" vsizetype="Fixed">
                   <horstretch>0</horstretch>
                   <verstretch>0</verstretch>
                  </sizepolicy>
                 </property>
                 <property name="maximumSize">
                  <size>
                   <width>22</width>
                   <height>22</height>
                  </size>
                 </property>
                 <property name="toolTip">
                  <string>Refresh Changelists</string>
                 </property>
                 <property name="text">
                  <string/>
                 </property>
                 <property name="icon">
                  <iconset resource="resources.qrc">
                   <normaloff>:/icons/images/icon_refresh.png</normaloff>
                   <disabledoff>:/icons/images/icon_refresh_disabled.png</disabledoff>
                   <activeoff>:/icons/images/icon_refresh_hover.png</activeoff>:/icons/images/icon_refresh.png</iconset>
                 </property>
                 <property name="iconSize">
                  <size>
                   <width>18</width>
                   <height>18</height>
                  </size>
                 </property>
                 <property name="frameless" stdset="0">
                  <bool>true</bool>
                 </property>
                </widget>
               </item>
              </layout>
             </item>
            </layout>
           </item>
           <item>
            <widget class="ControlQPushButton" name="sync_all_button">
             <property name="sizePolicy">
              <sizepolicy hsizetype="Preferred" vsizetype="Fixed">
               <horstretch>0</horstretch>
               <verstretch>0</verstretch>
              </sizepolicy>
             </property>
             <property name="minimumSize">
              <size>
               <width>25</width>
               <height>30</height>
              </size>
             </property>
             <property name="toolTip">
              <string>Sync all devices</string>
             </property>
             <property name="text">
              <string/>
             </property>
             <property name="icon">
              <iconset>
               <normalon>:/icons/images/icon_sync.png</normalon>
               <disabledon>:/icons/images/icon_sync_disabled.png</disabledon>
               <activeon>:/icons/images/icon_sync_hover.png</activeon>
              </iconset>
             </property>
             <property name="iconSize">
              <size>
               <width>21</width>
               <height>21</height>
              </size>
             </property>
             <property name="frameless" stdset="0">
              <bool>true</bool>
             </property>
            </widget>
           </item>
           <item>
            <widget class="ControlQPushButton" name="build_all_button">
             <property name="sizePolicy">
              <sizepolicy hsizetype="Preferred" vsizetype="Fixed">
               <horstretch>0</horstretch>
               <verstretch>0</verstretch>
              </sizepolicy>
             </property>
             <property name="minimumSize">
              <size>
               <width>25</width>
               <height>30</height>
              </size>
             </property>
             <property name="toolTip">
              <string>Build on all devices</string>
             </property>
             <property name="text">
              <string/>
             </property>
             <property name="icon">
              <iconset>
               <normalon>:/icons/images/icon_build.png</normalon>
               <activeon>:/icons/images/icon_build_hover.png</activeon>
              </iconset>
             </property>
             <property name="iconSize">
              <size>
               <width>21</width>
               <height>21</height>
              </size>
             </property>
             <property name="frameless" stdset="0">
              <bool>true</bool>
             </property>
            </widget>
           </item>
           <item>
            <widget class="ControlQPushButton" name="sync_and_build_all_button">
             <property name="sizePolicy">
              <sizepolicy hsizetype="Preferred" vsizetype="Fixed">
               <horstretch>0</horstretch>
               <verstretch>0</verstretch>
              </sizepolicy>
             </property>
             <property name="minimumSize">
              <size>
               <width>25</width>
               <height>30</height>
              </size>
             </property>
             <property name="toolTip">
              <string>Sync all devices and immediately build after completion</string>
             </property>
             <property name="text">
              <string/>
             </property>
             <property name="icon">
              <iconset>
               <normalon>:/icons/images/icon_sync_and_build.png</normalon>
               <disabledon>:/icons/images/icon_sync_and_build_disabled.png</disabledon>
               <activeon>:/icons/images/icon_sync_and_build_hover.png</activeon>
              </iconset>
             </property>
             <property name="iconSize">
              <size>
               <width>21</width>
               <height>21</height>
              </size>
             </property>
             <property name="frameless" stdset="0">
              <bool>true</bool>
             </property>
            </widget>
           </item>
           <item>
            <spacer name="horizontalSpacer">
             <property name="orientation">
              <enum>Qt::Horizontal</enum>
             </property>
             <property name="sizeHint" stdset="0">
              <size>
               <width>10</width>
               <height>20</height>
              </size>
             </property>
            </spacer>
           </item>
           <item>
            <widget class="ControlQPushButton" name="device_settings_button">
             <property name="enabled">
              <bool>false</bool>
             </property>
             <property name="sizePolicy">
              <sizepolicy hsizetype="Preferred" vsizetype="Fixed">
               <horstretch>0</horstretch>
               <verstretch>0</verstretch>
              </sizepolicy>
             </property>
             <property name="minimumSize">
              <size>
               <width>25</width>
               <height>30</height>
              </size>
             </property>
             <property name="toolTip">
              <string>Configure Unreal Insights capture</string>
             </property>
             <property name="text">
              <string/>
             </property>
             <property name="icon">
              <iconset resource="resources.qrc">
               <normaloff>:/icons/images/icon_settings_disabled.png</normaloff>
               <normalon>:/icons/images/icon_settings.png</normalon>
               <activeon>:/icons/images/icon_settings_hover.png</activeon>:/icons/images/icon_settings_disabled.png</iconset>
             </property>
             <property name="iconSize">
              <size>
               <width>21</width>
               <height>21</height>
              </size>
             </property>
             <property name="checkable">
              <bool>true</bool>
             </property>
             <property name="frameless" stdset="0">
              <bool>true</bool>
             </property>
            </widget>
           </item>
           <item>
            <widget class="ControlQPushButton" name="launch_all_button">
             <property name="enabled">
              <bool>false</bool>
             </property>
             <property name="sizePolicy">
              <sizepolicy hsizetype="Preferred" vsizetype="Fixed">
               <horstretch>0</horstretch>
               <verstretch>0</verstretch>
              </sizepolicy>
             </property>
             <property name="minimumSize">
              <size>
               <width>25</width>
               <height>30</height>
              </size>
             </property>
             <property name="toolTip">
              <string>Start all connected devices</string>
             </property>
             <property name="text">
              <string/>
             </property>
             <property name="icon">
              <iconset resource="resources.qrc">
               <normaloff>:/icons/images/icon_open.png</normaloff>
               <normalon>:/icons/images/icon_close.png</normalon>
               <disabledoff>:/icons/images/icon_open_disabled.png</disabledoff>
               <activeoff>:/icons/images/icon_open_hover.png</activeoff>
               <activeon>:/icons/images/icon_close_hover.png</activeon>:/icons/images/icon_open.png</iconset>
             </property>
             <property name="iconSize">
              <size>
               <width>21</width>
               <height>21</height>
              </size>
             </property>
             <property name="checkable">
              <bool>true</bool>
             </property>
             <property name="frameless" stdset="0">
              <bool>true</bool>
             </property>
            </widget>
           </item>
           <item>
            <widget class="ControlQPushButton" name="connect_all_button">
             <property name="sizePolicy">
              <sizepolicy hsizetype="Preferred" vsizetype="Fixed">
               <horstretch>0</horstretch>
               <verstretch>0</verstretch>
              </sizepolicy>
             </property>
             <property name="minimumSize">
              <size>
               <width>25</width>
               <height>30</height>
              </size>
             </property>
             <property name="toolTip">
              <string>Connect all devices</string>
             </property>
             <property name="text">
              <string/>
             </property>
             <property name="icon">
              <iconset resource="resources.qrc">
               <normaloff>:/icons/images/icon_connect.png</normaloff>
               <normalon>:/icons/images/icon_connected.png</normalon>
               <activeoff>:/icons/images/icon_connect_hover.png</activeoff>
               <activeon>:/icons/images/icon_connected_hover.png</activeon>:/icons/images/icon_connect.png</iconset>
             </property>
             <property name="iconSize">
              <size>
               <width>21</width>
               <height>21</height>
              </size>
             </property>
             <property name="checkable">
              <bool>true</bool>
             </property>
             <property name="frameless" stdset="0">
              <bool>true</bool>
             </property>
            </widget>
           </item>
          </layout>
         </item>
         <item>
          <widget class="DeviceListWidget" name="device_list_widget">
           <property name="sizePolicy">
            <sizepolicy hsizetype="Expanding" vsizetype="Expanding">
             <horstretch>0</horstretch>
             <verstretch>0</verstretch>
            </sizepolicy>
           </property>
           <property name="minimumSize">
            <size>
             <width>0</width>
             <height>0</height>
            </size>
           </property>
           <property name="verticalScrollBarPolicy">
            <enum>Qt::ScrollBarAsNeeded</enum>
           </property>
           <property name="horizontalScrollBarPolicy">
            <enum>Qt::ScrollBarAlwaysOff</enum>
           </property>
          </widget>
         </item>
         <item>
          <widget class="Line" name="line_3">
           <property name="orientation">
            <enum>Qt::Horizontal</enum>
           </property>
          </widget>
         </item>
         <item>
          <widget class="QLabel" name="muserver_label">
           <property name="text">
            <string>Multi-user Server</string>
           </property>
          </widget>
         </item>
         <item>
          <layout class="QVBoxLayout" name="verticalLayout_8">
           <property name="leftMargin">
            <number>2</number>
           </property>
           <property name="topMargin">
            <number>2</number>
           </property>
           <property name="rightMargin">
            <number>2</number>
           </property>
           <property name="bottomMargin">
            <number>2</number>
           </property>
           <item>
            <layout class="QHBoxLayout" name="horizontalLayout_5">
             <property name="leftMargin">
              <number>0</number>
             </property>
             <property name="topMargin">
              <number>0</number>
             </property>
             <property name="rightMargin">
              <number>0</number>
             </property>
             <property name="bottomMargin">
              <number>0</number>
             </property>
             <item>
              <widget class="ControlQPushButton" name="muserver_start_stop_button">
               <property name="enabled">
                <bool>true</bool>
               </property>
               <property name="toolTip">
                <string>Start / stop Unreal Multi-user Server</string>
               </property>
               <property name="icon">
                <iconset resource="resources.qrc">
                 <normaloff>:/icons/images/icon_open.png</normaloff>
                 <normalon>:/icons/images/icon_close.png</normalon>
                 <disabledoff>:/icons/images/icon_open_disabled.png</disabledoff>
                 <activeoff>:/icons/images/icon_open_hover.png</activeoff>
                 <activeon>:/icons/images/icon_close_hover.png</activeon>:/icons/images/icon_open.png</iconset>
               </property>
               <property name="checkable">
                <bool>true</bool>
               </property>
               <property name="frameless" stdset="0">
                <bool>true</bool>
               </property>
              </widget>
             </item>
             <item>
              <widget class="QCheckBox" name="use_device_autojoin_setting_checkbox">
               <property name="toolTip">
                <string>If this setting is checked then devices will join a Multi-user session based on their individual auto-join setting.</string>
               </property>
               <property name="layoutDirection">
                <enum>Qt::RightToLeft</enum>
               </property>
               <property name="text">
                <string>Auto-join</string>
               </property>
               <property name="checked">
                <bool>true</bool>
               </property>
              </widget>
             </item>
             <item>
              <widget class="Line" name="line_4">
               <property name="orientation">
                <enum>Qt::Vertical</enum>
               </property>
              </widget>
             </item>
             <item>
              <widget class="QLabel" name="label">
               <property name="font">
                <font>
                 <weight>75</weight>
                 <bold>true</bold>
                </font>
               </property>
               <property name="text">
                <string>Session:</string>
               </property>
              </widget>
             </item>
             <item>
              <widget class="FramelessQLineEdit" name="multiuser_session_lineEdit">
               <property name="sizePolicy">
                <sizepolicy hsizetype="MinimumExpanding" vsizetype="Fixed">
                 <horstretch>0</horstretch>
                 <verstretch>0</verstretch>
                </sizepolicy>
               </property>
               <property name="minimumSize">
                <size>
                 <width>0</width>
                 <height>0</height>
                </size>
               </property>
               <property name="maximumSize">
                <size>
                 <width>16777215</width>
                 <height>16777215</height>
                </size>
               </property>
               <property name="font">
                <font>
                 <pointsize>8</pointsize>
                </font>
               </property>
               <property name="text">
                <string>SessionName</string>
               </property>
               <property name="recording" stdset="0">
                <bool>true</bool>
               </property>
              </widget>
             </item>
             <item>
              <widget class="QPushButton" name="multiuser_session_inc_button">
               <property name="enabled">
                <bool>true</bool>
               </property>
               <property name="sizePolicy">
                <sizepolicy hsizetype="Minimum" vsizetype="Preferred">
                 <horstretch>0</horstretch>
                 <verstretch>0</verstretch>
                </sizepolicy>
               </property>
               <property name="toolTip">
                <string>Increment the multi-user server session name.</string>
               </property>
               <property name="icon">
                <iconset resource="resources.qrc">
                 <normaloff>:/icons/images/icon_add_disabled.png</normaloff>
                 <normalon>:/icons/images/icon_add.png</normalon>:/icons/images/icon_add_disabled.png</iconset>
               </property>
               <property name="frameless" stdset="0">
                <bool>true</bool>
               </property>
              </widget>
             </item>
            </layout>
           </item>
          </layout>
         </item>
         <item>
          <layout class="QHBoxLayout" name="horizontalLayout_2">
           <property name="spacing">
            <number>3</number>
           </property>
          </layout>
         </item>
        </layout>
       </widget>
      </item>
      <item>
       <widget class="QFrame" name="frame_right">
        <property name="minimumSize">
         <size>
          <width>100</width>
          <height>0</height>
         </size>
        </property>
        <property name="frameShape">
         <enum>QFrame::StyledPanel</enum>
        </property>
        <property name="frameShadow">
         <enum>QFrame::Raised</enum>
        </property>
        <layout class="QVBoxLayout" name="verticalLayout_2">
         <property name="spacing">
          <number>6</number>
         </property>
         <property name="leftMargin">
          <number>0</number>
         </property>
         <property name="topMargin">
          <number>0</number>
         </property>
         <property name="rightMargin">
          <number>0</number>
         </property>
         <property name="bottomMargin">
          <number>0</number>
         </property>
         <item>
          <widget class="QSplitter" name="splitter">
           <property name="orientation">
            <enum>Qt::Vertical</enum>
           </property>
           <widget class="QTabWidget" name="tabs_main">
            <property name="currentIndex">
             <number>-1</number>
            </property>
           </widget>
           <widget class="QFrame" name="frame_logger">
            <property name="minimumSize">
             <size>
              <width>0</width>
              <height>100</height>
             </size>
            </property>
            <property name="frameShape">
             <enum>QFrame::StyledPanel</enum>
            </property>
            <property name="frameShadow">
             <enum>QFrame::Raised</enum>
            </property>
            <layout class="QVBoxLayout" name="verticalLayout_4">
             <property name="spacing">
              <number>4</number>
             </property>
             <property name="leftMargin">
              <number>0</number>
             </property>
             <property name="topMargin">
              <number>0</number>
             </property>
             <property name="rightMargin">
              <number>0</number>
             </property>
             <property name="bottomMargin">
              <number>0</number>
             </property>
             <item>
              <layout class="QHBoxLayout" name="logger_options">
               <item>
                <widget class="QComboBox" name="logger_level_comboBox">
                 <property name="sizePolicy">
                  <sizepolicy hsizetype="Preferred" vsizetype="Fixed">
                   <horstretch>1</horstretch>
                   <verstretch>0</verstretch>
                  </sizepolicy>
                 </property>
                 <property name="currentIndex">
                  <number>2</number>
                 </property>
                 <item>
                  <property name="text">
                   <string>Message</string>
                  </property>
                 </item>
                 <item>
                  <property name="text">
                   <string>OSC</string>
                  </property>
                 </item>
                 <item>
                  <property name="text">
                   <string>Debug</string>
                  </property>
                 </item>
                 <item>
                  <property name="text">
                   <string>Info</string>
                  </property>
                 </item>
                </widget>
               </item>
               <item>
                <spacer name="logger_options_spacer_0">
                 <property name="orientation">
                  <enum>Qt::Horizontal</enum>
                 </property>
                 <property name="sizeHint" stdset="0">
                  <size>
                   <width>40</width>
                   <height>20</height>
                  </size>
                 </property>
                </spacer>
               </item>
               <item>
                <widget class="QCheckBox" name="logger_wrap_checkbox">
                 <property name="toolTip">
                  <string>If checked, long lines are wrapped to fit within view. Otherwise, you may need to scroll horizontally to read long lines.</string>
                 </property>
                </widget>
               </item>
               <item>
                <widget class="QLabel" name="logger_wrap_label">
                 <property name="toolTip">
                  <string>If checked, long lines are wrapped to fit within view. Otherwise, you may need to scroll horizontally to read long lines.</string>
                 </property>
                 <property name="text">
                  <string>Line Wrap</string>
                 </property>
                </widget>
               </item>
               <item>
                <spacer name="logger_options_spacer_1">
                 <property name="orientation">
                  <enum>Qt::Horizontal</enum>
                 </property>
                 <property name="sizeHint" stdset="0">
                  <size>
                   <width>20</width>
                   <height>20</height>
                  </size>
                 </property>
                </spacer>
               </item>
               <item>
                <widget class="QCheckBox" name="logger_autoscroll_checkbox">
                 <property name="toolTip">
                  <string>If checked, the log view will automatically scroll to the end any time new output is appended.</string>
                 </property>
                </widget>
               </item>
               <item>
                <widget class="QLabel" name="logger_autoscroll_label">
                 <property name="toolTip">
                  <string>If checked, the log view will automatically scroll to the end any time new output is appended.</string>
                 </property>
                 <property name="text">
                  <string>Auto Scroll</string>
                 </property>
                </widget>
               </item>
               <item>
                <spacer name="logger_options_spacer_2">
                 <property name="orientation">
                  <enum>Qt::Horizontal</enum>
                 </property>
                 <property name="sizeHint" stdset="0">
                  <size>
                   <width>20</width>
                   <height>0</height>
                  </size>
                 </property>
                </spacer>
               </item>
               <item>
                <widget class="QPushButton" name="clear_log_button">
                 <property name="text">
                  <string>Clear</string>
                 </property>
                 <property name="toolTip">
                  <string>Clear the log window</string>
                 </property>
                </widget>
               </item>
              </layout>
             </item>
             <item>
              <widget class="QPlainTextEdit" name="base_console">
               <property name="palette">
                <palette>
                 <active>
                  <colorrole role="WindowText">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>177</red>
                     <green>177</green>
                     <blue>177</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Button">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>36</red>
                     <green>36</green>
                     <blue>36</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Text">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>177</red>
                     <green>177</green>
                     <blue>177</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="ButtonText">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>177</red>
                     <green>177</green>
                     <blue>177</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Base">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>36</red>
                     <green>36</green>
                     <blue>36</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Window">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>36</red>
                     <green>36</green>
                     <blue>36</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Highlight">
                   <brush brushstyle="LinearGradientPattern">
                    <gradient startx="0.000000000000000" starty="0.000000000000000" endx="0.000000000000000" endy="1.000000000000000" type="LinearGradient" spread="PadSpread" coordinatemode="ObjectBoundingMode">
                     <gradientstop position="0.000000000000000">
                      <color alpha="255">
                       <red>255</red>
                       <green>160</green>
                       <blue>47</blue>
                      </color>
                     </gradientstop>
                     <gradientstop position="1.000000000000000">
                      <color alpha="255">
                       <red>215</red>
                       <green>128</green>
                       <blue>26</blue>
                      </color>
                     </gradientstop>
                    </gradient>
                   </brush>
                  </colorrole>
                 </active>
                 <inactive>
                  <colorrole role="WindowText">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>177</red>
                     <green>177</green>
                     <blue>177</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Button">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>36</red>
                     <green>36</green>
                     <blue>36</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Text">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>177</red>
                     <green>177</green>
                     <blue>177</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="ButtonText">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>177</red>
                     <green>177</green>
                     <blue>177</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Base">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>36</red>
                     <green>36</green>
                     <blue>36</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Window">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>36</red>
                     <green>36</green>
                     <blue>36</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Highlight">
                   <brush brushstyle="LinearGradientPattern">
                    <gradient startx="0.000000000000000" starty="0.000000000000000" endx="0.000000000000000" endy="1.000000000000000" type="LinearGradient" spread="PadSpread" coordinatemode="ObjectBoundingMode">
                     <gradientstop position="0.000000000000000">
                      <color alpha="255">
                       <red>255</red>
                       <green>160</green>
                       <blue>47</blue>
                      </color>
                     </gradientstop>
                     <gradientstop position="1.000000000000000">
                      <color alpha="255">
                       <red>215</red>
                       <green>128</green>
                       <blue>26</blue>
                      </color>
                     </gradientstop>
                    </gradient>
                   </brush>
                  </colorrole>
                 </inactive>
                 <disabled>
                  <colorrole role="WindowText">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>64</red>
                     <green>64</green>
                     <blue>64</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Button">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>50</red>
                     <green>50</green>
                     <blue>50</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Text">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>64</red>
                     <green>64</green>
                     <blue>64</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="ButtonText">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>64</red>
                     <green>64</green>
                     <blue>64</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Base">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>50</red>
                     <green>50</green>
                     <blue>50</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Window">
                   <brush brushstyle="SolidPattern">
                    <color alpha="255">
                     <red>50</red>
                     <green>50</green>
                     <blue>50</blue>
                    </color>
                   </brush>
                  </colorrole>
                  <colorrole role="Highlight">
                   <brush brushstyle="LinearGradientPattern">
                    <gradient startx="0.000000000000000" starty="0.000000000000000" endx="0.000000000000000" endy="1.000000000000000" type="LinearGradient" spread="PadSpread" coordinatemode="ObjectBoundingMode">
                     <gradientstop position="0.000000000000000">
                      <color alpha="255">
                       <red>255</red>
                       <green>160</green>
                       <blue>47</blue>
                      </color>
                     </gradientstop>
                     <gradientstop position="1.000000000000000">
                      <color alpha="255">
                       <red>215</red>
                       <green>128</green>
                       <blue>26</blue>
                      </color>
                     </gradientstop>
                    </gradient>
                   </brush>
                  </colorrole>
                 </disabled>
                </palette>
               </property>
               <property name="font">
                <font>
                 <family>DroidSansMono</family>
                </font>
               </property>
               <property name="undoRedoEnabled">
                <bool>false</bool>
               </property>
               <property name="lineWrapMode">
                <enum>QPlainTextEdit::NoWrap</enum>
               </property>
               <property name="readOnly">
                <bool>true</bool>
               </property>
               <property name="centerOnScroll">
                <bool>false</bool>
               </property>
              </widget>
             </item>
             <item>
              <layout class="QHBoxLayout" name="horizontalLayout_6">
               <item>
                <spacer name="horizontalSpacer_3">
                 <property name="orientation">
                  <enum>Qt::Horizontal</enum>
                 </property>
                 <property name="sizeHint" stdset="0">
                  <size>
                   <width>40</width>
                   <height>20</height>
                  </size>
                 </property>
                </spacer>
               </item>
               <item>
                <widget class="QLabel" name="recording_status">
                 <property name="text">
                  <string>Recording Status</string>
                 </property>
                </widget>
               </item>
               <item>
                <widget class="Line" name="line_6">
                 <property name="orientation">
                  <enum>Qt::Vertical</enum>
                 </property>
                </widget>
               </item>
               <item>
                <widget class="QLineEdit" name="current_address_value">
                 <property name="sizePolicy">
                  <sizepolicy hsizetype="Preferred" vsizetype="Fixed">
                   <horstretch>0</horstretch>
                   <verstretch>0</verstretch>
                  </sizepolicy>
                 </property>
                 <property name="styleSheet">
                  <string notr="true">border: 0px; background-color: rgba(0, 0, 0, 0)</string>
                 </property>
                 <property name="text">
                  <string>Address</string>
                 </property>
                 <property name="alignment">
                  <set>Qt::AlignRight|Qt::AlignTrailing|Qt::AlignVCenter</set>
                 </property>
                </widget>
               </item>
               <item>
                <widget class="Line" name="line_5">
                 <property name="orientation">
                  <enum>Qt::Vertical</enum>
                 </property>
                </widget>
               </item>
               <item>
                <widget class="QLabel" name="current_config_file_value">
                 <property name="text">
                  <string>TextLabel</string>
                 </property>
                </widget>
               </item>
              </layout>
             </item>
            </layout>
           </widget>
          </widget>
         </item>
        </layout>
       </widget>
      </item>
      <item>
       <widget class="Line" name="line_2">
        <property name="orientation">
         <enum>Qt::Vertical</enum>
        </property>
       </widget>
      </item>
     </layout>
    </item>
   </layout>
   <zorder>line</zorder>
  </widget>
  <widget class="QStatusBar" name="statusbar"/>
  <widget class="QMenuBar" name="menu_bar">
   <property name="geometry">
    <rect>
     <x>0</x>
     <y>0</y>
     <width>1192</width>
     <height>21</height>
    </rect>
   </property>
   <widget class="QMenu" name="menu_configs">
    <property name="title">
     <string>Settings</string>
    </property>
    <addaction name="update_settings"/>
   </widget>
   <widget class="QMenu" name="menuConfigs">
    <property name="title">
     <string>Configs</string>
    </property>
    <widget class="QMenu" name="menu_load_config">
     <property name="title">
      <string>Load Config</string>
     </property>
    </widget>
    <addaction name="menu_new_config"/>
    <addaction name="menu_save_config_as"/>
    <addaction name="menu_delete_config"/>
    <addaction name="separator"/>
    <addaction name="menu_load_config"/>
   </widget>
   <addaction name="menuConfigs"/>
   <addaction name="menu_configs"/>
  </widget>
  <action name="menu_p4_refresh_cl">
   <property name="text">
    <string>Refresh CL</string>
   </property>
  </action>
  <action name="actionMinimal_Sync">
   <property name="text">
    <string>Minimal Sync</string>
   </property>
  </action>
  <action name="update_settings">
   <property name="text">
    <string>Settings</string>
   </property>
  </action>
  <action name="actionAsd">
   <property name="text">
    <string>asd</string>
   </property>
  </action>
  <action name="menu_new_config">
   <property name="text">
    <string>New Config</string>
   </property>
  </action>
  <action name="menu_save_config_as">
   <property name="text">
    <string>Save Config As...</string>
   </property>
  </action>
  <action name="menu_delete_config">
   <property name="text">
    <string>Delete Current Config</string>
   </property>
  </action>
  <action name="actionTest">
   <property name="text">
    <string>Test</string>
   </property>
  </action>
 </widget>
 <customwidgets>
  <customwidget>
   <class>FramelessQLineEdit</class>
   <extends>QLineEdit</extends>
   <header>switchboard/switchboard.h</header>
  </customwidget>
  <customwidget>
   <class>DeviceListWidget</class>
   <extends>QListWidget</extends>
   <header>switchboard/switchboard.h</header>
  </customwidget>
  <customwidget>
   <class>ControlQPushButton</class>
   <extends>QPushButton</extends>
   <header>switchboard/switchboard.h</header>
  </customwidget>
  <customwidget>
   <class>TakeSpinBox</class>
   <extends>QSpinBox</extends>
   <header>switchboard/switchboard.h</header>
  </customwidget>
 </customwidgets>
 <resources>
  <include location="resources.qrc"/>
 </resources>
 <connections/>
</ui>
