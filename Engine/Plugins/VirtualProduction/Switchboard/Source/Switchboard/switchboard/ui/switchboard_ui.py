# Copyright Epic Games, Inc. All Rights Reserved.
#
# To regenerate:
#     > cd (...)\Engine\Plugins\VirtualProduction\Switchboard\Source\Switchboard\switchboard
#     > ..\..\..\..\..\..\Extras\ThirdPartyNotUE\SwitchboardThirdParty\Python\Scripts\pyside6-uic.exe ui\switchboard.ui -o ui\switchboard_ui.py
#
# Manual fix-ups:
#  * Add this comment block, including copyright, to the top of the file
#  * Fix generated `from switchboard` import paths
#  * Remove `import resources_rc`
#  * Change Ui_switchboard_window base class from `object` to `QMainWindow`

################################################################################
## Form generated from reading UI file 'switchboard.ui'
##
## Created by: Qt User Interface Compiler version 6.5.3
##
## WARNING! All changes made in this file will be lost when recompiling UI file!
################################################################################

from PySide6.QtCore import (QCoreApplication, QDate, QDateTime, QLocale,
    QMetaObject, QObject, QPoint, QRect,
    QSize, QTime, QUrl, Qt)
from PySide6.QtGui import (QAction, QBrush, QColor, QConicalGradient,
    QCursor, QFont, QFontDatabase, QGradient,
    QIcon, QImage, QKeySequence, QLinearGradient,
    QPainter, QPalette, QPixmap, QRadialGradient,
    QTransform)
from PySide6.QtWidgets import (QApplication, QCheckBox, QComboBox, QFrame,
    QHBoxLayout, QLabel, QLineEdit, QListWidgetItem,
    QMainWindow, QMenu, QMenuBar, QPlainTextEdit,
    QPushButton, QSizePolicy, QSpacerItem, QSplitter,
    QStatusBar, QTabWidget, QToolButton, QVBoxLayout,
    QWidget)

from switchboard.device_list_widget import DeviceListWidget
from switchboard.switchboard_widgets import (ControlQPushButton, FramelessQLineEdit)
from switchboard.ui.take_spin_box import TakeSpinBox


class Ui_switchboard_window(QMainWindow):
    def setupUi(self, switchboard_window):
        if not switchboard_window.objectName():
            switchboard_window.setObjectName(u"switchboard_window")
        switchboard_window.resize(1280, 952)
        switchboard_window.setMinimumSize(QSize(900, 500))
        icon = QIcon()
        icon.addFile(u":/icons/images/switchboard.png", QSize(), QIcon.Normal, QIcon.Off)
        switchboard_window.setWindowIcon(icon)
        self.menu_p4_refresh_cl = QAction(switchboard_window)
        self.menu_p4_refresh_cl.setObjectName(u"menu_p4_refresh_cl")
        self.actionMinimal_Sync = QAction(switchboard_window)
        self.actionMinimal_Sync.setObjectName(u"actionMinimal_Sync")
        self.update_settings = QAction(switchboard_window)
        self.update_settings.setObjectName(u"update_settings")
        self.actionAsd = QAction(switchboard_window)
        self.actionAsd.setObjectName(u"actionAsd")
        self.menu_new_config = QAction(switchboard_window)
        self.menu_new_config.setObjectName(u"menu_new_config")
        self.menu_save_config_as = QAction(switchboard_window)
        self.menu_save_config_as.setObjectName(u"menu_save_config_as")
        self.menu_delete_config = QAction(switchboard_window)
        self.menu_delete_config.setObjectName(u"menu_delete_config")
        self.actionTest = QAction(switchboard_window)
        self.actionTest.setObjectName(u"actionTest")
        self.centralwidget = QWidget(switchboard_window)
        self.centralwidget.setObjectName(u"centralwidget")
        sizePolicy = QSizePolicy(QSizePolicy.Preferred, QSizePolicy.Preferred)
        sizePolicy.setHorizontalStretch(0)
        sizePolicy.setVerticalStretch(0)
        sizePolicy.setHeightForWidth(self.centralwidget.sizePolicy().hasHeightForWidth())
        self.centralwidget.setSizePolicy(sizePolicy)
        self.verticalLayout_3 = QVBoxLayout(self.centralwidget)
        self.verticalLayout_3.setObjectName(u"verticalLayout_3")
        self.verticalLayout_3.setContentsMargins(-1, -1, -1, 2)
        self.horizontalLayout = QHBoxLayout()
        self.horizontalLayout.setSpacing(15)
        self.horizontalLayout.setObjectName(u"horizontalLayout")
        self.label_4 = QLabel(self.centralwidget)
        self.label_4.setObjectName(u"label_4")
        self.label_4.setMinimumSize(QSize(50, 0))
        font = QFont()
        font.setFamilies([u"Roboto"])
        font.setPointSize(9)
        font.setBold(True)
        self.label_4.setFont(font)
        self.label_4.setAlignment(Qt.AlignRight|Qt.AlignTrailing|Qt.AlignVCenter)

        self.horizontalLayout.addWidget(self.label_4)

        self.sequence_line_edit = FramelessQLineEdit(self.centralwidget)
        self.sequence_line_edit.setObjectName(u"sequence_line_edit")
        sizePolicy1 = QSizePolicy(QSizePolicy.MinimumExpanding, QSizePolicy.Fixed)
        sizePolicy1.setHorizontalStretch(0)
        sizePolicy1.setVerticalStretch(0)
        sizePolicy1.setHeightForWidth(self.sequence_line_edit.sizePolicy().hasHeightForWidth())
        self.sequence_line_edit.setSizePolicy(sizePolicy1)
        self.sequence_line_edit.setMinimumSize(QSize(300, 0))
        self.sequence_line_edit.setMaximumSize(QSize(16777215, 16777215))
        font1 = QFont()
        font1.setPointSize(20)
        self.sequence_line_edit.setFont(font1)
        self.sequence_line_edit.setProperty("recording", True)

        self.horizontalLayout.addWidget(self.sequence_line_edit)

        self.slate_label = QLabel(self.centralwidget)
        self.slate_label.setObjectName(u"slate_label")
        self.slate_label.setEnabled(True)
        sizePolicy2 = QSizePolicy(QSizePolicy.Fixed, QSizePolicy.Preferred)
        sizePolicy2.setHorizontalStretch(0)
        sizePolicy2.setVerticalStretch(0)
        sizePolicy2.setHeightForWidth(self.slate_label.sizePolicy().hasHeightForWidth())
        self.slate_label.setSizePolicy(sizePolicy2)
        self.slate_label.setFont(font)
        self.slate_label.setAlignment(Qt.AlignCenter)
        self.slate_label.setWordWrap(True)

        self.horizontalLayout.addWidget(self.slate_label)

        self.slate_line_edit = FramelessQLineEdit(self.centralwidget)
        self.slate_line_edit.setObjectName(u"slate_line_edit")
        sizePolicy1.setHeightForWidth(self.slate_line_edit.sizePolicy().hasHeightForWidth())
        self.slate_line_edit.setSizePolicy(sizePolicy1)
        self.slate_line_edit.setMinimumSize(QSize(0, 0))
        self.slate_line_edit.setMaximumSize(QSize(16777215, 16777215))
        font2 = QFont()
        font2.setFamilies([u"Roboto"])
        font2.setPointSize(20)
        self.slate_line_edit.setFont(font2)
        self.slate_line_edit.setAlignment(Qt.AlignLeading|Qt.AlignLeft|Qt.AlignVCenter)

        self.horizontalLayout.addWidget(self.slate_line_edit)

        self.take_label = QLabel(self.centralwidget)
        self.take_label.setObjectName(u"take_label")
        self.take_label.setEnabled(True)
        sizePolicy2.setHeightForWidth(self.take_label.sizePolicy().hasHeightForWidth())
        self.take_label.setSizePolicy(sizePolicy2)
        self.take_label.setFont(font)
        self.take_label.setAlignment(Qt.AlignCenter)
        self.take_label.setWordWrap(True)

        self.horizontalLayout.addWidget(self.take_label)

        self.take_spin_box = TakeSpinBox(self.centralwidget)
        self.take_spin_box.setObjectName(u"take_spin_box")
        sizePolicy3 = QSizePolicy(QSizePolicy.Fixed, QSizePolicy.Fixed)
        sizePolicy3.setHorizontalStretch(0)
        sizePolicy3.setVerticalStretch(0)
        sizePolicy3.setHeightForWidth(self.take_spin_box.sizePolicy().hasHeightForWidth())
        self.take_spin_box.setSizePolicy(sizePolicy3)
        self.take_spin_box.setMinimumSize(QSize(80, 0))
        self.take_spin_box.setMaximumSize(QSize(16777215, 16777215))
        self.take_spin_box.setFont(font2)
        self.take_spin_box.setAlignment(Qt.AlignCenter)
        self.take_spin_box.setMinimum(1)
        self.take_spin_box.setMaximum(999)

        self.horizontalLayout.addWidget(self.take_spin_box)

        self.record_button = QPushButton(self.centralwidget)
        self.record_button.setObjectName(u"record_button")
        self.record_button.setEnabled(True)
        sizePolicy3.setHeightForWidth(self.record_button.sizePolicy().hasHeightForWidth())
        self.record_button.setSizePolicy(sizePolicy3)
        self.record_button.setMinimumSize(QSize(75, 75))
        self.record_button.setMaximumSize(QSize(75, 75))
        self.record_button.setBaseSize(QSize(0, 0))
        icon1 = QIcon()
        icon1.addFile(u":/icons/images/record_stop.png", QSize(), QIcon.Normal, QIcon.On)
        self.record_button.setIcon(icon1)
        self.record_button.setIconSize(QSize(60, 60))
        self.record_button.setCheckable(True)
        self.record_button.setChecked(False)
        self.record_button.setProperty("no_background", True)
        self.record_button.setProperty("no_border", True)

        self.horizontalLayout.addWidget(self.record_button)


        self.verticalLayout_3.addLayout(self.horizontalLayout)

        self.line = QFrame(self.centralwidget)
        self.line.setObjectName(u"line")
        self.line.setFrameShape(QFrame.HLine)
        self.line.setFrameShadow(QFrame.Sunken)

        self.verticalLayout_3.addWidget(self.line)

        self.horizontalLayout_4 = QHBoxLayout()
        self.horizontalLayout_4.setObjectName(u"horizontalLayout_4")
        self.device_widget = QWidget(self.centralwidget)
        self.device_widget.setObjectName(u"device_widget")
        self.device_widget.setMaximumSize(QSize(600, 16777215))
        self.verticalLayout = QVBoxLayout(self.device_widget)
        self.verticalLayout.setSpacing(6)
        self.verticalLayout.setObjectName(u"verticalLayout")
        self.verticalLayout.setContentsMargins(-1, 0, -1, 0)
        self.horizontalLayout_7 = QHBoxLayout()
        self.horizontalLayout_7.setObjectName(u"horizontalLayout_7")
        self.settings_button = QPushButton(self.device_widget)
        self.settings_button.setObjectName(u"settings_button")
        self.settings_button.setMinimumSize(QSize(0, 0))
        self.settings_button.setStyleSheet(u"QPushButton { background-color: transparent; border: 0px }")
        icon2 = QIcon()
        icon2.addFile(u":/icons/images/Settings_40x.png", QSize(), QIcon.Normal, QIcon.On)
        self.settings_button.setIcon(icon2)

        self.horizontalLayout_7.addWidget(self.settings_button)

        self.device_add_tool_button = QToolButton(self.device_widget)
        self.device_add_tool_button.setObjectName(u"device_add_tool_button")
        self.device_add_tool_button.setPopupMode(QToolButton.MenuButtonPopup)
        self.device_add_tool_button.setToolButtonStyle(Qt.ToolButtonTextOnly)
        self.device_add_tool_button.setArrowType(Qt.DownArrow)

        self.horizontalLayout_7.addWidget(self.device_add_tool_button)

        self.horizontalSpacer_2 = QSpacerItem(50, 20, QSizePolicy.Minimum, QSizePolicy.Minimum)

        self.horizontalLayout_7.addItem(self.horizontalSpacer_2)

        self.label_2 = QLabel(self.device_widget)
        self.label_2.setObjectName(u"label_2")
        self.label_2.setMinimumSize(QSize(0, 0))
        font3 = QFont()
        font3.setFamilies([u"Roboto"])
        font3.setBold(True)
        self.label_2.setFont(font3)
        self.label_2.setAlignment(Qt.AlignRight|Qt.AlignTrailing|Qt.AlignVCenter)

        self.horizontalLayout_7.addWidget(self.label_2)


        self.verticalLayout.addLayout(self.horizontalLayout_7)

        self.horizontalLayout_3 = QHBoxLayout()
        self.horizontalLayout_3.setSpacing(2)
        self.horizontalLayout_3.setObjectName(u"horizontalLayout_3")
        self.horizontalLayout_3.setContentsMargins(-1, 6, 0, 0)
        self.verticalLayout_6 = QVBoxLayout()
        self.verticalLayout_6.setObjectName(u"verticalLayout_6")
        self.horizontalLayout_10 = QHBoxLayout()
        self.horizontalLayout_10.setObjectName(u"horizontalLayout_10")
        self.engine_cl_label = QLabel(self.device_widget)
        self.engine_cl_label.setObjectName(u"engine_cl_label")
        self.engine_cl_label.setMinimumSize(QSize(60, 0))
        font4 = QFont()
        font4.setFamilies([u"Roboto"])
        font4.setBold(False)
        self.engine_cl_label.setFont(font4)
        self.engine_cl_label.setAlignment(Qt.AlignLeading|Qt.AlignLeft|Qt.AlignVCenter)

        self.horizontalLayout_10.addWidget(self.engine_cl_label)

        self.engine_cl_combo_box = QComboBox(self.device_widget)
        self.engine_cl_combo_box.setObjectName(u"engine_cl_combo_box")
        self.engine_cl_combo_box.setMinimumSize(QSize(80, 0))
        self.engine_cl_combo_box.setEditable(True)
        self.engine_cl_combo_box.setMaxVisibleItems(11)

        self.horizontalLayout_10.addWidget(self.engine_cl_combo_box)

        self.refresh_engine_cl_button = ControlQPushButton(self.device_widget)
        self.refresh_engine_cl_button.setObjectName(u"refresh_engine_cl_button")
        self.refresh_engine_cl_button.setMaximumSize(QSize(22, 22))
        icon3 = QIcon()
        icon3.addFile(u":/icons/images/icon_refresh.png", QSize(), QIcon.Normal, QIcon.Off)
        icon3.addFile(u":/icons/images/icon_refresh_disabled.png", QSize(), QIcon.Disabled, QIcon.Off)
        icon3.addFile(u":/icons/images/icon_refresh_hover.png", QSize(), QIcon.Active, QIcon.Off)
        self.refresh_engine_cl_button.setIcon(icon3)
        self.refresh_engine_cl_button.setIconSize(QSize(18, 18))
        self.refresh_engine_cl_button.setProperty("frameless", True)

        self.horizontalLayout_10.addWidget(self.refresh_engine_cl_button)


        self.verticalLayout_6.addLayout(self.horizontalLayout_10)

        self.horizontalLayout_9 = QHBoxLayout()
        self.horizontalLayout_9.setObjectName(u"horizontalLayout_9")
        self.project_cl_label = QLabel(self.device_widget)
        self.project_cl_label.setObjectName(u"project_cl_label")
        self.project_cl_label.setMinimumSize(QSize(60, 0))
        self.project_cl_label.setFont(font4)
        self.project_cl_label.setAlignment(Qt.AlignLeading|Qt.AlignLeft|Qt.AlignVCenter)

        self.horizontalLayout_9.addWidget(self.project_cl_label)

        self.project_cl_combo_box = QComboBox(self.device_widget)
        self.project_cl_combo_box.addItem("")
        self.project_cl_combo_box.setObjectName(u"project_cl_combo_box")
        sizePolicy4 = QSizePolicy(QSizePolicy.Preferred, QSizePolicy.Fixed)
        sizePolicy4.setHorizontalStretch(0)
        sizePolicy4.setVerticalStretch(0)
        sizePolicy4.setHeightForWidth(self.project_cl_combo_box.sizePolicy().hasHeightForWidth())
        self.project_cl_combo_box.setSizePolicy(sizePolicy4)
        self.project_cl_combo_box.setMinimumSize(QSize(80, 0))
        self.project_cl_combo_box.setMaximumSize(QSize(16777215, 16777215))
        font5 = QFont()
        font5.setFamilies([u"Roboto"])
        self.project_cl_combo_box.setFont(font5)
        self.project_cl_combo_box.setLayoutDirection(Qt.LeftToRight)
        self.project_cl_combo_box.setEditable(True)
        self.project_cl_combo_box.setMaxVisibleItems(11)

        self.horizontalLayout_9.addWidget(self.project_cl_combo_box)

        self.refresh_project_cl_button = ControlQPushButton(self.device_widget)
        self.refresh_project_cl_button.setObjectName(u"refresh_project_cl_button")
        sizePolicy4.setHeightForWidth(self.refresh_project_cl_button.sizePolicy().hasHeightForWidth())
        self.refresh_project_cl_button.setSizePolicy(sizePolicy4)
        self.refresh_project_cl_button.setMaximumSize(QSize(22, 22))
        self.refresh_project_cl_button.setIcon(icon3)
        self.refresh_project_cl_button.setIconSize(QSize(18, 18))
        self.refresh_project_cl_button.setProperty("frameless", True)

        self.horizontalLayout_9.addWidget(self.refresh_project_cl_button)


        self.verticalLayout_6.addLayout(self.horizontalLayout_9)


        self.horizontalLayout_3.addLayout(self.verticalLayout_6)

        self.sync_all_button = ControlQPushButton(self.device_widget)
        self.sync_all_button.setObjectName(u"sync_all_button")
        sizePolicy4.setHeightForWidth(self.sync_all_button.sizePolicy().hasHeightForWidth())
        self.sync_all_button.setSizePolicy(sizePolicy4)
        self.sync_all_button.setMinimumSize(QSize(25, 30))
        icon4 = QIcon()
        icon4.addFile(u":/icons/images/icon_sync.png", QSize(), QIcon.Normal, QIcon.On)
        icon4.addFile(u":/icons/images/icon_sync_disabled.png", QSize(), QIcon.Disabled, QIcon.On)
        icon4.addFile(u":/icons/images/icon_sync_hover.png", QSize(), QIcon.Active, QIcon.On)
        self.sync_all_button.setIcon(icon4)
        self.sync_all_button.setIconSize(QSize(21, 21))
        self.sync_all_button.setProperty("frameless", True)

        self.horizontalLayout_3.addWidget(self.sync_all_button)

        self.build_all_button = ControlQPushButton(self.device_widget)
        self.build_all_button.setObjectName(u"build_all_button")
        sizePolicy4.setHeightForWidth(self.build_all_button.sizePolicy().hasHeightForWidth())
        self.build_all_button.setSizePolicy(sizePolicy4)
        self.build_all_button.setMinimumSize(QSize(25, 30))
        icon5 = QIcon()
        icon5.addFile(u":/icons/images/icon_build.png", QSize(), QIcon.Normal, QIcon.On)
        icon5.addFile(u":/icons/images/icon_build_hover.png", QSize(), QIcon.Active, QIcon.On)
        self.build_all_button.setIcon(icon5)
        self.build_all_button.setIconSize(QSize(21, 21))
        self.build_all_button.setProperty("frameless", True)

        self.horizontalLayout_3.addWidget(self.build_all_button)

        self.sync_and_build_all_button = ControlQPushButton(self.device_widget)
        self.sync_and_build_all_button.setObjectName(u"sync_and_build_all_button")
        sizePolicy4.setHeightForWidth(self.sync_and_build_all_button.sizePolicy().hasHeightForWidth())
        self.sync_and_build_all_button.setSizePolicy(sizePolicy4)
        self.sync_and_build_all_button.setMinimumSize(QSize(25, 30))
        icon6 = QIcon()
        icon6.addFile(u":/icons/images/icon_sync_and_build.png", QSize(), QIcon.Normal, QIcon.On)
        icon6.addFile(u":/icons/images/icon_sync_and_build_disabled.png", QSize(), QIcon.Disabled, QIcon.On)
        icon6.addFile(u":/icons/images/icon_sync_and_build_hover.png", QSize(), QIcon.Active, QIcon.On)
        self.sync_and_build_all_button.setIcon(icon6)
        self.sync_and_build_all_button.setIconSize(QSize(21, 21))
        self.sync_and_build_all_button.setProperty("frameless", True)

        self.horizontalLayout_3.addWidget(self.sync_and_build_all_button)

        self.horizontalSpacer = QSpacerItem(10, 20, QSizePolicy.Expanding, QSizePolicy.Minimum)

        self.horizontalLayout_3.addItem(self.horizontalSpacer)

        self.device_settings_button = ControlQPushButton(self.device_widget)
        self.device_settings_button.setObjectName(u"device_settings_button")
        self.device_settings_button.setEnabled(False)
        sizePolicy4.setHeightForWidth(self.device_settings_button.sizePolicy().hasHeightForWidth())
        self.device_settings_button.setSizePolicy(sizePolicy4)
        self.device_settings_button.setMinimumSize(QSize(25, 30))
        icon7 = QIcon()
        icon7.addFile(u":/icons/images/icon_settings_disabled.png", QSize(), QIcon.Normal, QIcon.Off)
        icon7.addFile(u":/icons/images/icon_settings.png", QSize(), QIcon.Normal, QIcon.On)
        icon7.addFile(u":/icons/images/icon_settings_hover.png", QSize(), QIcon.Active, QIcon.On)
        self.device_settings_button.setIcon(icon7)
        self.device_settings_button.setIconSize(QSize(21, 21))
        self.device_settings_button.setCheckable(True)
        self.device_settings_button.setProperty("frameless", True)

        self.horizontalLayout_3.addWidget(self.device_settings_button)

        self.launch_all_button = ControlQPushButton(self.device_widget)
        self.launch_all_button.setObjectName(u"launch_all_button")
        self.launch_all_button.setEnabled(False)
        sizePolicy4.setHeightForWidth(self.launch_all_button.sizePolicy().hasHeightForWidth())
        self.launch_all_button.setSizePolicy(sizePolicy4)
        self.launch_all_button.setMinimumSize(QSize(25, 30))
        icon8 = QIcon()
        icon8.addFile(u":/icons/images/icon_open.png", QSize(), QIcon.Normal, QIcon.Off)
        icon8.addFile(u":/icons/images/icon_close.png", QSize(), QIcon.Normal, QIcon.On)
        icon8.addFile(u":/icons/images/icon_open_disabled.png", QSize(), QIcon.Disabled, QIcon.Off)
        icon8.addFile(u":/icons/images/icon_open_hover.png", QSize(), QIcon.Active, QIcon.Off)
        icon8.addFile(u":/icons/images/icon_close_hover.png", QSize(), QIcon.Active, QIcon.On)
        self.launch_all_button.setIcon(icon8)
        self.launch_all_button.setIconSize(QSize(21, 21))
        self.launch_all_button.setCheckable(True)
        self.launch_all_button.setProperty("frameless", True)

        self.horizontalLayout_3.addWidget(self.launch_all_button)

        self.connect_all_button = ControlQPushButton(self.device_widget)
        self.connect_all_button.setObjectName(u"connect_all_button")
        sizePolicy4.setHeightForWidth(self.connect_all_button.sizePolicy().hasHeightForWidth())
        self.connect_all_button.setSizePolicy(sizePolicy4)
        self.connect_all_button.setMinimumSize(QSize(25, 30))
        icon9 = QIcon()
        icon9.addFile(u":/icons/images/icon_connect.png", QSize(), QIcon.Normal, QIcon.Off)
        icon9.addFile(u":/icons/images/icon_connected.png", QSize(), QIcon.Normal, QIcon.On)
        icon9.addFile(u":/icons/images/icon_connect_hover.png", QSize(), QIcon.Active, QIcon.Off)
        icon9.addFile(u":/icons/images/icon_connected_hover.png", QSize(), QIcon.Active, QIcon.On)
        self.connect_all_button.setIcon(icon9)
        self.connect_all_button.setIconSize(QSize(21, 21))
        self.connect_all_button.setCheckable(True)
        self.connect_all_button.setProperty("frameless", True)

        self.horizontalLayout_3.addWidget(self.connect_all_button)


        self.verticalLayout.addLayout(self.horizontalLayout_3)

        self.device_list_widget = DeviceListWidget(self.device_widget)
        self.device_list_widget.setObjectName(u"device_list_widget")
        sizePolicy5 = QSizePolicy(QSizePolicy.Expanding, QSizePolicy.Expanding)
        sizePolicy5.setHorizontalStretch(0)
        sizePolicy5.setVerticalStretch(0)
        sizePolicy5.setHeightForWidth(self.device_list_widget.sizePolicy().hasHeightForWidth())
        self.device_list_widget.setSizePolicy(sizePolicy5)
        self.device_list_widget.setMinimumSize(QSize(0, 0))
        self.device_list_widget.setVerticalScrollBarPolicy(Qt.ScrollBarAsNeeded)
        self.device_list_widget.setHorizontalScrollBarPolicy(Qt.ScrollBarAlwaysOff)

        self.verticalLayout.addWidget(self.device_list_widget)

        self.line_3 = QFrame(self.device_widget)
        self.line_3.setObjectName(u"line_3")
        self.line_3.setFrameShape(QFrame.HLine)
        self.line_3.setFrameShadow(QFrame.Sunken)

        self.verticalLayout.addWidget(self.line_3)

        self.muserver_label = QLabel(self.device_widget)
        self.muserver_label.setObjectName(u"muserver_label")

        self.verticalLayout.addWidget(self.muserver_label)

        self.verticalLayout_8 = QVBoxLayout()
        self.verticalLayout_8.setObjectName(u"verticalLayout_8")
        self.verticalLayout_8.setContentsMargins(2, 2, 2, 2)
        self.horizontalLayout_5 = QHBoxLayout()
        self.horizontalLayout_5.setObjectName(u"horizontalLayout_5")
        self.horizontalLayout_5.setContentsMargins(0, 0, 0, 0)
        self.muserver_start_stop_button = ControlQPushButton(self.device_widget)
        self.muserver_start_stop_button.setObjectName(u"muserver_start_stop_button")
        self.muserver_start_stop_button.setEnabled(True)
        self.muserver_start_stop_button.setIcon(icon8)
        self.muserver_start_stop_button.setCheckable(True)
        self.muserver_start_stop_button.setProperty("frameless", True)

        self.horizontalLayout_5.addWidget(self.muserver_start_stop_button)

        self.use_device_autojoin_setting_checkbox = QCheckBox(self.device_widget)
        self.use_device_autojoin_setting_checkbox.setObjectName(u"use_device_autojoin_setting_checkbox")
        self.use_device_autojoin_setting_checkbox.setLayoutDirection(Qt.RightToLeft)
        self.use_device_autojoin_setting_checkbox.setChecked(True)

        self.horizontalLayout_5.addWidget(self.use_device_autojoin_setting_checkbox)

        self.line_4 = QFrame(self.device_widget)
        self.line_4.setObjectName(u"line_4")
        self.line_4.setFrameShape(QFrame.VLine)
        self.line_4.setFrameShadow(QFrame.Sunken)

        self.horizontalLayout_5.addWidget(self.line_4)

        self.label = QLabel(self.device_widget)
        self.label.setObjectName(u"label")
        font6 = QFont()
        font6.setBold(True)
        self.label.setFont(font6)

        self.horizontalLayout_5.addWidget(self.label)

        self.multiuser_session_lineEdit = FramelessQLineEdit(self.device_widget)
        self.multiuser_session_lineEdit.setObjectName(u"multiuser_session_lineEdit")
        sizePolicy1.setHeightForWidth(self.multiuser_session_lineEdit.sizePolicy().hasHeightForWidth())
        self.multiuser_session_lineEdit.setSizePolicy(sizePolicy1)
        self.multiuser_session_lineEdit.setMinimumSize(QSize(0, 0))
        self.multiuser_session_lineEdit.setMaximumSize(QSize(16777215, 16777215))
        font7 = QFont()
        font7.setPointSize(8)
        self.multiuser_session_lineEdit.setFont(font7)
        self.multiuser_session_lineEdit.setProperty("recording", True)

        self.horizontalLayout_5.addWidget(self.multiuser_session_lineEdit)

        self.multiuser_session_inc_button = QPushButton(self.device_widget)
        self.multiuser_session_inc_button.setObjectName(u"multiuser_session_inc_button")
        self.multiuser_session_inc_button.setEnabled(True)
        sizePolicy6 = QSizePolicy(QSizePolicy.Minimum, QSizePolicy.Preferred)
        sizePolicy6.setHorizontalStretch(0)
        sizePolicy6.setVerticalStretch(0)
        sizePolicy6.setHeightForWidth(self.multiuser_session_inc_button.sizePolicy().hasHeightForWidth())
        self.multiuser_session_inc_button.setSizePolicy(sizePolicy6)
        icon10 = QIcon()
        icon10.addFile(u":/icons/images/icon_add_disabled.png", QSize(), QIcon.Normal, QIcon.Off)
        icon10.addFile(u":/icons/images/icon_add.png", QSize(), QIcon.Normal, QIcon.On)
        self.multiuser_session_inc_button.setIcon(icon10)
        self.multiuser_session_inc_button.setProperty("frameless", True)

        self.horizontalLayout_5.addWidget(self.multiuser_session_inc_button)


        self.verticalLayout_8.addLayout(self.horizontalLayout_5)


        self.verticalLayout.addLayout(self.verticalLayout_8)

        self.horizontalLayout_2 = QHBoxLayout()
        self.horizontalLayout_2.setSpacing(3)
        self.horizontalLayout_2.setObjectName(u"horizontalLayout_2")

        self.verticalLayout.addLayout(self.horizontalLayout_2)


        self.horizontalLayout_4.addWidget(self.device_widget)

        self.frame_right = QFrame(self.centralwidget)
        self.frame_right.setObjectName(u"frame_right")
        self.frame_right.setMinimumSize(QSize(100, 0))
        self.frame_right.setFrameShape(QFrame.StyledPanel)
        self.frame_right.setFrameShadow(QFrame.Raised)
        self.verticalLayout_2 = QVBoxLayout(self.frame_right)
        self.verticalLayout_2.setSpacing(6)
        self.verticalLayout_2.setObjectName(u"verticalLayout_2")
        self.verticalLayout_2.setContentsMargins(0, 0, 0, 0)
        self.splitter = QSplitter(self.frame_right)
        self.splitter.setObjectName(u"splitter")
        self.splitter.setOrientation(Qt.Vertical)
        self.tabs_main = QTabWidget(self.splitter)
        self.tabs_main.setObjectName(u"tabs_main")
        self.splitter.addWidget(self.tabs_main)
        self.frame_logger = QFrame(self.splitter)
        self.frame_logger.setObjectName(u"frame_logger")
        self.frame_logger.setMinimumSize(QSize(0, 100))
        self.frame_logger.setFrameShape(QFrame.StyledPanel)
        self.frame_logger.setFrameShadow(QFrame.Raised)
        self.verticalLayout_4 = QVBoxLayout(self.frame_logger)
        self.verticalLayout_4.setSpacing(4)
        self.verticalLayout_4.setObjectName(u"verticalLayout_4")
        self.verticalLayout_4.setContentsMargins(0, 0, 0, 0)
        self.logger_options = QHBoxLayout()
        self.logger_options.setObjectName(u"logger_options")
        self.logger_level_comboBox = QComboBox(self.frame_logger)
        self.logger_level_comboBox.addItem("")
        self.logger_level_comboBox.addItem("")
        self.logger_level_comboBox.addItem("")
        self.logger_level_comboBox.addItem("")
        self.logger_level_comboBox.setObjectName(u"logger_level_comboBox")
        sizePolicy7 = QSizePolicy(QSizePolicy.Preferred, QSizePolicy.Fixed)
        sizePolicy7.setHorizontalStretch(1)
        sizePolicy7.setVerticalStretch(0)
        sizePolicy7.setHeightForWidth(self.logger_level_comboBox.sizePolicy().hasHeightForWidth())
        self.logger_level_comboBox.setSizePolicy(sizePolicy7)

        self.logger_options.addWidget(self.logger_level_comboBox)

        self.logger_options_spacer_0 = QSpacerItem(40, 20, QSizePolicy.Expanding, QSizePolicy.Minimum)

        self.logger_options.addItem(self.logger_options_spacer_0)

        self.logger_wrap_checkbox = QCheckBox(self.frame_logger)
        self.logger_wrap_checkbox.setObjectName(u"logger_wrap_checkbox")

        self.logger_options.addWidget(self.logger_wrap_checkbox)

        self.logger_wrap_label = QLabel(self.frame_logger)
        self.logger_wrap_label.setObjectName(u"logger_wrap_label")

        self.logger_options.addWidget(self.logger_wrap_label)

        self.logger_options_spacer_1 = QSpacerItem(20, 20, QSizePolicy.Expanding, QSizePolicy.Minimum)

        self.logger_options.addItem(self.logger_options_spacer_1)

        self.logger_autoscroll_checkbox = QCheckBox(self.frame_logger)
        self.logger_autoscroll_checkbox.setObjectName(u"logger_autoscroll_checkbox")

        self.logger_options.addWidget(self.logger_autoscroll_checkbox)

        self.logger_autoscroll_label = QLabel(self.frame_logger)
        self.logger_autoscroll_label.setObjectName(u"logger_autoscroll_label")

        self.logger_options.addWidget(self.logger_autoscroll_label)

        self.logger_options_spacer_2 = QSpacerItem(20, 0, QSizePolicy.Expanding, QSizePolicy.Minimum)

        self.logger_options.addItem(self.logger_options_spacer_2)

        self.clear_log_button = QPushButton(self.frame_logger)
        self.clear_log_button.setObjectName(u"clear_log_button")

        self.logger_options.addWidget(self.clear_log_button)


        self.verticalLayout_4.addLayout(self.logger_options)

        self.base_console = QPlainTextEdit(self.frame_logger)
        self.base_console.setObjectName(u"base_console")
        palette = QPalette()
        brush = QBrush(QColor(177, 177, 177, 255))
        brush.setStyle(Qt.SolidPattern)
        palette.setBrush(QPalette.Active, QPalette.WindowText, brush)
        brush1 = QBrush(QColor(36, 36, 36, 255))
        brush1.setStyle(Qt.SolidPattern)
        palette.setBrush(QPalette.Active, QPalette.Button, brush1)
        palette.setBrush(QPalette.Active, QPalette.Text, brush)
        palette.setBrush(QPalette.Active, QPalette.ButtonText, brush)
        palette.setBrush(QPalette.Active, QPalette.Base, brush1)
        palette.setBrush(QPalette.Active, QPalette.Window, brush1)
        gradient = QLinearGradient(0, 0, 0, 1)
        gradient.setSpread(QGradient.PadSpread)
        gradient.setCoordinateMode(QGradient.ObjectBoundingMode)
        gradient.setColorAt(0, QColor(255, 160, 47, 255))
        gradient.setColorAt(1, QColor(215, 128, 26, 255))
        brush2 = QBrush(gradient)
        palette.setBrush(QPalette.Active, QPalette.Highlight, brush2)
        palette.setBrush(QPalette.Inactive, QPalette.WindowText, brush)
        palette.setBrush(QPalette.Inactive, QPalette.Button, brush1)
        palette.setBrush(QPalette.Inactive, QPalette.Text, brush)
        palette.setBrush(QPalette.Inactive, QPalette.ButtonText, brush)
        palette.setBrush(QPalette.Inactive, QPalette.Base, brush1)
        palette.setBrush(QPalette.Inactive, QPalette.Window, brush1)
        gradient1 = QLinearGradient(0, 0, 0, 1)
        gradient1.setSpread(QGradient.PadSpread)
        gradient1.setCoordinateMode(QGradient.ObjectBoundingMode)
        gradient1.setColorAt(0, QColor(255, 160, 47, 255))
        gradient1.setColorAt(1, QColor(215, 128, 26, 255))
        brush3 = QBrush(gradient1)
        palette.setBrush(QPalette.Inactive, QPalette.Highlight, brush3)
        brush4 = QBrush(QColor(64, 64, 64, 255))
        brush4.setStyle(Qt.SolidPattern)
        palette.setBrush(QPalette.Disabled, QPalette.WindowText, brush4)
        brush5 = QBrush(QColor(50, 50, 50, 255))
        brush5.setStyle(Qt.SolidPattern)
        palette.setBrush(QPalette.Disabled, QPalette.Button, brush5)
        palette.setBrush(QPalette.Disabled, QPalette.Text, brush4)
        palette.setBrush(QPalette.Disabled, QPalette.ButtonText, brush4)
        palette.setBrush(QPalette.Disabled, QPalette.Base, brush5)
        palette.setBrush(QPalette.Disabled, QPalette.Window, brush5)
        gradient2 = QLinearGradient(0, 0, 0, 1)
        gradient2.setSpread(QGradient.PadSpread)
        gradient2.setCoordinateMode(QGradient.ObjectBoundingMode)
        gradient2.setColorAt(0, QColor(255, 160, 47, 255))
        gradient2.setColorAt(1, QColor(215, 128, 26, 255))
        brush6 = QBrush(gradient2)
        palette.setBrush(QPalette.Disabled, QPalette.Highlight, brush6)
        self.base_console.setPalette(palette)
        font8 = QFont()
        font8.setFamilies([u"DroidSansMono"])
        self.base_console.setFont(font8)
        self.base_console.setUndoRedoEnabled(False)
        self.base_console.setLineWrapMode(QPlainTextEdit.NoWrap)
        self.base_console.setReadOnly(True)
        self.base_console.setCenterOnScroll(False)

        self.verticalLayout_4.addWidget(self.base_console)

        self.horizontalLayout_6 = QHBoxLayout()
        self.horizontalLayout_6.setObjectName(u"horizontalLayout_6")
        self.horizontalSpacer_3 = QSpacerItem(40, 20, QSizePolicy.Expanding, QSizePolicy.Minimum)

        self.horizontalLayout_6.addItem(self.horizontalSpacer_3)

        self.recording_status = QLabel(self.frame_logger)
        self.recording_status.setObjectName(u"recording_status")

        self.horizontalLayout_6.addWidget(self.recording_status)

        self.line_6 = QFrame(self.frame_logger)
        self.line_6.setObjectName(u"line_6")
        self.line_6.setFrameShape(QFrame.VLine)
        self.line_6.setFrameShadow(QFrame.Sunken)

        self.horizontalLayout_6.addWidget(self.line_6)

        self.current_address_value = QLineEdit(self.frame_logger)
        self.current_address_value.setObjectName(u"current_address_value")
        sizePolicy4.setHeightForWidth(self.current_address_value.sizePolicy().hasHeightForWidth())
        self.current_address_value.setSizePolicy(sizePolicy4)
        self.current_address_value.setStyleSheet(u"border: 0px; background-color: rgba(0, 0, 0, 0)")
        self.current_address_value.setAlignment(Qt.AlignRight|Qt.AlignTrailing|Qt.AlignVCenter)

        self.horizontalLayout_6.addWidget(self.current_address_value)

        self.line_5 = QFrame(self.frame_logger)
        self.line_5.setObjectName(u"line_5")
        self.line_5.setFrameShape(QFrame.VLine)
        self.line_5.setFrameShadow(QFrame.Sunken)

        self.horizontalLayout_6.addWidget(self.line_5)

        self.current_config_file_value = QLabel(self.frame_logger)
        self.current_config_file_value.setObjectName(u"current_config_file_value")

        self.horizontalLayout_6.addWidget(self.current_config_file_value)


        self.verticalLayout_4.addLayout(self.horizontalLayout_6)

        self.splitter.addWidget(self.frame_logger)

        self.verticalLayout_2.addWidget(self.splitter)


        self.horizontalLayout_4.addWidget(self.frame_right)

        self.line_2 = QFrame(self.centralwidget)
        self.line_2.setObjectName(u"line_2")
        self.line_2.setFrameShape(QFrame.VLine)
        self.line_2.setFrameShadow(QFrame.Sunken)

        self.horizontalLayout_4.addWidget(self.line_2)


        self.verticalLayout_3.addLayout(self.horizontalLayout_4)

        switchboard_window.setCentralWidget(self.centralwidget)
        self.line.raise_()
        self.statusbar = QStatusBar(switchboard_window)
        self.statusbar.setObjectName(u"statusbar")
        switchboard_window.setStatusBar(self.statusbar)
        self.menu_bar = QMenuBar(switchboard_window)
        self.menu_bar.setObjectName(u"menu_bar")
        self.menu_bar.setGeometry(QRect(0, 0, 1192, 21))
        self.menu_configs = QMenu(self.menu_bar)
        self.menu_configs.setObjectName(u"menu_configs")
        self.menuConfigs = QMenu(self.menu_bar)
        self.menuConfigs.setObjectName(u"menuConfigs")
        self.menu_load_config = QMenu(self.menuConfigs)
        self.menu_load_config.setObjectName(u"menu_load_config")
        switchboard_window.setMenuBar(self.menu_bar)

        self.menu_bar.addAction(self.menuConfigs.menuAction())
        self.menu_bar.addAction(self.menu_configs.menuAction())
        self.menu_configs.addAction(self.update_settings)
        self.menuConfigs.addAction(self.menu_new_config)
        self.menuConfigs.addAction(self.menu_save_config_as)
        self.menuConfigs.addAction(self.menu_delete_config)
        self.menuConfigs.addSeparator()
        self.menuConfigs.addAction(self.menu_load_config.menuAction())

        self.retranslateUi(switchboard_window)

        self.tabs_main.setCurrentIndex(-1)
        self.logger_level_comboBox.setCurrentIndex(2)


        QMetaObject.connectSlotsByName(switchboard_window)
    # setupUi

    def retranslateUi(self, switchboard_window):
        switchboard_window.setWindowTitle(QCoreApplication.translate("switchboard_window", u"Switchboard", None))
        self.menu_p4_refresh_cl.setText(QCoreApplication.translate("switchboard_window", u"Refresh CL", None))
        self.actionMinimal_Sync.setText(QCoreApplication.translate("switchboard_window", u"Minimal Sync", None))
        self.update_settings.setText(QCoreApplication.translate("switchboard_window", u"Settings", None))
        self.actionAsd.setText(QCoreApplication.translate("switchboard_window", u"asd", None))
        self.menu_new_config.setText(QCoreApplication.translate("switchboard_window", u"New Config", None))
        self.menu_save_config_as.setText(QCoreApplication.translate("switchboard_window", u"Save Config As...", None))
        self.menu_delete_config.setText(QCoreApplication.translate("switchboard_window", u"Delete Current Config", None))
        self.actionTest.setText(QCoreApplication.translate("switchboard_window", u"Test", None))
        self.label_4.setText(QCoreApplication.translate("switchboard_window", u"Sequence", None))
        self.sequence_line_edit.setText(QCoreApplication.translate("switchboard_window", u"Default", None))
        self.slate_label.setText(QCoreApplication.translate("switchboard_window", u"Slate", None))
        self.slate_line_edit.setText(QCoreApplication.translate("switchboard_window", u"SLATE", None))
        self.slate_line_edit.setPlaceholderText("")
        self.take_label.setText(QCoreApplication.translate("switchboard_window", u"Take", None))
        self.record_button.setText("")
        self.settings_button.setText("")
        self.device_add_tool_button.setText(QCoreApplication.translate("switchboard_window", u"Add Device", None))
        self.label_2.setText(QCoreApplication.translate("switchboard_window", u"Level: ", None))
        self.engine_cl_label.setText(QCoreApplication.translate("switchboard_window", u"Engine CL", None))
        self.engine_cl_combo_box.setCurrentText(QCoreApplication.translate("switchboard_window", u"-- None --", None))
        self.refresh_engine_cl_button.setText("")
        self.project_cl_label.setText(QCoreApplication.translate("switchboard_window", u"Project CL", None))
        self.project_cl_combo_box.setItemText(0, QCoreApplication.translate("switchboard_window", u"5435842", None))

        self.project_cl_combo_box.setCurrentText(QCoreApplication.translate("switchboard_window", u"5435842", None))
#if QT_CONFIG(tooltip)
        self.refresh_project_cl_button.setToolTip(QCoreApplication.translate("switchboard_window", u"Refresh Changelists", None))
#endif // QT_CONFIG(tooltip)
        self.refresh_project_cl_button.setText("")
#if QT_CONFIG(tooltip)
        self.sync_all_button.setToolTip(QCoreApplication.translate("switchboard_window", u"Sync all devices", None))
#endif // QT_CONFIG(tooltip)
        self.sync_all_button.setText("")
#if QT_CONFIG(tooltip)
        self.build_all_button.setToolTip(QCoreApplication.translate("switchboard_window", u"Build on all devices", None))
#endif // QT_CONFIG(tooltip)
        self.build_all_button.setText("")
#if QT_CONFIG(tooltip)
        self.sync_and_build_all_button.setToolTip(QCoreApplication.translate("switchboard_window", u"Sync all devices and immediately build after completion", None))
#endif // QT_CONFIG(tooltip)
        self.sync_and_build_all_button.setText("")
#if QT_CONFIG(tooltip)
        self.device_settings_button.setToolTip(QCoreApplication.translate("switchboard_window", u"Configure Unreal Insights capture", None))
#endif // QT_CONFIG(tooltip)
        self.device_settings_button.setText("")
#if QT_CONFIG(tooltip)
        self.launch_all_button.setToolTip(QCoreApplication.translate("switchboard_window", u"Start all connected devices", None))
#endif // QT_CONFIG(tooltip)
        self.launch_all_button.setText("")
#if QT_CONFIG(tooltip)
        self.connect_all_button.setToolTip(QCoreApplication.translate("switchboard_window", u"Connect all devices", None))
#endif // QT_CONFIG(tooltip)
        self.connect_all_button.setText("")
        self.muserver_label.setText(QCoreApplication.translate("switchboard_window", u"Multi-user Server", None))
#if QT_CONFIG(tooltip)
        self.muserver_start_stop_button.setToolTip(QCoreApplication.translate("switchboard_window", u"Start / stop Unreal Multi-user Server", None))
#endif // QT_CONFIG(tooltip)
#if QT_CONFIG(tooltip)
        self.use_device_autojoin_setting_checkbox.setToolTip(QCoreApplication.translate("switchboard_window", u"If this setting is checked then devices will join a Multi-user session based on their individual auto-join setting.", None))
#endif // QT_CONFIG(tooltip)
        self.use_device_autojoin_setting_checkbox.setText(QCoreApplication.translate("switchboard_window", u"Auto-join", None))
        self.label.setText(QCoreApplication.translate("switchboard_window", u"Session:", None))
        self.multiuser_session_lineEdit.setText(QCoreApplication.translate("switchboard_window", u"SessionName", None))
#if QT_CONFIG(tooltip)
        self.multiuser_session_inc_button.setToolTip(QCoreApplication.translate("switchboard_window", u"Increment the multi-user server session name.", None))
#endif // QT_CONFIG(tooltip)
        self.logger_level_comboBox.setItemText(0, QCoreApplication.translate("switchboard_window", u"Message", None))
        self.logger_level_comboBox.setItemText(1, QCoreApplication.translate("switchboard_window", u"OSC", None))
        self.logger_level_comboBox.setItemText(2, QCoreApplication.translate("switchboard_window", u"Debug", None))
        self.logger_level_comboBox.setItemText(3, QCoreApplication.translate("switchboard_window", u"Info", None))

#if QT_CONFIG(tooltip)
        self.logger_wrap_checkbox.setToolTip(QCoreApplication.translate("switchboard_window", u"If checked, long lines are wrapped to fit within view. Otherwise, you may need to scroll horizontally to read long lines.", None))
#endif // QT_CONFIG(tooltip)
#if QT_CONFIG(tooltip)
        self.logger_wrap_label.setToolTip(QCoreApplication.translate("switchboard_window", u"If checked, long lines are wrapped to fit within view. Otherwise, you may need to scroll horizontally to read long lines.", None))
#endif // QT_CONFIG(tooltip)
        self.logger_wrap_label.setText(QCoreApplication.translate("switchboard_window", u"Line Wrap", None))
#if QT_CONFIG(tooltip)
        self.logger_autoscroll_checkbox.setToolTip(QCoreApplication.translate("switchboard_window", u"If checked, the log view will automatically scroll to the end any time new output is appended.", None))
#endif // QT_CONFIG(tooltip)
#if QT_CONFIG(tooltip)
        self.logger_autoscroll_label.setToolTip(QCoreApplication.translate("switchboard_window", u"If checked, the log view will automatically scroll to the end any time new output is appended.", None))
#endif // QT_CONFIG(tooltip)
        self.logger_autoscroll_label.setText(QCoreApplication.translate("switchboard_window", u"Auto Scroll", None))
        self.clear_log_button.setText(QCoreApplication.translate("switchboard_window", u"Clear", None))
#if QT_CONFIG(tooltip)
        self.clear_log_button.setToolTip(QCoreApplication.translate("switchboard_window", u"Clear the log window", None))
#endif // QT_CONFIG(tooltip)
        self.recording_status.setText(QCoreApplication.translate("switchboard_window", u"Recording Status", None))
        self.current_address_value.setText(QCoreApplication.translate("switchboard_window", u"Address", None))
        self.current_config_file_value.setText(QCoreApplication.translate("switchboard_window", u"TextLabel", None))
        self.menu_configs.setTitle(QCoreApplication.translate("switchboard_window", u"Settings", None))
        self.menuConfigs.setTitle(QCoreApplication.translate("switchboard_window", u"Configs", None))
        self.menu_load_config.setTitle(QCoreApplication.translate("switchboard_window", u"Load Config", None))
    # retranslateUi

