# Copyright Epic Games, Inc. All Rights Reserved.

from PySide6.QtCore import Signal, Qt
from PySide6.QtWidgets import QWidget, QHBoxLayout, QVBoxLayout, QLineEdit, QPushButton, QSizePolicy
from PySide6.QtGui import QFont


def set_qt_property(widget, prop, value):
    ''' Sets Qt property and refreshes the styles '''
    widget.setProperty(prop, value)
    widget.style().unpolish(widget)
    widget.style().polish(widget)
    widget.update()


class TakeSpinBox(QWidget):
    ''' Widget that combines a line edit with increment/decrement buttons for specifying the current take.
    This was previously done with a spinbox but it was split so that the completion behavior could be customized.
    The edit box appearance matches the behavior of that of slate.
    '''
    
    # Signals
    valueChanged = Signal(int)
    editingFinished = Signal()
    
    def __init__(self, parent=None):
        super().__init__(parent)
        self._value = 1
        self._minimum = 1
        self._maximum = 999
        
        # Main horizontal layout
        self._layout = QHBoxLayout(self)
        self._layout.setSpacing(0)
        self._layout.setContentsMargins(0, 0, 0, 0)
        
        # Line edit with frameless styling matching the slate's

        line_edit_width = 60  # Fixed width prevents expansion.
        self._line_edit = QLineEdit(self)
        self._line_edit.setAlignment(Qt.AlignCenter)
        self._line_edit.setText(str(self._value))
        self._line_edit.setFixedWidth(line_edit_width)
        self._line_edit.editingFinished.connect(self._on_editing_finished)
        
        set_qt_property(self._line_edit, "frameless", True)
        
        # Event filter for hovering.
        self._line_edit.installEventFilter(self)
        
        # Layout to contain inc/dec buttons.
        self._button_widget = QWidget(self)
        self._button_layout = QVBoxLayout(self._button_widget)
        self._button_layout.setSpacing(0)
        self._button_layout.setContentsMargins(0, 0, 0, 0)
        
        buttons_width = 20

        # Up button
        self._up_button = QPushButton("▲", self._button_widget)
        self._up_button.setFixedSize(buttons_width, 13)
        self._up_button.setFont(QFont("Arial", 7))
        self._up_button.clicked.connect(self._increment)
        
        # Down button
        self._down_button = QPushButton("▼", self._button_widget)
        self._down_button.setFixedSize(buttons_width, 13)
        self._down_button.setFont(QFont("Arial", 7))
        self._down_button.clicked.connect(self._decrement)
        
        # Add buttons to button layout
        self._button_layout.addWidget(self._up_button)
        self._button_layout.addWidget(self._down_button)
        
        # Add to main layout
        self._layout.addWidget(self._line_edit)
        self._layout.addWidget(self._button_widget)
        
        # Fixed size policy to match original spinbox
        self.setSizePolicy(QSizePolicy.Fixed, QSizePolicy.Fixed)
        self.setFixedWidth(line_edit_width + buttons_width)
    
    def value(self):
        return self._value
    
    def setValue(self, value):
        value = max(self._minimum, min(self._maximum, int(value)))
        if value != self._value:
            self._value = value
            self._line_edit.setText(str(value))
            self.valueChanged.emit(value)
    
    def setMinimum(self, minimum):
        self._minimum = minimum
        
    def setMaximum(self, maximum):
        self._maximum = maximum
        
    def minimum(self):
        return self._minimum
        
    def maximum(self):
        return self._maximum
        
    def setAlignment(self, alignment):
        self._line_edit.setAlignment(alignment)
        
    def setFont(self, font):
        self._line_edit.setFont(font)
        
    def _on_editing_finished(self):
        try:
            value = int(self._line_edit.text())
            value = max(self._minimum, min(self._maximum, value))

            if value != self._value:
                self._value = value
                self._line_edit.setText(str(value))
                self.valueChanged.emit(value)

            self.editingFinished.emit()

        except ValueError:
            # Reset to current value if input is invalid
            self._line_edit.setText(str(self._value))
            self.editingFinished.emit()
    
    def _increment(self):
        new_value = min(self._maximum, self._value + 1)
        if new_value != self._value:
            self.setValue(new_value)
    
    def _decrement(self):
        new_value = max(self._minimum, self._value - 1)
        if new_value != self._value:
            self.setValue(new_value)
    
    def eventFilter(self, source, event):
        ''' Handle hover events for frameless styling and key press events '''
        if source == self._line_edit:
            if event.type() == event.Type.Enter:
                # Mouse enters: show frame
                set_qt_property(self._line_edit, "frameless", False)
                return False
            elif event.type() == event.Type.Leave:
                # Mouse leaves: hide frame (unless focused)
                if not self._line_edit.hasFocus():
                    set_qt_property(self._line_edit, "frameless", True)
                return False
            elif event.type() == event.Type.KeyPress:
                # Enter key releases focus
                if event.key() in (Qt.Key_Return, Qt.Key_Enter):
                    self._line_edit.clearFocus()
                    set_qt_property(self._line_edit, "frameless", True)
                    return False
            elif event.type() == event.Type.FocusOut:
                set_qt_property(self._line_edit, "frameless", True)
                return False
        return super().eventFilter(source, event)