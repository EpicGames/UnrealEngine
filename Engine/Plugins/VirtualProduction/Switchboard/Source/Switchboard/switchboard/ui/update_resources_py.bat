
:: Switchboard will read the resources from ../resources.py and not from resources.qrc.
:: So when adding a new image, copy it to /images and then manually reference it in resources.qrc.
:: Then run this script, which will check out ../resources.py from perforce and update it.
:: It requires pyside2-rcc.exe to be in the path (typically found inside <python>/Scripts).
:: You should also add resources.qrc and the image files to your perforce changelist.

p4 edit ../resources.py
pyside2-rcc.exe resources.qrc -o ../resources.py