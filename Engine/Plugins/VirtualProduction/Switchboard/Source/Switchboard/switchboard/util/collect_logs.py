# Copyright Epic Games, Inc. All Rights Reserved.

import contextlib
import os
import pathlib
import sqlite3
import typing
from datetime import datetime
from zipfile import ZipFile

from PySide6 import QtCore, QtGui, QtWidgets

from switchboard.config import Config
from switchboard.devices.device_base import DeviceStatus
from switchboard.devices.unreal.plugin_unreal import DeviceUnreal
from ..switchboard_logging import LOGGER, FILE_HANDLER


def open_logs_folder():
    path = DeviceUnreal.get_log_download_dir()
    if path and path.is_dir():
        url = QtCore.QUrl.fromLocalFile(str(path))
        QtGui.QDesktopServices.openUrl(url)


def execute_zip_logs_workflow(config: Config, devices: typing.List[DeviceUnreal]):
    """ Zip up all important log files asking the user about preferences in certain edge cases. """
    def should_skip_because_running_devices():
        running_devices = [device for device in devices if
                           device.status in [DeviceStatus.CONNECTING, DeviceStatus.OPEN, DeviceStatus.CLOSING]]
        if len(running_devices) > 0:
            pretty_devices_str = ""
            for name in list(map(lambda device: device.name, running_devices)):
                pretty_devices_str += f" - {name}\n"
            continue_answer = QtWidgets.QMessageBox.question(
                None,
                "Devices still running",
                f"The following devices are still running:\n{pretty_devices_str}"
                "\nThese devices should be closed to have their latest logs included.\nDo you want to continue anyway?",
                QtWidgets.QMessageBox.Yes | QtWidgets.QMessageBox.No
            )

            return True if continue_answer == QtWidgets.QMessageBox.No else False
        return False

    def ask_log_zip_name():
        modtime = datetime.now()
        default_name = f"Logs-{modtime.strftime('%Y.%m.%d-%H.%M.%S')}"
        result = QtWidgets.QInputDialog.getText(
            None,
            "Name .zip file",
            "New .zip file name:",
            QtWidgets.QLineEdit.Normal,
            default_name
        )

        if not result[1]:
            return None

        file_path = os.path.join(DeviceUnreal.get_log_download_dir(), result[0])
        if not file_path.endswith(".zip"):
            file_path += ".zip"
        return file_path

    if should_skip_because_running_devices():
        return False

    zip_destination = ask_log_zip_name()
    if zip_destination is not None:
        zip_logs(zip_destination=zip_destination, config=config, devices=devices)
        return True

    return False


def zip_logs(zip_destination: str, config: Config, devices: typing.List[DeviceUnreal]):
    """ Collects all relevant logs and saves them.
    """
    def process_log_file(abs_file_path: str, zip_path_name: str, zip_file: ZipFile):
        # Access to .db files must be synchronized - these files are primarily used by multiuser
        if abs_file_path.endswith(".db"):
            copy_db_file(abs_file_path, zip_path_name, zip_file)
        # Skip temporary files created by backup in copy_db_file
        elif abs_file_path.endswith(".db-shm") or abs_file_path.endswith(".db-wal"):
            return
        else:
            zip_file.write(abs_file_path, zip_path_name)

    def copy_db_file(abs_file_path: str, zip_path_name: str, zip_file: ZipFile):
        copy_path = os.path.join(os.path.dirname(zip_destination), os.path.basename(zip_path_name))
        # Track the copy path for cleanup before opening connections so a partial
        # failure (e.g. backup error) still removes any file sqlite3 created.
        # Cannot be removed before zip_file is closed.
        files_to_remove.append(copy_path)
        try:
            with contextlib.closing(sqlite3.connect(abs_file_path)) as original, \
                    contextlib.closing(sqlite3.connect(copy_path)) as copy:
                original.backup(copy)
            zip_file.write(copy_path, zip_path_name)
        except sqlite3.Error:
            LOGGER.error(f"Failed to access database file {abs_file_path}")
        except (OSError, IOError):
            LOGGER.error(f"Failed to create temporary file {copy_path} while backing up original {abs_file_path}")

    files_to_remove = []
    with ZipFile(zip_destination, "w") as zip_file:
        iterate_log_files(
            config,
            devices,
            lambda abs_file_path, zip_path_name, zip_file=zip_file: process_log_file(abs_file_path, zip_path_name, zip_file)
        )

    for file_to_remove in files_to_remove:
        try:
            os.remove(file_to_remove)
        except FileNotFoundError:
            # Backup never created (e.g. sqlite3.connect failed); nothing to clean up.
            pass
        except OSError as e:
            # Defensive: any lingering OS lock (Windows WinError 32) or permission issue.
            LOGGER.warning(f"Failed to remove temporary file {file_to_remove}: {e}")


def iterate_log_files(config: Config, devices: typing.List[DeviceUnreal], consumer: typing.Callable[[str, str], None]):
    """
        Passes the abolute files paths of all important log files:
        - all latest .log files in Switchboard log folder
        - all latest .utrace files in Switchboard log folder
        - all multiuser session files in Programs/UnrealMultiUserServerIntermediateMultiUser
        - multiuser log file ProgramsUnrealMultiUserServerSavedLogsUnrealMultiUserServer.log

        The arguments of the consumer:
            1 The absolute file path of the file
            2 A short unique relative file path (includes file name), useful for zipping
    """
    def iterate_file_list(files: typing.Iterable[str]):
        for file_path in files:
            if os.path.isfile(file_path):
                consumer(file_path, os.path.join("Devices", os.path.basename(file_path))) 

    def iterate_mu_session_files():
        for current_dir, _, files in os.walk(config.multiuser_server_session_directory_path()):
            for file in files:
                abs_path = os.path.join(current_dir, file)
                root_mu_path = pathlib.Path(config.multiuser_server_session_directory_path())
                relative_path = pathlib.Path(abs_path).relative_to(root_mu_path)
                consumer(abs_path, os.path.join("MultiUserServer", relative_path))

    latest_log_file_names = map(lambda device: device.last_log_path.get_value(), devices)
    iterate_file_list(latest_log_file_names)

    latest_trace_paths = map(lambda device: device.last_trace_path.get_value(), devices)
    iterate_file_list(latest_trace_paths)

    iterate_mu_session_files()
    consumer(config.multiuser_server_log_path(), os.path.join("MultiUserServer", "UnrealMultiUserServer.log"))

    consumer(FILE_HANDLER.baseFilename, "Switchboard.log")
