To run these tests:
    > cd ...\Engine\Plugins\VirtualProduction\Switchboard\Source\Switchboard
    > ..\..\..\..\..\Extras\ThirdPartyNotUE\SwitchboardThirdParty\Python\Scripts\activate
    (switchboard_venv) > python -m unittest discover -v -s tests\
