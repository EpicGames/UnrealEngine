# Copyright Epic Games, Inc. All Rights Reserved.

import sys
from typing import Union
import unittest
from unittest.mock import patch

from switchboard import switchboard_application


class CommandLineExtractorTestCase(unittest.TestCase):
    def _extractor(self, args: Union[str, list[str]]):
        return switchboard_application.CommandLineExtractor(args)

    def _tokens(self, args: Union[str, list[str]]):
        return self._extractor(args).tokens

    @property
    def _mu_path(self):
        if sys.platform.startswith('win'):
            return (R'C:\Program Files\Epic Games\UE_5.2'
                    R'\Engine\Binaries\Win64\UnrealMultiUserSlateServer.EXE')
        else:
            return ('/home/epic/UE 5.2/'
                    'Engine/Binaries/Linux/UnrealMultiUserSlateServer')

    def _test_multiuser(self):
        ''' Inner test, called with varying `sys.platform` mock patches. '''

        self.assertEqual(
            self._tokens(f'"{self._mu_path}"'),
            [self._mu_path],
            'argv[0] containing spaces and/or backslashes')

        TRANSPORT_UNICAST = '127.0.0.1:9030'
        SERVER_NAME = 'MU_Server-001'

        EXPECTED_TOKENS = [
            self._mu_path, f'-CONCERTSERVER={SERVER_NAME}',
            f'-UDPMESSAGING_TRANSPORT_UNICAST={TRANSPORT_UNICAST}']

        self.assertEqual(
            self._tokens(
                f'"{self._mu_path}"'
                f' -CONCERTSERVER={SERVER_NAME}'
                f' -UDPMESSAGING_TRANSPORT_UNICAST={TRANSPORT_UNICAST}'),
            EXPECTED_TOKENS,
            'argv[0] token should be unquoted')

        self.assertEqual(
            self._tokens(
                f'"{self._mu_path}"'
                f' -CONCERTSERVER="{SERVER_NAME}"'
                f' -UDPMESSAGING_TRANSPORT_UNICAST="{TRANSPORT_UNICAST}"'),
            EXPECTED_TOKENS,
            'argv[1:] quoted value quotes removed')

        self.assertEqual(
            self._tokens(
                f'"{self._mu_path}"'
                f' "-CONCERTSERVER={SERVER_NAME}"'
                f' "-UDPMESSAGING_TRANSPORT_UNICAST={TRANSPORT_UNICAST}"'),
            EXPECTED_TOKENS,
            'argv[1:] enclosing quotes removed')

    def test_tokenization(self):
        self.assertEqual(
            self._tokens('foo'),
            ['foo'],
            'Base case')

        self.assertEqual(
            self._tokens('foo bar baz'),
            ['foo', 'bar', 'baz'],
            'Unquoted spaces split tokens')

        self.assertEqual(
            self._tokens('foo "bar baz"'),
            ['foo', 'bar baz'],
            'Quoted token containing space')

    def test_get_value_for_switch(self):
        extractor = self._extractor(
            '"argv0"'
            ' -foo=bar'
            R' -baz="forty \"two\""')

        self.assertEqual(
            extractor.get_value_for_switch('foo'),
            'bar')

        self.assertEqual(
            extractor.get_value_for_switch('baz'),
            'forty "two"')

    @patch('sys.platform', 'win32')
    def test_multiuser_windows(self):
        with self.subTest("Multi-user (sys.platform == 'win32')"):
            self.assertTrue(self._mu_path.startswith('C:\\'))
            self._test_multiuser()

    @patch('sys.platform', 'linux')
    def test_multiuser_linux(self):
        with self.subTest("Multi-user (sys.platform == 'linux')"):
            self.assertTrue(self._mu_path.startswith('/home/'))
            self._test_multiuser()
