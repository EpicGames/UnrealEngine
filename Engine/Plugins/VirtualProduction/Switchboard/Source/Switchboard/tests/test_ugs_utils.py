# Copyright Epic Games, Inc. All Rights Reserved.

import unittest
import uuid

from switchboard import ugs_utils


# To run these tests:
# > cd ...\Engine\Plugins\VirtualProduction\Switchboard\Source\Switchboard
# > ..\..\..\..\..\Extras\ThirdPartyNotUE\SwitchboardThirdParty\Python\Scripts\activate
# (switchboard_venv) > python -m unittest discover -v -s tests\


class IniParserTestCase(unittest.TestCase):
    def setUp(self):
        self.parser = ugs_utils.IniParser()

    def add_ini_sources(self, sources: dict[str, str]):
        # Takes a dict of `filename -> file contents` and loads them as INIs.
        for source_name, source_contents in sources.items():
            self.parser.read_string(source_contents, source_name)

    def test_basic(self):
        self.add_ini_sources({
            'Basic.ini':
                '[Section]\n'
                'Key=Value\n',
        })

        self.assertEqual(
            self.parser.try_get('Section', 'Key'),
            ['Value']
        )

        self.assertEqual(
            self.parser.try_get('Section', 'NonexistentKey'),
            None
        )

        self.assertEqual(
            self.parser.try_get('NonexistentSection', 'Key'),
            None
        )

    def test_append(self):
        self.add_ini_sources({
            'AppendOne.ini':
                '[Section]\n'
                '+SingleSourceKey=ValueA\n'
                '+SingleSourceKey=ValueB\n'
                'TwoSourceKey=ValueOne',
            'AppendTwo.ini':
                '[Section]\n'
                '+TwoSourceKey=ValueTwo',
        })

        self.assertEqual(
            self.parser.try_get('Section', 'SingleSourceKey'),
            ['ValueA', 'ValueB'],
            'Appending a value within the same file'
        )

        self.assertEqual(
            self.parser.try_get('Section', 'TwoSourceKey'),
            ['ValueOne', 'ValueTwo'],
            'Appending a value from a subsequent file'
        )

    def test_overwrite(self):
        self.add_ini_sources({
            'Original.ini':
                '[Section]\n'
                'Key=OriginalValue\n',
            'Overwrite.ini':
                '[Section]\n'
                'Key=OverwrittenValue\n',
        })

        self.assertEqual(
            self.parser.try_get('Section', 'Key'),
            ['OverwrittenValue'],
            'Overwrite.ini value should replace Original.ini value'
        )


class SyncFiltersTestCase(unittest.TestCase):
    @classmethod
    def setUpClass(cls):
        if not ugs_utils.SyncFilters.supported():
            raise RuntimeError('SyncFilters not available')

    def setUp(self):
        self.sync_filters = ugs_utils.SyncFilters()

    def add_new_category(self, name: str, paths: list[str]) -> uuid.UUID:
        id = uuid.uuid4()
        self.sync_filters.create_category(id, name, paths)
        return id

    def test_exclusions(self):
        DEPOT = [
            '/GenerateProjectFiles.bat',
            '/Engine/Content/Maps/Entry.umap',
            '/Engine/Source/UnrealEditor.Target.cs',
        ]

        for path in DEPOT:
            self.assertTrue(self.sync_filters.includes_path(path))

        content_id = self.add_new_category('Content', ['*.uasset', '*.umap'])
        self.sync_filters.exclude_category(content_id)

        self.assertEqual(list(map(self.sync_filters.includes_path, DEPOT)),
                         [True, False, True],
                         'Should exclude only the umap file')

        source_id = self.add_new_category('Source', ['/Engine/Source/...'])
        self.sync_filters.exclude_category(source_id)

        self.assertEqual(list(map(self.sync_filters.includes_path, DEPOT)),
                         [True, False, False],
                         'Should now exclude the umap and source files')
