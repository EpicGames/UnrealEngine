// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardListenerAutolaunch.h"

#include "Logging/LogMacros.h"
#include "Misc/ScopeExit.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <winreg.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

static const FString WindowsRunRegKeyPath = "Software\\Microsoft\\Windows\\CurrentVersion\\Run";
static const FString ListenerAutolaunchRegValueName = "SwitchboardListener";
#endif


namespace UE::SwitchboardListener::Autolaunch
{
#if SWITCHBOARD_LISTENER_AUTOLAUNCH
	HKEY OpenHkcuRunKey(FLogCategoryBase& InLogCategory)
	{
		HKEY HkcuRunKey = nullptr;
		const LSTATUS OpenResult = RegOpenKeyEx(HKEY_CURRENT_USER, *WindowsRunRegKeyPath, 0, KEY_ALL_ACCESS, &HkcuRunKey);
		if (OpenResult != ERROR_SUCCESS)
		{
			UE_LOG_REF(InLogCategory, Log, TEXT("Error opening registry key %s (%08X)"), *WindowsRunRegKeyPath, OpenResult);
			return nullptr;
		}
		return HkcuRunKey;
	}


	FString GetInvocation(FLogCategoryBase& InLogCategory)
	{
		HKEY HkcuRunKey = OpenHkcuRunKey(InLogCategory);
		if (!HkcuRunKey)
		{
			return FString();
		}

		ON_SCOPE_EXIT
		{
			RegCloseKey(HkcuRunKey);
		};

		DWORD ValueType;
		DWORD ValueSizeBytes = 0;
		const LSTATUS SizeResult = RegQueryValueEx(HkcuRunKey, *ListenerAutolaunchRegValueName, nullptr, &ValueType, nullptr, &ValueSizeBytes);
		if (SizeResult == ERROR_FILE_NOT_FOUND)
		{
			return FString();
		}
		else if (SizeResult != ERROR_SUCCESS)
		{
			UE_LOG_REF(InLogCategory, Log, TEXT("Error reading registry value %s:\"%s\" (%08X)"), *WindowsRunRegKeyPath, *ListenerAutolaunchRegValueName, SizeResult);
			return FString();
		}

		FString Value;
		TArray<TCHAR, FString::AllocatorType>& CharArray = Value.GetCharArray();
		const uint32 ValueLenChars = ValueSizeBytes / sizeof(TCHAR);
		CharArray.SetNumUninitialized(ValueLenChars);

		const LSTATUS QueryResult = RegQueryValueEx(HkcuRunKey, *ListenerAutolaunchRegValueName, nullptr, &ValueType, reinterpret_cast<LPBYTE>(CharArray.GetData()), &ValueSizeBytes);
		if (QueryResult != ERROR_SUCCESS)
		{
			UE_LOG_REF(InLogCategory, Log, TEXT("Error reading registry value %s:\"%s\" (%08X)"), *WindowsRunRegKeyPath, *ListenerAutolaunchRegValueName, QueryResult);
			return FString();
		}
		else if (ValueType != REG_SZ)
		{
			UE_LOG_REF(InLogCategory, Log, TEXT("Registry value %s:\"%s\" has wrong type (%u)"), *WindowsRunRegKeyPath, *ListenerAutolaunchRegValueName, ValueType);
			return FString();
		}

		if (CharArray[CharArray.Num() - 1] != TEXT('\0'))
		{
			CharArray.Add(TEXT('\0'));
		}

		return Value;
	}


	FString GetInvocationExecutable(FLogCategoryBase& InLogCategory)
	{
		FString AutolaunchCommand = GetInvocation(InLogCategory);
		if (AutolaunchCommand.IsEmpty())
		{
			return FString();
		}

		TArray<FString> QuoteParts;
		AutolaunchCommand.ParseIntoArray(QuoteParts, TEXT("\""));
		return QuoteParts[0];
	}


	bool SetInvocation(const FString& InNewInvocation, FLogCategoryBase& InLogCategory)
	{
		HKEY HkcuRunKey = OpenHkcuRunKey(InLogCategory);
		if (!HkcuRunKey)
		{
			return false;
		}

		ON_SCOPE_EXIT
		{
			RegCloseKey(HkcuRunKey);
		};

		const TArray<TCHAR, FString::AllocatorType>& CharArray = InNewInvocation.GetCharArray();
		const LSTATUS SetResult = RegSetValueEx(HkcuRunKey, *ListenerAutolaunchRegValueName, 0, REG_SZ, reinterpret_cast<const BYTE*>(CharArray.GetData()), CharArray.Num() * sizeof(TCHAR));
		if (SetResult != ERROR_SUCCESS)
		{
			UE_LOG_REF(InLogCategory, Error, TEXT("Error setting registry value %s:\"%s\" (%08X)"), *WindowsRunRegKeyPath, *ListenerAutolaunchRegValueName, SetResult);
			return false;
		}

		return true;
	}


	bool RemoveInvocation(FLogCategoryBase& InLogCategory)
	{
		HKEY HkcuRunKey = OpenHkcuRunKey(InLogCategory);
		if (!HkcuRunKey)
		{
			return true;
		}

		ON_SCOPE_EXIT
		{
			RegCloseKey(HkcuRunKey);
		};

		const LSTATUS DeleteResult = RegDeleteValue(HkcuRunKey, *ListenerAutolaunchRegValueName);
		if (DeleteResult != ERROR_SUCCESS)
		{
			UE_LOG_REF(InLogCategory, Error, TEXT("Error deleting registry value %s:\"%s\" (%08X)"), *WindowsRunRegKeyPath, *ListenerAutolaunchRegValueName, DeleteResult);
			return false;
		}

		return true;
	}
#endif // #if SWITCHBOARD_LISTENER_AUTOLAUNCH
} // namespace UE::SwitchboardListener::Autolaunch
