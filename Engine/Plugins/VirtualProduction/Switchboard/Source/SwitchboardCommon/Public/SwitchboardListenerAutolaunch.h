// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"


#if PLATFORM_WINDOWS
#	define SWITCHBOARD_LISTENER_AUTOLAUNCH 1
#else
#	define SWITCHBOARD_LISTENER_AUTOLAUNCH 0
#endif


struct FLogCategoryBase;


namespace UE::SwitchboardListener::Autolaunch
{
#if SWITCHBOARD_LISTENER_AUTOLAUNCH
	SWITCHBOARDCOMMON_API FString GetInvocation(FLogCategoryBase& InLogCategory);
	SWITCHBOARDCOMMON_API FString GetInvocationExecutable(FLogCategoryBase& InLogCategory);
	SWITCHBOARDCOMMON_API bool SetInvocation(const FString& InNewInvocation, FLogCategoryBase& InLogCategory);
	SWITCHBOARDCOMMON_API bool RemoveInvocation(FLogCategoryBase& InLogCategory);
#endif // #if SWITCHBOARD_LISTENER_AUTOLAUNCH
}
