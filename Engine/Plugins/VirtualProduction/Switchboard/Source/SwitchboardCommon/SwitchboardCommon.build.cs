// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SwitchboardCommon : ModuleRules
{
	public SwitchboardCommon(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			});
	}
}
