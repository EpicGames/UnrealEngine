// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterRootActorReferenceDetailCustomization.h"

#include "AssetRegistry/AssetData.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "GameFramework/Actor.h"
#include "IPropertyUtilities.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "SwitchboardEditorModule.h"
#include "SwitchboardTypes.h"


#define LOCTEXT_NAMESPACE "DisplayClusterRootActorReferenceDetailCustomization"



TSharedRef<IPropertyTypeCustomization> FDisplayClusterRootActorReferenceDetailCustomization::MakeInstance()
{
	return MakeShareable(new FDisplayClusterRootActorReferenceDetailCustomization);
}

FDisplayClusterRootActorReference* FDisplayClusterRootActorReferenceDetailCustomization::GetDisplayClusterRootActorReference() const
{
	// Access the underlying structure
	TArray<void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	if (RawData.Num() != 1)
	{
		return nullptr;
	}

	return reinterpret_cast<FDisplayClusterRootActorReference*>(RawData[0]);
}

void FDisplayClusterRootActorReferenceDetailCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Early return if the nDisplay actor class cannot be found. This property will not be displayed.
	if (!FSwitchboardEditorModule::GetDisplayClusterRootActorClass())
	{
		HeaderRow.Visibility(EVisibility::Collapsed);
		return;
	}

	// Keep our property handle so that the widget can access it.
	StructPropertyHandle = InPropertyHandle;

	// Make sure the property is of the expected type
	check(CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty())->Struct == FDisplayClusterRootActorReference::StaticStruct());

	// Access the underlying structure
	TArray<void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	check(RawData.Num() == 1);
	FDisplayClusterRootActorReference* DCRAReference = reinterpret_cast<FDisplayClusterRootActorReference*>(RawData[0]);

	// Customize the widget

	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

	HeaderRow.NameContent()
	[
		// Just the name
		StructPropertyHandle->CreatePropertyNameWidget(LOCTEXT("nDisplayActor", "nDisplay Actor"))
	]
	.ValueContent()
	.MinDesiredWidth(300)
	[
		// nDisplay actor picker

		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SObjectPropertyEntryBox)
			.DisplayBrowse(false)
			.AllowedClass(FSwitchboardEditorModule::GetDisplayClusterRootActorClass())
			.OnObjectChanged_Lambda([&](const FAssetData& AssetData) -> void
			{
				FDisplayClusterRootActorReference* DCRAReference = GetDisplayClusterRootActorReference();

				if (!DCRAReference)
				{
					return;
				}

				if (AssetData.IsValid())
				{
					DCRAReference->DCRA = Cast<AActor>(AssetData.GetAsset());
				}
				else
				{
					DCRAReference->DCRA.Reset();
				}
			})
			.ObjectPath_Lambda([&]() -> FString
			{
				FDisplayClusterRootActorReference* DCRAReference = GetDisplayClusterRootActorReference();

				if (DCRAReference && DCRAReference->DCRA.IsValid())
				{
					FAssetData AssetData(DCRAReference->DCRA.Get(), true);
					return AssetData.GetObjectPathString();
				}

				return TEXT("");
			})
		]
	].IsEnabled(MakeAttributeLambda([=] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
}



#undef LOCTEXT_NAMESPACE