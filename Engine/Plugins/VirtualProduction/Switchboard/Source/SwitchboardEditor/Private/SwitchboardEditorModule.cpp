// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardEditorModule.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/Async.h"
#include "DesktopPlatformModule.h"
#include "DisplayClusterRootActorReferenceDetailCustomization.h"
#include "Editor.h"
#include "EditorUtilitySubsystem.h"
#include "Interfaces/IPluginManager.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "JsonObjectConverter.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/ScopeExit.h"
#include "PropertyEditorModule.h"
#include "SwitchboardEditorSettings.h"
#include "SwitchboardListenerAutolaunch.h"
#include "SwitchboardMenuEntry.h"
#include "SwitchboardProjectSettings.h"
#include "SwitchboardSettingsCustomization.h"
#include "SwitchboardTypes.h"

#include <filesystem>


#define LOCTEXT_NAMESPACE "SwitchboardEditorModule"

DEFINE_LOG_CATEGORY(LogSwitchboardPlugin);


TSoftObjectPtr<UClass> FSwitchboardEditorModule::DisplayClusterRootActorClass;

// static
const FString& FSwitchboardEditorModule::GetSbScriptsPath()
{
	using namespace UE::Switchboard::Private;
	static const FString SbScriptsPath = ConcatPaths(FPaths::EnginePluginsDir(),
		"VirtualProduction", "Switchboard", "Source", "Switchboard");
	return SbScriptsPath;
}


// static
const FString& FSwitchboardEditorModule::GetSbThirdPartyPath()
{
	using namespace UE::Switchboard::Private;
	static const FString SbThirdPartyPath = ConcatPaths(FPaths::EngineDir(),
		"Extras", "ThirdPartyNotUE", "SwitchboardThirdParty");
	return SbThirdPartyPath;
}


// static
const FString& FSwitchboardEditorModule::GetSbExePath()
{
#if PLATFORM_WINDOWS
	static const FString ExePath = GetSbScriptsPath() / TEXT("switchboard.bat");
#elif PLATFORM_LINUX
	static const FString ExePath = GetSbScriptsPath() / TEXT("switchboard.sh");
#endif

	return ExePath;
}

// static
UClass* FSwitchboardEditorModule::GetDisplayClusterRootActorClass()
{
	// Searching by name of the class avoids dependency on nDisplay plugin

	// Early return if we have already found it
	if (DisplayClusterRootActorClass.IsValid())
	{
		return DisplayClusterRootActorClass.Get();
	}

	// Try to find it by name.
	DisplayClusterRootActorClass = FindObject<UClass>(nullptr, TEXT("/Script/DisplayCluster.DisplayClusterRootActor"));

	return DisplayClusterRootActorClass.Get();
}


void FSwitchboardEditorModule::StartupModule()
{
	FSwitchboardMenuEntry::Register();

	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		ISettingsSectionPtr EditorSettingsSection = SettingsModule->RegisterSettings("Editor", "Plugins", "Switchboard",
			LOCTEXT("EditorSettingsName", "Switchboard"),
			LOCTEXT("EditorSettingsDescription", "Configure the Switchboard launcher."),
			GetMutableDefault<USwitchboardEditorSettings>()
		);

		ISettingsSectionPtr ProjectSettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "Switchboard",
			LOCTEXT("ProjectSettingsName", "Switchboard"),
			LOCTEXT("ProjectSettingsDescription", "Configure the Switchboard launcher."),
			GetMutableDefault<USwitchboardProjectSettings>()
		);

		EditorSettingsSection->OnModified().BindRaw(this, &FSwitchboardEditorModule::OnEditorSettingsModified);
	}

	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomClassLayout(
		USwitchboardEditorSettings::StaticClass()->GetFName(), 
		FOnGetDetailCustomizationInstance::CreateStatic(&FSwitchboardEditorSettingsCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomClassLayout(
		USwitchboardProjectSettings::StaticClass()->GetFName(), 
		FOnGetDetailCustomizationInstance::CreateStatic(&FSwitchboardProjectSettingsCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FDisplayClusterRootActorReference::StaticStruct()->GetFName(), 
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDisplayClusterRootActorReferenceDetailCustomization::MakeInstance)
	);

	DeferredStartDelegateHandle = FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FSwitchboardEditorModule::OnEngineInitComplete);

#if SWITCHBOARD_LISTENER_AUTOLAUNCH
	bCachedAutolaunchEnabled = GetListenerAutolaunchEnabled_Internal();
#endif
}


void FSwitchboardEditorModule::ShutdownModule()
{
	if (!IsRunningCommandlet() && UObjectInitialized() && !IsEngineExitRequested())
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout(USwitchboardEditorSettings::StaticClass()->GetFName());
		PropertyModule.UnregisterCustomClassLayout(USwitchboardProjectSettings::StaticClass()->GetFName());

		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->UnregisterSettings("Editor", "Plugins", "Switchboard");
			SettingsModule->UnregisterSettings("Project", "Plugins", "Switchboard");
		}

		FSwitchboardMenuEntry::Unregister();
	}
}


bool FSwitchboardEditorModule::LaunchSwitchboard(const FString& Arguments)
{
	const FString ScriptArgs = FString::Printf(TEXT("\"%s\" %s"),
		*GetDefault<USwitchboardEditorSettings>()->VirtualEnvironmentPath.Path,
		*Arguments
	);

	return RunProcess(GetSbExePath(), ScriptArgs);
}

bool FSwitchboardEditorModule::CompileSwitchboardListener() const
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	if (!DesktopPlatform || !DesktopPlatform->IsUnrealBuildToolAvailable() || DesktopPlatform->IsUnrealBuildToolRunning())
	{
		return false;
	}

	FFeedbackContext* FeedbackContext = GWarn;
	check(FeedbackContext);

	FScopedSlowTask SlowTask(1.0f, LOCTEXT("BuildingSwitchboardListener", "Building SwitchboardListener..."), true /* bIsEnabled */, *FeedbackContext);
	SlowTask.MakeDialog(false /*bShowCancelButton*/, false /*bAllowInPie*/);

	const FString Arguments = FString::Printf(TEXT("SwitchboardListener Development %s -Progress"),
		FPlatformMisc::GetUBTPlatform()
	);

	int32 ExitCode;

	DesktopPlatform->RunUnrealBuildTool(
		LOCTEXT("BuildingSwitchboardListener", "Building SwitchboardListener..."),
		FPaths::RootDir(),
		Arguments,
		FeedbackContext,
		ExitCode
	);

	return !ExitCode;
}

bool FSwitchboardEditorModule::LaunchListener()
{
	const FString ListenerPath = GetDefault<USwitchboardEditorSettings>()->GetListenerPlatformPath();
	const FString ListenerArgs = GetDefault<USwitchboardEditorSettings>()->ListenerCommandlineArguments;

	return RunProcess(ListenerPath, ListenerArgs);
}

bool FSwitchboardEditorModule::CreateNewConfig(const FSwitchboardNewConfigUserOptions& NewConfigUserOptions)
{
	FSwitchboardScriptArguments ScriptArgs;

	ScriptArgs.ConfigName = NewConfigUserOptions.ConfigName;
	ScriptArgs.EngineDir = FPaths::ConvertRelativePathToFull(FPaths::EngineDir());
	ScriptArgs.ProjectPath = FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath());

	// Find persistent map
	if (const UWorld* World = GEditor ? GEditor->GetEditorWorldContext(false).World() : nullptr)
	{
		FString ClassName, PackageName, ObjectName, SubObjectName;
		FPackageName::SplitFullObjectPath(World->PersistentLevel.GetPathName(), ClassName, PackageName, ObjectName, SubObjectName, true);
		ScriptArgs.Map = PackageName;
	}

	// Get path to DCRA asset
	if (const AActor* DCRA = NewConfigUserOptions.DCRA.DCRA.Get())
	{
		FString ClassName, PackageName, ObjectName, SubObjectName;

		FPackageName::SplitFullObjectPath(DCRA->GetClass()->GetFullName(), ClassName, PackageName, ObjectName, SubObjectName, true);

		const FString Extension = FPackageName::GetAssetPackageExtension();
		const FString RelativeFilepath = FPackageName::LongPackageNameToFilename(PackageName, Extension);
		const FString AbsoluteFilepath = FPaths::ConvertRelativePathToFull(RelativeFilepath);

		if (FPaths::FileExists(AbsoluteFilepath))
		{
			ScriptArgs.DisplayClusterConfigPath = AbsoluteFilepath;
		}
	}

	ScriptArgs.bUseLocalhost = NewConfigUserOptions.bUseLocalhost;
	ScriptArgs.bAutoConnect = NewConfigUserOptions.bAutoConnect;
	ScriptArgs.NumEditorDevices = NewConfigUserOptions.NumEditorDevices;

	// Convert to Json and save it to a temp file

	FString ScriptArgsString;
	{
		const bool bSBArgsOk = FJsonObjectConverter::UStructToJsonObjectString(ScriptArgs, ScriptArgsString, 0, 0, 0, nullptr, false);
		check(bSBArgsOk); // This should always succeed because we're converting a known USTRUCT.
	}

	const FString ScriptArgsFilepath = FPaths::CreateTempFilename(
		WCHAR_TO_TCHAR(std::filesystem::temp_directory_path().wstring().c_str()),
		TEXT("sb_script_args_"),
		TEXT(".json")
	);

	if (!FFileHelper::SaveStringToFile(ScriptArgsString, *ScriptArgsFilepath))
	{
		UE_LOGF(LogSwitchboardPlugin, Log, "Could not save temp script arguments file '%ls'", *ScriptArgsFilepath);
		return false;
	}
	
	UE_LOGF(LogSwitchboardPlugin, Verbose, "Will launch Switchboard using script arguments: '%ls' which were saved in: '%ls'",
		*ScriptArgsString, *ScriptArgsFilepath
	);

	// The script that we will pass to Switchoard is in this plugin's Content/Python folder.
	const FString PluginContentDir = IPluginManager::Get().FindPlugin(TEXT("Switchboard"))->GetContentDir();
	const FString ScriptPath = UE::Switchboard::Private::ConcatPaths(PluginContentDir, "Python", "sb_script_new_config.py");

	// Here we tell Switchboard to run the given script with the given argument, 
	// and the argument is the path to a json file that will be interpreted by the script.
	// We also pass --defaultenv so that SB doesn't interpret the first argument as the virtual environment.
	const FString Arguments = FString::Printf(TEXT("--defaultenv --script \"%s\" --scriptargs \"%s\""),
		*ScriptPath,
		*ScriptArgsFilepath
	);

	return LaunchSwitchboard(Arguments);
}

bool FSwitchboardEditorModule::RunProcess(const FString& InExe, const FString& InArgs)
{
	const bool bLaunchDetached = false;
	const bool bLaunchHidden = false;
	const bool bLaunchReallyHidden = false;
	uint32* OutProcessId = nullptr;
	const int32 PriorityModifier = 0;
	const TCHAR* WorkingDirectory = nullptr;
	void* PipeWriteChild = nullptr;

	UE_LOGF(LogSwitchboardPlugin, Display, "Launching: %ls %ls", *InExe, *InArgs);

	const FProcHandle Handle = FPlatformProcess::CreateProc(*InExe, *InArgs, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, OutProcessId, PriorityModifier, WorkingDirectory, PipeWriteChild);
	return Handle.IsValid();
}


FSwitchboardEditorModule::ESwitchboardInstallState FSwitchboardEditorModule::GetSwitchboardInstallState()
{
	TSharedFuture<FSwitchboardVerifyResult> Result = GetVerifyResult();

	if (!Result.IsReady())
	{
		return ESwitchboardInstallState::VerifyInProgress;
	}

	if (Result.Get().Summary != FSwitchboardVerifyResult::ESummary::Success)
	{
		return ESwitchboardInstallState::NeedInstallOrRepair;
	}

#if SWITCHBOARD_SHORTCUTS
	const bool bListenerDesktopShortcutExists = DoesShortcutExist(EShortcutApp::Listener, EShortcutLocation::Desktop) == EShortcutCompare::AlreadyExists;
	const bool bListenerProgramsShortcutExists = DoesShortcutExist(EShortcutApp::Listener, EShortcutLocation::Programs) == EShortcutCompare::AlreadyExists;
	const bool bAppDesktopShortcutExists = DoesShortcutExist(EShortcutApp::Switchboard, EShortcutLocation::Desktop) == EShortcutCompare::AlreadyExists;
	const bool bAppProgramsShortcutExists = DoesShortcutExist(EShortcutApp::Switchboard, EShortcutLocation::Programs) == EShortcutCompare::AlreadyExists;

	if (!bListenerDesktopShortcutExists || !bListenerProgramsShortcutExists
		|| !bAppDesktopShortcutExists || !bAppProgramsShortcutExists)
	{
		return ESwitchboardInstallState::ShortcutsMissing;
	}
#endif // #if SWITCHBOARD_SHORTCUTS

	return ESwitchboardInstallState::Nominal;
}


TSharedFuture<FSwitchboardVerifyResult> FSwitchboardEditorModule::GetVerifyResult(bool bForceRefresh /* = false */)
{
	const FString CurrentVenv = GetDefault<USwitchboardEditorSettings>()->VirtualEnvironmentPath.Path;

	if (bForceRefresh || !VerifyResult.IsValid() || VerifyPath != CurrentVenv)
	{
#if SWITCHBOARD_SHORTCUTS
		CachedShortcutCompares.Empty();
#endif

		UE_LOGF(LogSwitchboardPlugin, Log, "Issuing verify for venv: %ls", *CurrentVenv);

		TFuture<FSwitchboardVerifyResult> Future = FSwitchboardVerifyResult::RunVerify(CurrentVenv);
		Future = Future.Next([Venv = CurrentVenv](const FSwitchboardVerifyResult& Result)
		{
			UE_LOGF(LogSwitchboardPlugin, Log, "Verify complete for venv: %ls", *Venv);
			UE_LOGF(LogSwitchboardPlugin, Log, "Verify summary: %d", int(Result.Summary));
			UE_LOGF(LogSwitchboardPlugin, Log, "Verify log: %ls", *Result.Log);

			// Disabled hiding for now; new dropdown menu items have been added which have no other UI,
			// and some user confusion has been reported. We need to revisit this UX.
#if 0 // #if SWITCHBOARD_SHORTCUTS
			// On platforms where we support creating shortcuts, once shortcuts have been created,
			// we hide our toolbar button to yield space.

			// This task gets us from the future back onto the main thread prior to manipulating the UI.
			Async(EAsyncExecution::TaskGraphMainThread, []() {
				if (FSwitchboardEditorModule::Get().GetSwitchboardInstallState() == ESwitchboardInstallState::Nominal)
				{
					FSwitchboardMenuEntry::RemoveMenu();
				}
				else
				{
					FSwitchboardMenuEntry::AddMenu();
				}
			});
#endif

			return Result;
		});

		VerifyPath = CurrentVenv;
		VerifyResult = Future.Share();
	}

	return VerifyResult;
}


#if SWITCHBOARD_LISTENER_AUTOLAUNCH
bool FSwitchboardEditorModule::IsListenerAutolaunchEnabled(bool bForceRefreshCache /* = false */)
{
	if (bForceRefreshCache)
	{
		bCachedAutolaunchEnabled = GetListenerAutolaunchEnabled_Internal();
	}

	return bCachedAutolaunchEnabled;
}


bool FSwitchboardEditorModule::GetListenerAutolaunchEnabled_Internal() const
{
	const FString& ExistingCmd = UE::SwitchboardListener::Autolaunch::GetInvocation(LogSwitchboardPlugin).TrimEnd();
	const FString& ConfigCmd = GetDefault<USwitchboardEditorSettings>()->GetListenerInvocation().TrimEnd();
	return ExistingCmd == ConfigCmd;
}


bool FSwitchboardEditorModule::SetListenerAutolaunchEnabled(bool bEnabled)
{
	bool bSucceeded = false;

	if (bEnabled)
	{
		const FString CommandLine = GetDefault<USwitchboardEditorSettings>()->GetListenerInvocation();
		bSucceeded = UE::SwitchboardListener::Autolaunch::SetInvocation(CommandLine, LogSwitchboardPlugin);
	}
	else
	{
		bSucceeded = UE::SwitchboardListener::Autolaunch::RemoveInvocation(LogSwitchboardPlugin);
	}

	bCachedAutolaunchEnabled = GetListenerAutolaunchEnabled_Internal();
	return bSucceeded;
}
#endif // #if SBL_AUTOLAUNCH


#if SWITCHBOARD_SHORTCUTS
FSwitchboardEditorModule::EShortcutCompare FSwitchboardEditorModule::DoesShortcutExist(
	EShortcutApp App,
	EShortcutLocation Location,
	bool bForceRefreshCache /* = false */
)
{
	using namespace UE::Switchboard::Private::Shorcuts;

	const TPair<EShortcutApp, EShortcutLocation> CacheKey{ App, Location };

	if (!bForceRefreshCache)
	{
		if (const EShortcutCompare* CachedResult = CachedShortcutCompares.Find(CacheKey))
		{
			return *CachedResult;
		}
	}

	const FShortcutParams ExpectedParams = BuildShortcutParams(App, Location);
	const EShortcutCompare Result = CompareShortcut(ExpectedParams);
	CachedShortcutCompares.FindOrAdd(CacheKey) = Result;
	return Result;
}


bool FSwitchboardEditorModule::CreateOrUpdateShortcut(EShortcutApp App, EShortcutLocation Location)
{
	using namespace UE::Switchboard::Private::Shorcuts;

	const FShortcutParams Params = BuildShortcutParams(App, Location);
	const bool bResult = UE::Switchboard::Private::Shorcuts::CreateOrUpdateShortcut(Params);

	const bool bForceUpdateCache = true;
	(void)DoesShortcutExist(App, Location, bForceUpdateCache);

	return bResult;
}
#endif // #if SWITCHBOARD_SHORTCUTS


void FSwitchboardEditorModule::OnEngineInitComplete()
{
	FCoreDelegates::OnFEngineLoopInitComplete.Remove(DeferredStartDelegateHandle);
	DeferredStartDelegateHandle.Reset();

	RunDefaultOSCListener();

	// Populate initial verification results.
	GetVerifyResult(true);
}


bool FSwitchboardEditorModule::OnEditorSettingsModified()
{
#if SWITCHBOARD_LISTENER_AUTOLAUNCH
	// If the existing entry's listener executable path matches our current engine / path,
	// then we ensure the command line arguments also stay in sync with our settings.
	const FString AutolaunchEntryExecutable = UE::SwitchboardListener::Autolaunch::GetInvocationExecutable(LogSwitchboardPlugin);
	if (!AutolaunchEntryExecutable.IsEmpty())
	{
		const FString ConfigExecutable = GetDefault<USwitchboardEditorSettings>()->GetListenerPlatformPath();
		if (AutolaunchEntryExecutable == ConfigExecutable)
		{
			UE_LOGF(LogSwitchboardPlugin, Log, "Updating listener auto-launch entry");
			SetListenerAutolaunchEnabled(true);
		}
		else
		{
			UE_LOGF(LogSwitchboardPlugin, Log, "NOT updating listener auto-launch; paths differ:\n\t%ls\n\t%ls",
				*AutolaunchEntryExecutable, *ConfigExecutable);
		}
	}
#endif // #if SWITCHBOARD_LISTENER_AUTOLAUNCH

	return true;
}


void FSwitchboardEditorModule::RunDefaultOSCListener()
{
	USwitchboardProjectSettings* SwitchboardProjectSettings = USwitchboardProjectSettings::GetSwitchboardProjectSettings();
	FSoftObjectPath SwitchboardOSCListener = SwitchboardProjectSettings->SwitchboardOSCListener;
	if (SwitchboardOSCListener.IsValid())
	{
		UObject* SwitchboardOSCListenerObject = SwitchboardOSCListener.TryLoad();
		if (SwitchboardOSCListenerObject && IsValidChecked(SwitchboardOSCListenerObject) && !SwitchboardOSCListenerObject->IsUnreachable() && GEditor)
		{
			GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>()->TryRun(SwitchboardOSCListenerObject);
		}
	}
}


IMPLEMENT_MODULE(FSwitchboardEditorModule, SwitchboardEditor);

#undef LOCTEXT_NAMESPACE
