// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"
#include "Delegates/IDelegateInstance.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "SwitchboardListenerAutolaunch.h"
#include "SwitchboardShortcuts.h"
#include "SwitchboardScriptInterop.h"
#include "SwitchboardTypes.h"



DECLARE_LOG_CATEGORY_EXTERN(LogSwitchboardPlugin, Log, All);


namespace UE::Switchboard::Private
{
	template <typename... TPaths>
	FString ConcatPaths(FString BaseDir, TPaths... InPaths)
	{
		return (FPaths::ConvertRelativePathToFull(BaseDir) / ... / InPaths);
	}
} // namespace UE::Switchboard::Private


class FSwitchboardEditorModule : public IModuleInterface
{
public:
	static const FString& GetSbScriptsPath();
	static const FString& GetSbThirdPartyPath();
	static const FString& GetSbExePath();

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static FSwitchboardEditorModule& Get()
	{
		static const FName ModuleName = "SwitchboardEditor";
		return FModuleManager::LoadModuleChecked<FSwitchboardEditorModule>(ModuleName);
	};

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	bool LaunchSwitchboard(const FString& Arguments = FString());
	bool LaunchListener();

	/**
	 * Launches Switchboard with a new config that matches the current scene
	 *
	 * @param NewConfigUserOptions User options for New Config creation
	 * 
	 * @return Returns true if successful
	 */
	bool CreateNewConfig(const FSwitchboardNewConfigUserOptions& NewConfigUserOptions);

	/**
	 * Compiles SwitchboardListener (Development configuration)
	 *
	 * @return Returns true if successful
	 */
	bool CompileSwitchboardListener() const;

	enum class ESwitchboardInstallState
	{
		VerifyInProgress,

		NeedInstallOrRepair,
		ShortcutsMissing,
		Nominal,
	};

	ESwitchboardInstallState GetSwitchboardInstallState();

	TSharedFuture<FSwitchboardVerifyResult> GetVerifyResult(bool bForceRefresh = false);

#if SWITCHBOARD_LISTENER_AUTOLAUNCH
	/**
	 * Returns whether (this engine's) SwitchboardListener is configured to run automatically.
	 * Defaults to returning a cached value to avoid hitting the registry.
	 */
	bool IsListenerAutolaunchEnabled(bool bForceRefreshCache = false);

	/** Enables or disables auto-run of SwitchboardListener. */
	bool SetListenerAutolaunchEnabled(bool bEnabled);
#endif

#if SWITCHBOARD_SHORTCUTS
	using EShortcutApp = UE::Switchboard::Private::Shorcuts::EShortcutApp;
	using EShortcutLocation = UE::Switchboard::Private::Shorcuts::EShortcutLocation;
	using EShortcutCompare = UE::Switchboard::Private::Shorcuts::EShortcutCompare;

	/**
	 * Returns whether shortcuts exist for (this engine's) Switchboard / Listener.
	 * Defaults to returning a cached value to avoid hitting the filesystem.
	 */
	EShortcutCompare DoesShortcutExist(EShortcutApp App, EShortcutLocation Location, bool bForceRefreshCache = false);

	/** Creates (or replaces) shortcuts for Switchboard / Listener. */
	bool CreateOrUpdateShortcut(EShortcutApp App, EShortcutLocation Location);
#endif

	/** Returns the DCRA class, nullptr if it doesn't exist (i.e. nDisplay plugin is not enabled) */
	static UClass* GetDisplayClusterRootActorClass();

private:
	void OnEngineInitComplete();
	bool OnEditorSettingsModified();

	void RunDefaultOSCListener();

	bool RunProcess(const FString& InExe, const FString& InArgs);

private:

	FDelegateHandle DeferredStartDelegateHandle;

	FString VerifyPath;
	TSharedFuture<FSwitchboardVerifyResult> VerifyResult;

#if SWITCHBOARD_LISTENER_AUTOLAUNCH
	bool GetListenerAutolaunchEnabled_Internal() const;

	bool bCachedAutolaunchEnabled;
#endif

#if SWITCHBOARD_SHORTCUTS
	TMap< TPair<EShortcutApp, EShortcutLocation>, EShortcutCompare > CachedShortcutCompares;
#endif

	/** Cached DCRA Class */
	static TSoftObjectPtr<UClass> DisplayClusterRootActorClass;
};
