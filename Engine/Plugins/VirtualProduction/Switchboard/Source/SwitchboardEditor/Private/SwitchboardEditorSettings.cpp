// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardEditorSettings.h"
#include "SwitchboardEditorModule.h"
#include "GenericPlatform/GenericPlatformProperties.h"
#include "Misc/Paths.h"


USwitchboardEditorSettings::USwitchboardEditorSettings()
{
	using namespace UE::Switchboard::Private;

	VirtualEnvironmentPath.Path = ConcatPaths(FPaths::EngineDir(),
		"Extras", "ThirdPartyNotUE", "SwitchboardThirdParty", "Python");
}


USwitchboardEditorSettings* USwitchboardEditorSettings::GetSwitchboardEditorSettings()
{
	return GetMutableDefault<USwitchboardEditorSettings>();
}


FString USwitchboardEditorSettings::GetListenerPlatformPath() const
{
	FString Path = FPaths::ConvertRelativePathToFull(
		FPlatformProcess::GenerateApplicationPath(TEXT("SwitchboardListener"), EBuildConfiguration::Development));
	FPaths::MakePlatformFilename(Path);
	return Path;
}


FString USwitchboardEditorSettings::GetListenerInvocation() const
{
	return FString::Printf(TEXT("\"%s\" %s"), *GetListenerPlatformPath(), *ListenerCommandlineArguments);
}
