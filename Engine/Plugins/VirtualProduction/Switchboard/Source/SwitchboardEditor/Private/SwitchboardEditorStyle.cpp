// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardEditorStyle.h"

#include "Styling/AppStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"


FSwitchboardEditorStyle::FSwitchboardEditorStyle()
	: FSlateStyleSet("SwitchboardEditorStyle")
{
	static const FVector2D Icon32x32(32.0f, 32.0f);

	FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("Switchboard"))->GetContentDir();
	SetContentRoot(ContentDir);
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Key matches "(Context).(CommandId)" icon search used by our corresponding UI_COMMAND
	Set("Switchboard.LaunchSwitchboard", new IMAGE_BRUSH("Icons/Switchboard_40x", FVector2D(40.0f, 40.0f)));

	Set("Settings.RowBorder.Warning", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::Warning, 1.0f));
	Set("Settings.RowBorder.Nominal", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::Success, 1.0f));

	Set("Wizard.Background", new FSlateColorBrush(FStyleColors::Panel));

	Set("Settings.Icons.Warning", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-triangle-large", Icon32x32, FStyleColors::Warning));
	Set("Settings.Icons.Nominal", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-triangle-large", Icon32x32, FStyleColors::Success));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FSwitchboardEditorStyle::~FSwitchboardEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FSwitchboardEditorStyle& FSwitchboardEditorStyle::Get()
{
	static FSwitchboardEditorStyle Inst;
	return Inst;
}
