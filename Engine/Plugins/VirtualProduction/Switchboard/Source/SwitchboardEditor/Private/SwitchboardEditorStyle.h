// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

/**
 * Slate style set that defines all the styles for the Switchboard Toolbar
 */
class SWITCHBOARDEDITOR_API FSwitchboardEditorStyle : public FSlateStyleSet
{
public:
	static FSwitchboardEditorStyle& Get();

private:
	FSwitchboardEditorStyle();
	~FSwitchboardEditorStyle();
};
