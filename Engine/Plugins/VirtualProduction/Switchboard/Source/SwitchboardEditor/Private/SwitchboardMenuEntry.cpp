// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardMenuEntry.h"

#include "Dialog/SCustomDialog.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "IStructureDetailsView.h"
#include "Kismet/GameplayStatics.h"
#include "LevelEditor.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SwitchboardEditorModule.h"
#include "SwitchboardEditorSettings.h"
#include "SwitchboardEditorStyle.h"
#include "SwitchboardSetupWizard.h"
#include "SwitchboardTypes.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "SwitchboardEditor"


class FSwitchboardUICommands : public TCommands<FSwitchboardUICommands>
{
public:
	FSwitchboardUICommands()
		: TCommands<FSwitchboardUICommands>("Switchboard", LOCTEXT("SwitchboardCommands", "Switchboard"), NAME_None, FSwitchboardEditorStyle::Get().GetStyleSetName())
	{}

	virtual void RegisterCommands() override
	{
		UI_COMMAND(LaunchSwitchboard, "Launch Switchboard", "Launch Switchboard", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(LaunchSwitchboardListener, "Launch Switchboard Listener", "Launch Switchboard Listener", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(CreateNewConfig, "Create Config", "Create Config", EUserInterfaceActionType::Button, FInputChord());
	}

	TSharedPtr<FUICommandInfo> LaunchSwitchboard;
	TSharedPtr<FUICommandInfo> LaunchSwitchboardListener;
	TSharedPtr<FUICommandInfo> CreateNewConfig;
};


struct FSwitchboardMenuEntryImpl
{
	FSwitchboardMenuEntryImpl()
	{
		FSwitchboardUICommands::Register();

		Actions->MapAction(FSwitchboardUICommands::Get().LaunchSwitchboard,
			FExecuteAction::CreateRaw(this, &FSwitchboardMenuEntryImpl::OnLaunchSwitchboard),
			FCanExecuteAction());

		Actions->MapAction(FSwitchboardUICommands::Get().LaunchSwitchboardListener,
			FExecuteAction::CreateRaw(this, &FSwitchboardMenuEntryImpl::OnLaunchSwitchboardListener),
			FCanExecuteAction());

		Actions->MapAction(FSwitchboardUICommands::Get().CreateNewConfig,
			FExecuteAction::CreateRaw(this, &FSwitchboardMenuEntryImpl::OnLaunchWithNewConfig),
			FCanExecuteAction());

		AddMenu();
	}

	void AddMenu()
	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.User");
		if (Menu->FindSection("Switchboard"))
		{
			return;
		}

		FToolMenuSection& Section = Menu->AddSection("Switchboard");

		// Toolbar button
		{
			FToolMenuEntry SwitchboardButtonEntry = FToolMenuEntry::InitToolBarButton(FSwitchboardUICommands::Get().LaunchSwitchboard);
			SwitchboardButtonEntry.SetCommandList(Actions);

			Section.AddEntry(SwitchboardButtonEntry);
		}

		// Entries
		{
			const FToolMenuEntry SwitchboardComboEntry = FToolMenuEntry::InitComboButton(
				"SwitchboardMenu",
				FUIAction(),
				FOnGetContent::CreateRaw(this, &FSwitchboardMenuEntryImpl::CreateMenuEntries),
				TAttribute<FText>::Create([this]() { return LOCTEXT("LaunchSwitchboard", "Switchboard"); }),
				LOCTEXT("SwitchboardTooltip", "Actions related to the SwitchboardListener"),
				FSlateIcon(),
				true //bInSimpleComboBox
			);

			Section.AddEntry(SwitchboardComboEntry);
		}
	}

	void RemoveMenu()
	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.User");
		if (Menu->FindSection("Switchboard"))
		{
			Menu->RemoveSection("Switchboard");
		}
	}

	~FSwitchboardMenuEntryImpl()
	{
		if (!IsEngineExitRequested() && UObjectInitialized())
		{
			UToolMenus::Get()->RemoveSection("LevelEditor.LevelEditorToolBar.User", "Switchboard");
		}
	}

	TSharedRef<SWidget> CreateMenuEntries()
	{
		const bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, Actions);

		MenuBuilder.BeginSection("Switchboard", LOCTEXT("SwitchboardListener", "Listener"));
		{
			MenuBuilder.AddMenuEntry(FSwitchboardUICommands::Get().LaunchSwitchboardListener);

#if SWITCHBOARD_LISTENER_AUTOLAUNCH
			MenuBuilder.AddMenuSeparator();

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ListenerAutolaunchLabel", "Launch Switchboard Listener on Login"),
				LOCTEXT("ListenerAutolaunchTooltip", "Controls whether SwitchboardListener runs automatically when you log into Windows."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FSwitchboardMenuEntryImpl::ToggleAutolaunch),
					FCanExecuteAction(),
					FGetActionCheckState::CreateLambda([&]()
					{
						return FSwitchboardEditorModule::Get().IsListenerAutolaunchEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
#endif // #if SWITCHBOARD_LISTENER_AUTOLAUNCH
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("Switchboard", LOCTEXT("SwitchboardConfig", "Config"));
		{
			MenuBuilder.AddMenuEntry(FSwitchboardUICommands::Get().CreateNewConfig);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	void OnLaunchSwitchboardListener()
	{
		const FString ListenerPath = GetDefault<USwitchboardEditorSettings>()->GetListenerPlatformPath();

		if (!FPaths::FileExists(ListenerPath))
		{
			// Since SwitchboardListener (SBL) doesn't exist, ask the user if they want us to compile it.

			const FText Msg = LOCTEXT("CouldNotFindSwitchboardListenerCompileAskIfCompile", "Could not find SwitchboardListener. Would you like to compile it? ");

			if (EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, Msg))
			{
				// Compile SBL
				if (!FSwitchboardEditorModule::Get().CompileSwitchboardListener())
				{
					const FText ErrorMsg = LOCTEXT("FailedToCompileSwitchboardListenerCompileIDE", "Failed to compile SwitchboardListener. Please compile with UGS or your IDE");
					FMessageDialog::Open(EAppMsgType::Ok, ErrorMsg);
					return;
				}
			}
			else
			{
				// User did not want us to compile SBL, so we return.
				return;
			}
		}

		if (!FPaths::FileExists(ListenerPath))
		{
			const FText ErrorMsg = LOCTEXT("CouldNotFindSwitchboardListenerCompileMakeSureCompiled", "Could not find SwitchboardListener! Make sure it was compiled.");
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMsg);
			return;
		}

		if (FSwitchboardEditorModule::Get().LaunchListener())
		{
			UE_LOGF(LogSwitchboardPlugin, Display, "Successfully started SwitchboardListener");
		}
		else
		{
			const FText ErrorMsg = LOCTEXT("UnableToStartTheListener", "Unable to start the listener! Make sure it was compiled. Check the log for details.");
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMsg);
		}
	}

#if SWITCHBOARD_LISTENER_AUTOLAUNCH
	void ToggleAutolaunch()
	{
		if (FSwitchboardEditorModule::Get().IsListenerAutolaunchEnabled())
		{
			FSwitchboardEditorModule::Get().SetListenerAutolaunchEnabled(false);
		}
		else
		{
			if (!FPaths::FileExists(GetDefault<USwitchboardEditorSettings>()->GetListenerPlatformPath()))
			{
				const FText ErrorMsg = LOCTEXT("CouldNotFindSwitchboardListenerCompileMakeSureHasBeenCompiled", "Could not find SwitchboardListener! Make sure it has been compiled.");
				FMessageDialog::Open(EAppMsgType::Ok, ErrorMsg);
				return;
			}

			FSwitchboardEditorModule::Get().SetListenerAutolaunchEnabled(true);
		}
	}
#endif // #if SWITCHBOARD_LISTENER_AUTOLAUNCH

	void OnLaunchSwitchboard()
	{
		const FSwitchboardVerifyResult& VerifyResult = FSwitchboardEditorModule::Get().GetVerifyResult().Get();
		if (VerifyResult.Summary != FSwitchboardVerifyResult::ESummary::Success)
		{
			SSwitchboardSetupWizard::OpenWindow();
			return;
		}

		if (FSwitchboardEditorModule::Get().LaunchSwitchboard())
		{
			UE_LOGF(LogSwitchboardPlugin, Display, "Successfully started Switchboard");
		}
		else
		{
			const FText ErrorMsg = LOCTEXT("UnableToStartSwitchboardCheckLog", "Unable to start Switchboard! Check the log for details.");
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMsg);
		}
	}

	void OnLaunchWithNewConfig()
	{
		const FSwitchboardVerifyResult& VerifyResult = FSwitchboardEditorModule::Get().GetVerifyResult().Get();
		if (VerifyResult.Summary != FSwitchboardVerifyResult::ESummary::Success)
		{
			SSwitchboardSetupWizard::OpenWindow();
			return;
		}

		TSharedPtr<TStructOnScope<FSwitchboardNewConfigUserOptions>> NewConfigUserOptions
			= MakeShared<TStructOnScope<FSwitchboardNewConfigUserOptions>>();

		NewConfigUserOptions->InitializeAs<FSwitchboardNewConfigUserOptions>();

		// Pre-populate the DCRA
		if (UClass* DisplayClusterRootActorClass = FSwitchboardEditorModule::GetDisplayClusterRootActorClass())
		{
			const UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;

			TArray<AActor*> DCRAs;
			UGameplayStatics::GetAllActorsOfClass(World, DisplayClusterRootActorClass, DCRAs);

			if (DCRAs.Num())
			{
				// Pick the first one
				NewConfigUserOptions->Get()->DCRA.DCRA = DCRAs[0];
			}
		}

		FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

		FStructureDetailsViewArgs StructureViewArgs;
		FDetailsViewArgs DetailArgs;

		DetailArgs.bAllowSearch = false;
		DetailArgs.bShowScrollBar = true;

		TSharedPtr<IStructureDetailsView> NewConfigUsernOptionsDetailsView
			= PropertyEditor.CreateStructureDetailView(DetailArgs, StructureViewArgs, NewConfigUserOptions);

		TSharedRef<SCustomDialog> OptionsWindow = SNew(SCustomDialog)
			.Title(LOCTEXT("SwitchboardNewConfigOptions", "Switchboard New Config Options"))
			.ToolTipText(LOCTEXT("CreateNewConfigTooltip", "This will open Switchboard and create a new config from the parameters below and the current map."))
			.Content()
			[
				NewConfigUsernOptionsDetailsView->GetWidget().ToSharedRef()
			]
			.Buttons
			({
				SCustomDialog::FButton(LOCTEXT("Ok", "Ok"), FSimpleDelegate::CreateLambda([NewConfigUserOptions]() -> void
				{
					check(NewConfigUserOptions.IsValid());
					FSwitchboardEditorModule::Get().CreateNewConfig(*NewConfigUserOptions->Get());
				})),
				SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel")),
			});

		OptionsWindow->Show(); // Showing it non-modal mostly because its DCRA actor picker doesn't work if the window is modal.
	}

public:
	static TUniquePtr<FSwitchboardMenuEntryImpl> Implementation;

	TSharedPtr<FUICommandList> Actions = MakeShared<FUICommandList>();
};

TUniquePtr<FSwitchboardMenuEntryImpl> FSwitchboardMenuEntryImpl::Implementation;

void FSwitchboardMenuEntry::Register()
{
	if (!IsRunningCommandlet())
	{
		FSwitchboardMenuEntryImpl::Implementation = MakeUnique<FSwitchboardMenuEntryImpl>();
	}
}

void FSwitchboardMenuEntry::AddMenu()
{
	if (FSwitchboardMenuEntryImpl::Implementation)
	{
		FSwitchboardMenuEntryImpl::Implementation->AddMenu();
	}
}

void FSwitchboardMenuEntry::RemoveMenu()
{
	if (FSwitchboardMenuEntryImpl::Implementation)
	{
		FSwitchboardMenuEntryImpl::Implementation->RemoveMenu();
	}
}

void FSwitchboardMenuEntry::Unregister()
{
	FSwitchboardMenuEntryImpl::Implementation.Reset();
}

#undef LOCTEXT_NAMESPACE
