// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardScriptInterop.h"
#include "SwitchboardEditorModule.h"
#include "Async/Async.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"


TSharedRef<FSwitchboardUtilScript> FSwitchboardUtilScript::RunInstall(FStringView VenvPath)
{
	using namespace UE::Switchboard::Private;

	const FString SbSetupPath = ConcatPaths(FSwitchboardEditorModule::Get().GetSbScriptsPath(), "sb_setup.py");

	const FString Args = FString::Printf(TEXT("\"%s\" install --venv-dir=\"%.*s\""),
		*SbSetupPath, VenvPath.Len(), VenvPath.GetData());

	// We don't pass the venv to the constructor because we use the engine python to bootstrap.
	TSharedRef<FSwitchboardUtilScript> Setup = MakeShared<FSwitchboardUtilScript>();
	ensure(Setup->Run(Args));
	return Setup;
}


FSwitchboardUtilScript::FSwitchboardUtilScript()
{
	using namespace UE::Switchboard::Private;

	PythonExe = ConcatPaths(
		FPaths::EngineDir(), "Binaries", "ThirdParty", "Python3",
		FPlatformProcess::GetBinariesSubdirectory(),
#if PLATFORM_WINDOWS
		"pythonw.exe"
#else
		"bin", "python3"
#endif
	);
}


FSwitchboardUtilScript::FSwitchboardUtilScript(FStringView PythonVenvDir)
{
	using namespace UE::Switchboard::Private;

	PythonExe = ConcatPaths(
		FString(PythonVenvDir),
#if PLATFORM_WINDOWS
		"Scripts", "pythonw.exe"
#else
		"bin", "python3"
#endif
	);
}


FSwitchboardUtilScript::~FSwitchboardUtilScript()
{
	FPlatformProcess::ClosePipe(StdoutParentReadPipe, StdoutChildWritePipe);

	if (ProcHandle.IsValid())
	{
		FPlatformProcess::TerminateProc(ProcHandle);
	}
}


bool FSwitchboardUtilScript::Run(const FString& Args)
{
	if (ProcHandle.IsValid())
	{
		// Not currently designed to Run() twice.
		return ensure(false);
	}

	if (!FPaths::FileExists(PythonExe))
	{
		return false;
	}

	const bool bLaunchDetached = false;
	const bool bLaunchHidden = true;
	const bool bLaunchReallyHidden = true;
	uint32 OutProcessId = 0;
	const int32 PriorityModifier = 0;
	const TCHAR* WorkingDirectory = nullptr;
	FPlatformProcess::CreatePipe(StdoutParentReadPipe, StdoutChildWritePipe);

	ProcHandle = FPlatformProcess::CreateProc(*PythonExe, *Args, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, &OutProcessId, PriorityModifier, WorkingDirectory, StdoutChildWritePipe);
	return ProcHandle.IsValid();
}


TOptional<int32> FSwitchboardUtilScript::PollStdoutAndReturnCode()
{
	if (ReturnCode.IsSet())
	{
		// Process exited previously; nothing to do.
		return ReturnCode;
	}

	if (!ProcHandle.IsValid())
	{
		return TOptional<int32>();
	}

	TArray<uint8> Output;
	if (FPlatformProcess::ReadPipeToArray(StdoutParentReadPipe, Output))
	{
		StdoutBuf.Append(Output);
	}

	int32 OutReturnCode;
	if (FPlatformProcess::GetProcReturnCode(ProcHandle, &OutReturnCode))
	{
		ReturnCode.Emplace(OutReturnCode);

		// Additional read necessary to ensure we've captured all output.
		if (FPlatformProcess::ReadPipeToArray(StdoutParentReadPipe, Output))
		{
			StdoutBuf.Append(Output);
		}
	}

	return ReturnCode;
}


//static
TFuture<FSwitchboardVerifyResult> FSwitchboardVerifyResult::RunVerify(FStringView VenvPath)
{
	using namespace UE::Switchboard::Private;

	return Async(EAsyncExecution::Thread,
		[
			VenvPath = FString(VenvPath),
			StartTimeSeconds = FPlatformTime::Seconds()
		]() -> FSwitchboardVerifyResult
		{
			FSwitchboardVerifyResult Result;

			FSwitchboardUtilScript Verify(VenvPath);

			if (!FPaths::FileExists(Verify.PythonExe))
			{
				UE_LOGF(LogSwitchboardPlugin, Log, "RunVerify: Python interpreter not found: %ls", *Verify.PythonExe);
				Result.Summary = FSwitchboardVerifyResult::ESummary::InterpreterMissing;
				return Result;
			}

			const FString SbSetupPath = ConcatPaths(FSwitchboardEditorModule::Get().GetSbScriptsPath(), "sb_setup.py");
			const FString Args = FString::Printf(TEXT("\"%s\" verify --output-json"), *SbSetupPath);
			if (!Verify.Run(Args))
			{
				UE_LOGF(LogSwitchboardPlugin, Log, "RunVerify: Failed to launch: \"%ls\" %ls",
					*Verify.PythonExe, *Args);
				Result.Summary = FSwitchboardVerifyResult::ESummary::LaunchFailed;
				return Result;
			}

			TOptional<int32> ReturnCode;
			while (!ReturnCode.IsSet())
			{
				ReturnCode = Verify.PollStdoutAndReturnCode();
				if (!ReturnCode.IsSet())
				{
					const double VerifyTimeoutSeconds = 10.0f;
					const double TimeElapsed = FPlatformTime::Seconds() - StartTimeSeconds;
					if (TimeElapsed < VerifyTimeoutSeconds)
					{
						FPlatformProcess::Sleep(0.1f);
					}
					else
					{
						UE_LOGF(LogSwitchboardPlugin, Warning, "RunVerify: Exceeded timeout");
						FPlatformProcess::TerminateProc(Verify.ProcHandle);
						Result.Summary = FSwitchboardVerifyResult::ESummary::Timeout;
						return Result;
					}
				}
			}

			FUTF8ToTCHAR Stdout(reinterpret_cast<const ANSICHAR*>(Verify.StdoutBuf.GetData()), Verify.StdoutBuf.Num());
			FStringView StdoutSV(Stdout.Get(), Stdout.Length());

			if (ReturnCode != 0)
			{
				UE_LOGF(LogSwitchboardPlugin, Warning, "RunVerify: Returned non-zero exit status %d", ReturnCode.GetValue());
				Result.Summary = FSwitchboardVerifyResult::ESummary::UnexpectedExitStatus;
				Result.Log = StdoutSV;
				return Result;
			}

			Result.SetFromJson(StdoutSV);
			return Result;
		});
}


void FSwitchboardVerifyResult::SetFromJson(FStringView JsonSV)
{
	TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(FString(JsonSV));
	TSharedPtr< FJsonObject > JsonObject;
	if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) || !JsonObject.IsValid())
	{
		UE_LOGF(LogSwitchboardPlugin, Warning, "FSwitchboardVerifyResult::SetFromJson: Couldn't deserialize");
		this->Summary = ESummary::InvalidOutput;
		return;
	}

	if (!JsonObject->TryGetStringField(TEXT("log"), this->Log))
	{
		UE_LOGF(LogSwitchboardPlugin, Error, "FSwitchboardVerifyResult: 'log' field missing");
		this->Log.Empty();
	}

	bool bSuccess = false;
	if (!JsonObject->TryGetBoolField(TEXT("success"), bSuccess))
	{
		UE_LOGF(LogSwitchboardPlugin, Error, "FSwitchboardVerifyResult: 'success' field missing");
		this->Summary = ESummary::InvalidOutput;
		return;
	}

	this->Summary = bSuccess ? ESummary::Success : ESummary::SelfTestFailed;
}
