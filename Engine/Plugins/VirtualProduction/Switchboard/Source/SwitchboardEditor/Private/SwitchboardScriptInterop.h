// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"


class FSwitchboardUtilScript
{
public:
	static TSharedRef<FSwitchboardUtilScript> RunInstall(FStringView VenvPath);

	/** Default constructor uses the engine python interpreter rather than a venv. */
	FSwitchboardUtilScript();
	explicit FSwitchboardUtilScript(FStringView PythonVenvDir);

	~FSwitchboardUtilScript();

	bool Run(const FString& Args);

	TOptional<int32> PollStdoutAndReturnCode();

public:
	FString PythonExe;
	FProcHandle ProcHandle;
	void* StdoutChildWritePipe = nullptr;
	void* StdoutParentReadPipe = nullptr;
	TArray<uint8> StdoutBuf;
	TOptional<int32> ReturnCode;
};


struct FSwitchboardVerifyResult
{
	static TFuture<FSwitchboardVerifyResult> RunVerify(FStringView VenvPath);

	enum class ESummary
	{
		InterpreterMissing,
		LaunchFailed,
		Timeout,
		UnexpectedExitStatus,
		InvalidOutput,
		SelfTestFailed,

		Success,
	};

	void SetFromJson(FStringView JsonSV);

	ESummary Summary;
	FString Log;
};
