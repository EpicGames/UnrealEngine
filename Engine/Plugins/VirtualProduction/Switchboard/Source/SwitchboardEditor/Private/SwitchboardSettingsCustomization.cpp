// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardSettingsCustomization.h"
#include "SwitchboardEditorModule.h"
#include "SwitchboardScriptInterop.h"
#include "SwitchboardEditorSettings.h"
#include "SwitchboardEditorStyle.h"
#include "SwitchboardSetupWizard.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ISettingsModule.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"


#define LOCTEXT_NAMESPACE "SwitchboardEditor"


// static
TSharedRef<IDetailCustomization> FSwitchboardEditorSettingsCustomization::MakeInstance()
{
	return MakeShared<FSwitchboardEditorSettingsCustomization>();
}


FText FSwitchboardEditorSettingsCustomization::GetHealthRowHintText() const
{
	switch (FSwitchboardEditorModule::Get().GetSwitchboardInstallState())
	{
		case FSwitchboardEditorModule::ESwitchboardInstallState::Nominal:
		case FSwitchboardEditorModule::ESwitchboardInstallState::ShortcutsMissing:
			return LOCTEXT("ReinstallEnvHint", "Switchboard requires additional library/framework dependencies to launch, which are installed. If you encounter issues launching Switchboard, it may help to reinstall them.");
		case FSwitchboardEditorModule::ESwitchboardInstallState::NeedInstallOrRepair:
		default:
			return LOCTEXT("InstallEnvHint", "Switchboard requires additional library/framework dependencies to launch, please install them.");
	}
}


FText FSwitchboardEditorSettingsCustomization::GetHealthRowButtonText() const
{
	switch (FSwitchboardEditorModule::Get().GetSwitchboardInstallState())
	{
		case FSwitchboardEditorModule::ESwitchboardInstallState::Nominal:
		case FSwitchboardEditorModule::ESwitchboardInstallState::ShortcutsMissing:
			return LOCTEXT("ReinstallEnvButton", "Reinstall Dependencies");
		case FSwitchboardEditorModule::ESwitchboardInstallState::NeedInstallOrRepair:
		default:
			return LOCTEXT("InstallEnvButton", "Install Dependencies");
	}
}


const FSlateBrush* FSwitchboardEditorSettingsCustomization::GetHealthRowBorderBrush() const
{
	switch (FSwitchboardEditorModule::Get().GetSwitchboardInstallState())
	{
		case FSwitchboardEditorModule::ESwitchboardInstallState::Nominal:
		case FSwitchboardEditorModule::ESwitchboardInstallState::ShortcutsMissing:
			return FSwitchboardEditorStyle::Get().GetBrush("Settings.RowBorder.Nominal");
		case FSwitchboardEditorModule::ESwitchboardInstallState::NeedInstallOrRepair:
		default:
			return FSwitchboardEditorStyle::Get().GetBrush("Settings.RowBorder.Warning");
	}
}


const FSlateBrush* FSwitchboardEditorSettingsCustomization::GetHealthRowIconBrush() const
{
	switch (FSwitchboardEditorModule::Get().GetSwitchboardInstallState())
	{
		case FSwitchboardEditorModule::ESwitchboardInstallState::Nominal:
		case FSwitchboardEditorModule::ESwitchboardInstallState::ShortcutsMissing:
			return FSwitchboardEditorStyle::Get().GetBrush("Settings.Icons.Nominal");
		case FSwitchboardEditorModule::ESwitchboardInstallState::NeedInstallOrRepair:
		default:
			return FSwitchboardEditorStyle::Get().GetBrush("Settings.Icons.Warning");
	}
}


void FSwitchboardEditorSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	// Force the status to refresh whenever the editor settings are closed and reopened.
	const bool bForceRefresh = true;
	FSwitchboardEditorModule::Get().GetVerifyResult(bForceRefresh);

	IDetailCategoryBuilder& SwitchboardCategory = DetailLayout.EditCategory("Switchboard");

	FText HealthFilterString = FText::GetEmpty();
	TSharedPtr<IPropertyHandle> VenvProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(USwitchboardEditorSettings, VirtualEnvironmentPath));
	if (VenvProperty)
	{
		// Filter health rows together with venv property row.
		HealthFilterString = VenvProperty->GetPropertyDisplayName();
	}

	// Install health row.
	SwitchboardCategory.AddCustomRow(HealthFilterString, false)
		.WholeRowWidget
		[
			SNew(SBorder)
			.BorderImage(TAttribute<const FSlateBrush*>::CreateSP(this, &FSwitchboardEditorSettingsCustomization::GetHealthRowBorderBrush))
			.Padding(FMargin(8.0f, 16.0f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
				[
					SNew(SImage)
					.Image(TAttribute<const FSlateBrush*>::CreateSP(this, &FSwitchboardEditorSettingsCustomization::GetHealthRowIconBrush))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(16.0f, 0.0f)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(TAttribute<FText>::CreateSP(this, &FSwitchboardEditorSettingsCustomization::GetHealthRowHintText))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(TAttribute<FText>::CreateSP(this, &FSwitchboardEditorSettingsCustomization::GetHealthRowButtonText))
					.OnClicked_Lambda([this]() {
						SSwitchboardSetupWizard::OpenWindow(SSwitchboardSetupWizard::EWizardPage::Intro);
						return FReply::Handled();
					})
				]
			]
		];

	// Add shortcuts row.
	FDetailWidgetRow& ShortcutsRow = SwitchboardCategory.AddCustomRow(HealthFilterString, false);

	ShortcutsRow.VisibilityAttr.Bind(TAttribute<EVisibility>::FGetter::CreateLambda([]() -> EVisibility
		{
			if (FSwitchboardEditorModule::Get().GetSwitchboardInstallState() ==
				FSwitchboardEditorModule::ESwitchboardInstallState::ShortcutsMissing)
			{
				return EVisibility::Visible;
			}
			else
			{
				return EVisibility::Collapsed;
			}
		}
	));

	ShortcutsRow.WholeRowWidget
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(0.f, 4.f, 0.f, 0.f)
		[
			SNew(SBorder)
			.BorderImage(FSwitchboardEditorStyle::Get().GetBrush("Settings.RowBorder.Warning"))
			.Padding(FMargin(8.0f, 16.0f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
				[
					SNew(SImage)
					.Image(FSwitchboardEditorStyle::Get().GetBrush("Settings.Icons.Warning"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(16.0f, 0.0f)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(LOCTEXT("AddShortcutsHint", "Add shortcuts to launch Switchboard from the Desktop and Start Menu."))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(LOCTEXT("AddShortcutsButton", "Add Shortcuts"))
					.OnClicked_Lambda([this]() {
						SSwitchboardSetupWizard::OpenWindow(SSwitchboardSetupWizard::EWizardPage::Shortcuts);
						return FReply::Handled();
					})
				]
			]
		]
	];

#if SWITCHBOARD_LISTENER_AUTOLAUNCH
	// This checkbox isn't backed by a property; it directly reflects the state
	// of a Windows registry entry, and therefore only exists on Windows.
	const FText& ListenerAutolaunchLabel = LOCTEXT("ListenerAutolaunchLabel", "Launch Switchboard Listener on Login");
	IDetailCategoryBuilder& ListenerCategory = DetailLayout.EditCategory("Switchboard Listener");
	FDetailWidgetRow& AutolaunchRow = ListenerCategory.AddCustomRow(ListenerAutolaunchLabel);
	AutolaunchRow.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(ListenerAutolaunchLabel)
	];

	AutolaunchRow.ValueContent()
	[
		SNew(SCheckBox)
		.IsChecked_Lambda([]()
		{
			return FSwitchboardEditorModule::Get().IsListenerAutolaunchEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
		.OnCheckStateChanged_Lambda([](ECheckBoxState NewAutolaunchState)
		{
			const bool bShouldAutolaunch = NewAutolaunchState == ECheckBoxState::Checked;
			if (bShouldAutolaunch && !FPaths::FileExists(GetDefault<USwitchboardEditorSettings>()->GetListenerPlatformPath()))
			{
				FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ListenerMissing", "Unable to locate SwitchboardListener. Make sure it has been compiled."));
				return;
			}

			FSwitchboardEditorModule::Get().SetListenerAutolaunchEnabled(bShouldAutolaunch);
		})
	];
#endif // #if SWITCHBOARD_LISTENER_AUTOLAUNCH
}


TSharedRef<IDetailCustomization> FSwitchboardProjectSettingsCustomization::MakeInstance()
{
	return MakeShared<FSwitchboardProjectSettingsCustomization>();
}


void FSwitchboardProjectSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	const FText& HintLabel = LOCTEXT("JumpToEditorPrefsHint", "Configure Switchboard to launch on your computer via Editor Preferences.");

	IDetailCategoryBuilder& EditorPrefsLinkCategory = DetailLayout.EditCategory("Configure Switchboard", LOCTEXT("JumpToEditorPrefsCategory", "Configure Switchboard"), ECategoryPriority::Uncommon);
	EditorPrefsLinkCategory.AddCustomRow(HintLabel, false)
		.WholeRowWidget
		[
			SNew(SBorder)
			.BorderImage(FSwitchboardEditorStyle::Get().GetBrush("Settings.RowBorder.Warning"))
			.Padding(FMargin(8.0f, 16.0f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(8.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.WarningWithColor.Large"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(16.0f, 0.0f)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(HintLabel)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(LOCTEXT("JumpToEditorPrefsButtonText", "Take Me There"))
					.OnClicked_Lambda([]() {
						FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "Plugins", "Switchboard");
						return FReply::Handled();
					})
				]
			]
		];
}


#undef LOCTEXT_NAMESPACE
