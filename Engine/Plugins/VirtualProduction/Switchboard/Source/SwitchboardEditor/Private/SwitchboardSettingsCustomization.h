// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"


struct FSlateBrush;


class FSwitchboardEditorSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	FText GetHealthRowHintText() const;
	FText GetHealthRowButtonText() const;
	const FSlateBrush* GetHealthRowBorderBrush() const;
	const FSlateBrush* GetHealthRowIconBrush() const;

	//~ Begin IDetailCustomization Interface
	void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};


class FSwitchboardProjectSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin IDetailCustomization Interface
	void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};
