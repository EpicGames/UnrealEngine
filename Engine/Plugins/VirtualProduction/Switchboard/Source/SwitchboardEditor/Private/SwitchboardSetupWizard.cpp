// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardSetupWizard.h"
#include "Framework/Application/SlateApplication.h"
#include "SwitchboardEditorModule.h"
#include "SwitchboardEditorSettings.h"
#include "SwitchboardEditorStyle.h"
#include "SwitchboardScriptInterop.h"
#include "Dialog/SMessageDialog.h"
#include "Editor.h"
#include "ISinglePropertyView.h"
#include "SPrimaryButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"


#define LOCTEXT_NAMESPACE "SwitchboardEditor"


// static
void SSwitchboardSetupWizard::OpenWindow(EWizardPage StartPage /* = EWizardPage::Intro */)
{
	TSharedRef<SWindow> ModalWindow = SNew(SWindow)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.ClientSize(FVector2D(700, 350));

	const TSharedRef<SSwitchboardSetupWizard> Wizard = SNew(SSwitchboardSetupWizard, ModalWindow, StartPage);
	ModalWindow->SetContent(Wizard);

	GEditor->EditorAddModalWindow(ModalWindow);
}


void SSwitchboardSetupWizard::Construct(const FArguments& InArgs, const TSharedRef<SWindow>& InWindow, EWizardPage InStartPage /* = EWizardPage::Intro */)
{
	StartPage = InStartPage;

	ModalWindow = InWindow;
	ModalWindow->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride::CreateSP(SharedThis(this), &SSwitchboardSetupWizard::Handle_RequestDestroyWindow));

	PageSwitcher = SNew(SWidgetSwitcher);

	PageSwitcher->AddSlot(static_cast<int32>(EWizardPage::Intro))
	[
		Construct_Page_Intro()
	];

	PageSwitcher->AddSlot(static_cast<int32>(EWizardPage::InstallProgress))
	[
		Construct_Page_InstallProgress()
	];

	PageSwitcher->AddSlot(static_cast<int32>(EWizardPage::Shortcuts))
	[
		Construct_Page_Shortcuts()
	];

	PageSwitcher->AddSlot(static_cast<int32>(EWizardPage::Autolaunch))
	[
		Construct_Page_Autolaunch()
	];

	PageSwitcher->AddSlot(static_cast<int32>(EWizardPage::Complete))
	[
		Construct_Page_Complete()
	];

	SwitchToPage(StartPage);

	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage(FSwitchboardEditorStyle::Get().GetBrush("Wizard.Background"))
		]
		+ SOverlay::Slot()
		.Padding(16.0f)
		[
			PageSwitcher.ToSharedRef()
		]
	];
}


TSharedRef<SWidget> SSwitchboardSetupWizard::Construct_Page_Intro()
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Wizard_Intro_TextBlock", "Switchboard requires the following library/framework dependencies to launch:\n\n\t\u25CF  aioquic\n\n\t\u25CF  PySide6\n\n\t\u25CF  python-osc\n\n\t\u25CF  requests\n\n\t\u25CF  six\n\n\t\u25CF  cwRsync\n\n\nUnreal Engine will install the dependencies into the following directory:\n"))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SEditableTextBox)
				.Text(FText::FromString(USwitchboardEditorSettings::GetSwitchboardEditorSettings()->VirtualEnvironmentPath.Path))
				.OnTextCommitted_Lambda([](const FText& NewText, ETextCommit::Type CommitType) { USwitchboardEditorSettings::GetSwitchboardEditorSettings()->VirtualEnvironmentPath.Path = NewText.ToString(); })
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(FMargin(1, 0))
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.FolderClosed"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
		+ SVerticalBox::Slot()
		[
			SNew(SSpacer)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("Wizard_Intro_InstallButton", "Install"))
				.OnClicked(this, &SSwitchboardSetupWizard::Handle_Intro_InstallClicked)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(16.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("Wizard_Intro_CancelButton", "Cancel"))
				.OnClicked_Lambda([this]() {
					ModalWindow->RequestDestroyWindow();
					return FReply::Handled();
				})
			]
		];
}


TSharedRef<SWidget> SSwitchboardSetupWizard::Construct_Page_InstallProgress()
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text_Lambda([this]() {
				switch (GetProgress())
				{
					case EInstallProgress::Running: return LOCTEXT("Wizard_Progress_InstallRunning", "Installation in progress...");
					case EInstallProgress::Succeeded: return LOCTEXT("Wizard_Progress_InstallSucceeded", "Installation completed successfully!");
					case EInstallProgress::Failed: return LOCTEXT("Wizard_Progress_InstallFailed", "Encountered an error during installation. Please refer to the log below for details.");
					case EInstallProgress::Idle:
					default:
						return FText::GetEmpty();
				}
			})
		]
		+ SVerticalBox::Slot()
		.Padding(0.0f, 16.0f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SAssignNew(InstallOutputTextBox, SMultiLineEditableTextBox)
			.IsReadOnly(true)
			.Font(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Log.Normal").Font)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SPrimaryButton)
				.Text_Lambda([this]() { return GetProgress() != EInstallProgress::Failed ? LOCTEXT("Wizard_Progress_ContinueButton", "Continue") : LOCTEXT("Wizard_Progress_ExitButton", "Exit"); })
				.IsEnabled_Lambda([this]() { return GetProgress() != EInstallProgress::Running; })
				.OnClicked(this, &SSwitchboardSetupWizard::Handle_InstallProgress_ContinueClicked)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(16.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("Wizard_Progress_CancelButton", "Cancel"))
				.OnClicked_Lambda([this]() {
					// Handle_RequestDestroyWindow will prompt the user to confirm if interrupting installer
					ModalWindow->RequestDestroyWindow();
					return FReply::Handled();
				})
			]
		];
}


TSharedRef<SWidget> SSwitchboardSetupWizard::Construct_Page_Shortcuts()
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Wizard_Shortcuts_TextBlock", "Create shortcuts for Switchboard and Switchboard Listener:"))
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SSpacer)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(24.0f, 0.0f, 12.0f, 0.0f)
			[
				SAssignNew(DesktopShortcutCheckbox, SCheckBox)
#if SWITCHBOARD_SHORTCUTS
				.IsChecked(ECheckBoxState::Checked)
#else
				.IsEnabled(false)
#endif
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
#if SWITCHBOARD_SHORTCUTS
				.Text(LOCTEXT("Wizard_Shortcuts_DesktopCheckboxLabel", "On the Desktop"))
#else
				.IsEnabled(false)
				.Text(LOCTEXT("Wizard_Shortcuts_DesktopCheckboxLabel_Unsupported", "On the Desktop\n(Not supported on this platform)"))
#endif
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(24.0f, 0.0f, 12.0f, 0.0f)
			[
				SAssignNew(ProgramsShortcutCheckbox, SCheckBox)
#if SWITCHBOARD_SHORTCUTS
				.IsChecked(ECheckBoxState::Checked)
#else
				.Visibility(EVisibility::Collapsed)
#endif
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Wizard_Shortcuts_StartMenuCheckboxLabel", "In the Start Menu"))
#if !SWITCHBOARD_SHORTCUTS
				.Visibility(EVisibility::Collapsed)
#endif
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(3.0f)
		[
			SNew(SSpacer)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("Wizard_Shortcuts_CreateButton", "Create"))
				.OnClicked_Lambda([this]() {
#if SWITCHBOARD_SHORTCUTS
					if (DesktopShortcutCheckbox->IsChecked())
					{
						FSwitchboardEditorModule::Get().CreateOrUpdateShortcut(FSwitchboardEditorModule::EShortcutApp::Switchboard, FSwitchboardEditorModule::EShortcutLocation::Desktop);
						FSwitchboardEditorModule::Get().CreateOrUpdateShortcut(FSwitchboardEditorModule::EShortcutApp::Listener, FSwitchboardEditorModule::EShortcutLocation::Desktop);
					}

					if (ProgramsShortcutCheckbox->IsChecked())
					{
						FSwitchboardEditorModule::Get().CreateOrUpdateShortcut(FSwitchboardEditorModule::EShortcutApp::Switchboard, FSwitchboardEditorModule::EShortcutLocation::Programs);
						FSwitchboardEditorModule::Get().CreateOrUpdateShortcut(FSwitchboardEditorModule::EShortcutApp::Listener, FSwitchboardEditorModule::EShortcutLocation::Programs);
					}
#endif

					// If the user just wanted to create shortcuts, we end here.
					if (StartPage == EWizardPage::Shortcuts)
					{
						ModalWindow->RequestDestroyWindow();
						return FReply::Handled();
					}

					SwitchToPage(EWizardPage::Autolaunch);
					return FReply::Handled();
				})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(16.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("Wizard_Shortcuts_SkipButton", "Skip"))
				.OnClicked_Lambda([this]() {
					// If the user just wanted to create shortcuts, we end here.
					if (StartPage == EWizardPage::Shortcuts)
					{
						ModalWindow->RequestDestroyWindow();
						return FReply::Handled();
					}

					SwitchToPage(EWizardPage::Autolaunch);
					return FReply::Handled();
				})
			]
		];
}


TSharedRef<SWidget> SSwitchboardSetupWizard::Construct_Page_Autolaunch()
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Wizard_Autolaunch_TextBlock", "Would you like to automatically launch Switchboard Listener on login?"))
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SSpacer)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(24.0f, 0.0f, 12.0f, 0.0f)
			[
				SAssignNew(AutolaunchCheckbox, SCheckBox)
#if SWITCHBOARD_LISTENER_AUTOLAUNCH
				.IsChecked(FSwitchboardEditorModule::Get().IsListenerAutolaunchEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
#else
				.IsEnabled(false)
#endif
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
#if SWITCHBOARD_LISTENER_AUTOLAUNCH
				.Text(LOCTEXT("Wizard_Autolaunch_CheckboxLabel", "Launch Switchboard Listener on Login"))
#else
				.IsEnabled(false)
				.Text(LOCTEXT("Wizard_Autolaunch_CheckboxLabel_Unsupported", "Launch Switchboard Listener on Login\n(Not supported on this platform)"))
#endif
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(3.0f)
		[
			SNew(SSpacer)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("Wizard_Autolaunch_ContinueButton", "Continue"))
				.OnClicked_Lambda([this]() {
#if SWITCHBOARD_LISTENER_AUTOLAUNCH
					FSwitchboardEditorModule::Get().SetListenerAutolaunchEnabled(AutolaunchCheckbox->IsChecked());
#endif
					SwitchToPage(EWizardPage::Complete);
					return FReply::Handled();
				})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(16.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("Wizard_Autolaunch_SkipButton", "Skip"))
				.OnClicked_Lambda([this]() {
					SwitchToPage(EWizardPage::Complete);
					return FReply::Handled();
				})
			]
		];
}


TSharedRef<SWidget> SSwitchboardSetupWizard::Construct_Page_Complete()
{
		return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Wizard_Complete_TextBlock", "Installation complete - Switchboard is now ready to launch!"))
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SSpacer)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("Wizard_Complete_LaunchButton", "Launch Switchboard"))
				.OnClicked_Lambda([this]() {
					FSwitchboardEditorModule::Get().LaunchSwitchboard();
					ModalWindow->RequestDestroyWindow();
					return FReply::Handled();
				})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(16.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("Wizard_Complete_CloseButton", "Close"))
				.OnClicked_Lambda([this]() {
					ModalWindow->RequestDestroyWindow();
					return FReply::Handled();
				})
			]
		];
}


void SSwitchboardSetupWizard::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (RunningInstall)
	{
		RunningInstall->PollStdoutAndReturnCode();

		const int32 StdoutBufLen = RunningInstall->StdoutBuf.Num();
		if (StdoutBufLen != LastStdoutNumBytes)
		{
			LastStdoutNumBytes = StdoutBufLen;
			FUTF8ToTCHAR Stdout(reinterpret_cast<const ANSICHAR*>(RunningInstall->StdoutBuf.GetData()), StdoutBufLen);
			InstallOutputTextBox->SetText(FText::FromStringView(FStringView(Stdout.Get(), Stdout.Length())));
		}
	}
}


void SSwitchboardSetupWizard::SwitchToPage(EWizardPage NewPage)
{
	PageSwitcher->SetActiveWidgetIndex(static_cast<int32>(NewPage));
	ModalWindow->SetTitle(GetTitleForPage(NewPage));
}


FText SSwitchboardSetupWizard::GetTitleForPage(EWizardPage Page)
{
	switch (Page)
	{
		case EWizardPage::Intro: return LOCTEXT("Wizard_Intro_WindowTitle", "Install Switchboard Dependencies");
		case EWizardPage::InstallProgress: return LOCTEXT("Wizard_InstallProgress_WindowTitle", "Install Switchboard Dependencies");
		case EWizardPage::Shortcuts: return LOCTEXT("Wizard_Shortcuts_WindowTitle", "Add Switchboard Shortcuts");
		case EWizardPage::Autolaunch: return LOCTEXT("Wizard_Autolaunch_WindowTitle", "Add Switchboard Shortcuts");
		case EWizardPage::Complete: return LOCTEXT("Wizard_Complete_WindowTitle", "Switchboard Installation Complete");
		default:
			ensure(false);
			return FText::GetEmpty();
	}
}


SSwitchboardSetupWizard::EInstallProgress SSwitchboardSetupWizard::GetProgress() const
{
	if (!RunningInstall)
	{
		return EInstallProgress::Idle;
	}
	else if (!RunningInstall->ReturnCode.IsSet())
	{
		return EInstallProgress::Running;
	}
	else if (RunningInstall->ReturnCode == 0)
	{
		return EInstallProgress::Succeeded;
	}
	else // ReturnCode != 0
	{
		return EInstallProgress::Failed;
	}
}


bool SSwitchboardSetupWizard::ConfirmAndInterruptInstall()
{
	TSharedRef<SMessageDialog> ConfirmDialog = SNew(SMessageDialog)
		.Title(FText(LOCTEXT("Wizard_ConfirmInterruptInstall_WindowTitle", "Cancel Switchboard Installation?")))
		.Message(LOCTEXT("Wizard_ConfirmInterruptInstall_TextBlock", "Are you sure you want to cancel installation?\n\nSwitchboard requires these additional library/framework dependencies to launch."))
		.Buttons({
			SCustomDialog::FButton(LOCTEXT("Wizard_ConfirmInterruptInstall_CancelInstallButton", "Cancel Installation")),
			SCustomDialog::FButton(LOCTEXT("Wizard_ConfirmInterruptInstall_ContinueInstallButton", "Continue Installation")),
		});

	const int32 ResultButtonIdx = ConfirmDialog->ShowModal();
	if (ResultButtonIdx == 0 && GetProgress() == EInstallProgress::Running)
	{
		FPlatformProcess::TerminateProc(RunningInstall->ProcHandle, true);
		return true;
	}

	return false;
}


void SSwitchboardSetupWizard::Handle_RequestDestroyWindow(const TSharedRef<SWindow>& InWindow)
{
	if (GetProgress() == EInstallProgress::Running)
	{
		if (!ConfirmAndInterruptInstall())
		{
			return;
		}
	}

	// Ensure we update the verification any time the wizard closes.
	const bool bVerifyForceRefresh = true;
	FSwitchboardEditorModule::Get().GetVerifyResult(bVerifyForceRefresh);

	FSlateApplicationBase::Get().RequestDestroyWindow(InWindow);
}


FReply SSwitchboardSetupWizard::Handle_Intro_InstallClicked()
{
	check(!RunningInstall.IsValid());

	const FString VenvPath = USwitchboardEditorSettings::GetSwitchboardEditorSettings()->VirtualEnvironmentPath.Path;
	RunningInstall = FSwitchboardUtilScript::RunInstall(VenvPath);

	SwitchToPage(EWizardPage::InstallProgress);

	return FReply::Handled();
}


FReply SSwitchboardSetupWizard::Handle_InstallProgress_ContinueClicked()
{
	if (GetProgress() == EInstallProgress::Succeeded)
	{
		SwitchToPage(EWizardPage::Shortcuts);
	}
	else if (GetProgress() == EInstallProgress::Failed)
	{
		ModalWindow->RequestDestroyWindow();
	}

	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
