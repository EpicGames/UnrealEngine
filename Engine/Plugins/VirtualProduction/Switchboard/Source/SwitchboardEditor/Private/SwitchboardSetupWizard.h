// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"


class FSwitchboardUtilScript;
class SCheckBox;
class SMultiLineEditableTextBox;
class SWidgetSwitcher;


class SSwitchboardSetupWizard : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SSwitchboardSetupWizard) {}
	SLATE_END_ARGS()

public:
	enum class EWizardPage
	{
		Intro,
		InstallProgress,
		Shortcuts,
		Autolaunch,
		Complete,
	};

	enum class EInstallProgress
	{
		Idle,
		Running,
		Succeeded,
		Failed,
	};

	static void OpenWindow(EWizardPage StartPage = EWizardPage::Intro);

	void Construct(const FArguments& InArgs, const TSharedRef<SWindow>& InWindow, EWizardPage InStartPage = EWizardPage::Intro);

protected:
	TSharedRef<SWidget> Construct_Page_Intro();
	TSharedRef<SWidget> Construct_Page_InstallProgress();
	TSharedRef<SWidget> Construct_Page_Shortcuts();
	TSharedRef<SWidget> Construct_Page_Autolaunch();
	TSharedRef<SWidget> Construct_Page_Complete();

	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void SwitchToPage(EWizardPage NewPage);
	FText GetTitleForPage(EWizardPage Page);
	EInstallProgress GetProgress() const;

	bool ConfirmAndInterruptInstall();

	void Handle_RequestDestroyWindow(const TSharedRef<SWindow>& InWindow);

	FReply Handle_Intro_InstallClicked();
	FReply Handle_InstallProgress_ContinueClicked();

protected:
	EWizardPage StartPage;
	TSharedPtr<SWindow> ModalWindow;
	TSharedPtr<SWidgetSwitcher> PageSwitcher;

	int32 LastStdoutNumBytes = 0;
	TSharedPtr<SMultiLineEditableTextBox> InstallOutputTextBox;
	TSharedPtr<SCheckBox> AutolaunchCheckbox;
	TSharedPtr<SCheckBox> DesktopShortcutCheckbox;
	TSharedPtr<SCheckBox> ProgramsShortcutCheckbox;

	TSharedPtr<FSwitchboardUtilScript> RunningInstall;
};
