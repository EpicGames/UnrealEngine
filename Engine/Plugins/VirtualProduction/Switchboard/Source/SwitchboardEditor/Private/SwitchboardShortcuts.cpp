// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardShortcuts.h"
#include "Misc/ScopeExit.h"
#include "SwitchboardEditorModule.h"
#include "SwitchboardEditorSettings.h"
#include "Misc/EngineVersion.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include <objbase.h>
#include <ShlGuid.h>
#include <ShlObj.h>
#include "Windows/HideWindowsPlatformTypes.h"


namespace UE::Switchboard::Private::Shorcuts {


FShortcutParams BuildShortcutParams(EShortcutApp App, EShortcutLocation Location)
{
	const TCHAR* BaseName = nullptr;
	switch (App)
	{
		case EShortcutApp::Switchboard: BaseName = TEXT("Switchboard"); break;
		case EShortcutApp::Listener: BaseName = TEXT("Switchboard Listener"); break;
		default:
			checkNoEntry();
			return FShortcutParams();
	}

	FShortcutParams Params;
	Params.Location = Location;
	Params.BaseName = FString::Printf(TEXT("%s (%d.%d)"), BaseName,
		FEngineVersion::Current().GetMajor(), FEngineVersion::Current().GetMinor());

	switch (App)
	{
		case EShortcutApp::Switchboard:
		{
			FString SbPath = FSwitchboardEditorModule::GetSbExePath();
			FString VenvPath = GetDefault<USwitchboardEditorSettings>()->VirtualEnvironmentPath.Path;
			FPaths::MakePlatformFilename(SbPath);
			FPaths::MakePlatformFilename(VenvPath);
			Params.Target = SbPath;
			Params.Args = FString::Printf(TEXT("\"%s\""), *VenvPath);
			break;
		}
		case EShortcutApp::Listener:
		{
			Params.Target = GetDefault<USwitchboardEditorSettings>()->GetListenerPlatformPath();
			Params.Args = GetDefault<USwitchboardEditorSettings>()->ListenerCommandlineArguments;
			break;
		}
		default:
			checkNoEntry();
			return FShortcutParams();
	}

	return Params;
}

bool CreateOrUpdateShortcut(const FShortcutParams& Params)
{
	if (!FWindowsPlatformMisc::CoInitialize())
	{
		return false;
	}

	ON_SCOPE_EXIT { FWindowsPlatformMisc::CoUninitialize(); };

	IShellLink* ShellLink = nullptr;
	HRESULT Hres = CoCreateInstance(CLSID_ShellLink, nullptr, CLSCTX_INPROC_SERVER, IID_IShellLink, reinterpret_cast<void**>(&ShellLink));
	if (FAILED(Hres) || !ShellLink)
	{
		return false;
	}

	ON_SCOPE_EXIT { SAFE_RELEASE(ShellLink); };

	IPersistFile* PersistFile = nullptr;
	Hres = ShellLink->QueryInterface(&PersistFile);
	if (FAILED(Hres) || !PersistFile)
	{
		return false;
	}

	ON_SCOPE_EXIT { SAFE_RELEASE(PersistFile); };

	const FString SaveDir = GetShortcutLocationDir(Params.Location);
	if (SaveDir.IsEmpty())
	{
		return false;
	}

	const FString SavePath = FString::Printf(TEXT("%s\\%s.lnk"), *SaveDir, *Params.BaseName);

	ensure(SUCCEEDED(ShellLink->SetPath(*Params.Target)));
	ensure(SUCCEEDED(ShellLink->SetArguments(*Params.Args)));
	ensure(SUCCEEDED(ShellLink->SetDescription(*Params.Description)));
	ensure(SUCCEEDED(ShellLink->SetIconLocation(*Params.IconPath, 0)));
	if (!ensure(SUCCEEDED(PersistFile->Save(*SavePath, true))))
	{
		return false;
	}

	return true;
}


bool ReadShortcutParams(FShortcutParams& InOutParams)
{
	if (!FWindowsPlatformMisc::CoInitialize())
	{
		return false;
	}

	ON_SCOPE_EXIT { FWindowsPlatformMisc::CoUninitialize(); };

	IShellLink* ShellLink = nullptr;
	HRESULT Hres = CoCreateInstance(CLSID_ShellLink, nullptr, CLSCTX_INPROC_SERVER, IID_IShellLink, reinterpret_cast<void**>(&ShellLink));
	if (FAILED(Hres) || !ShellLink)
	{
		return false;
	}

	ON_SCOPE_EXIT { SAFE_RELEASE(ShellLink); };

	IPersistFile* PersistFile = nullptr;
	Hres = ShellLink->QueryInterface(&PersistFile);
	if (FAILED(Hres) || !PersistFile)
	{
		return false;
	}

	ON_SCOPE_EXIT { SAFE_RELEASE(PersistFile); };

	const FString LoadDir = GetShortcutLocationDir(InOutParams.Location);
	if (LoadDir.IsEmpty())
	{
		return false;
	}

	const FString LoadPath = FString::Printf(TEXT("%s\\%s.lnk"), *LoadDir, *InOutParams.BaseName);

	if (!SUCCEEDED(PersistFile->Load(*LoadPath, STGM_READ)))
	{
		return false;
	}

	if (!SUCCEEDED(ShellLink->Resolve(nullptr, SLR_NO_UI | SLR_NOSEARCH)))
	{
		return false;
	}

	const int32 OutBufLen = 2048;
	TArray<TCHAR> OutBuf;
	OutBuf.SetNumUninitialized(OutBufLen);

	if (ensure(SUCCEEDED(ShellLink->GetPath(OutBuf.GetData(), OutBufLen, nullptr, 0))))
	{
		InOutParams.Target = OutBuf.GetData();
	}
	else
	{
		InOutParams.Target.Empty();
	}

	if (ensure(SUCCEEDED(ShellLink->GetArguments(OutBuf.GetData(), OutBufLen))))
	{
		InOutParams.Args = OutBuf.GetData();
	}
	else
	{
		InOutParams.Args.Empty();
	}

	if (ensure(SUCCEEDED(ShellLink->GetDescription(OutBuf.GetData(), OutBufLen))))
	{
		InOutParams.Description = OutBuf.GetData();
	}
	else
	{
		InOutParams.Description.Empty();
	}

	int32 OutIconIndex = 0;
	if (ensure(SUCCEEDED(ShellLink->GetIconLocation(OutBuf.GetData(), OutBufLen, &OutIconIndex))))
	{
		InOutParams.IconPath = OutBuf.GetData();
	}
	else
	{
		InOutParams.IconPath.Empty();
	}

	return true;
}


EShortcutCompare CompareShortcut(const FShortcutParams& InParams)
{
	FShortcutParams ExistingParams = InParams;
	if (!ReadShortcutParams(ExistingParams))
	{
		return EShortcutCompare::Missing;
	}

	const bool bTargetDiffers = ExistingParams.Target != InParams.Target;
	const bool bArgsDiffer = ExistingParams.Args != InParams.Args;
	const bool bDescDiffers = ExistingParams.Description != InParams.Description;
	const bool bIconDiffers = ExistingParams.IconPath != InParams.IconPath;

	if (bTargetDiffers || bArgsDiffer || bDescDiffers || bIconDiffers)
	{
		return EShortcutCompare::Different;
	}

	return EShortcutCompare::AlreadyExists;
}


FString GetShortcutLocationDir(EShortcutLocation Location)
{
	const KNOWNFOLDERID* KnownFolder = nullptr;
	switch (Location)
	{
		case EShortcutLocation::Desktop: KnownFolder = &FOLDERID_Desktop; break;
		case EShortcutLocation::Programs: KnownFolder = &FOLDERID_Programs; break;
		default: return FString();
	}

	TCHAR* OutKnownFolderPath = nullptr;
	ON_SCOPE_EXIT{ CoTaskMemFree(OutKnownFolderPath); };
	const HRESULT Hres = SHGetKnownFolderPath(*KnownFolder, 0, nullptr, &OutKnownFolderPath);
	if (FAILED(Hres))
	{
		UE_LOGF(LogSwitchboardPlugin, Log, "SHGetKnownFolderPath failed with HRESULT %08X", Hres);
		return FString();
	}

	return FString(OutKnownFolderPath);
}


} // namespace UE::Switchboard::Private::Shorcuts

#endif // #if PLATFORM_WINDOWS
