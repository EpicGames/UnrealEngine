// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_WINDOWS

namespace UE::Switchboard::Private::Shorcuts
{
	enum class EShortcutApp
	{
		Switchboard,
		Listener,
	};

	enum class EShortcutLocation
	{
		Desktop,
		Programs,
	};

	struct FShortcutParams
	{
		EShortcutLocation Location;
		FString BaseName;

		FString Target;
		FString Args;
		FString Description;
		FString IconPath;
	};

	enum class EShortcutCompare
	{
		Missing,
		Different,
		AlreadyExists,
	};

	FShortcutParams BuildShortcutParams(EShortcutApp App, EShortcutLocation Location);
	FString GetShortcutLocationDir(EShortcutLocation Location);

	bool CreateOrUpdateShortcut(const FShortcutParams& Params);
	bool ReadShortcutParams(FShortcutParams& InOutParams);
	EShortcutCompare CompareShortcut(const FShortcutParams& Params);
} // namespace UE::Switchboard::Private::Shorcuts

#endif // #if PLATFORM_WINDOWS