// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SwitchboardTypes.generated.h"

/**
 * Structure that holds the parameters to the sb_script_new_config.py script that will be passed to Switchboard when auto-creating a new config.
 * This structure is saved as a temporary json file and its path is passed to SB in the command line.
 */
USTRUCT()
struct FSwitchboardScriptArguments
{
	GENERATED_BODY()

	/* Optional name of the new config. If left empty, a name will be autogenerated. */
	UPROPERTY()
	FString ConfigName;

	/* Path to the Engine directory so that Switchboard knows which engine to launch */
	UPROPERTY()
	FString EngineDir;

	/* Path to the Project directory so that Switchboard knows which project to launch */
	UPROPERTY()
	FString ProjectPath;

	/* Gamepath to map that Switchboard should launch */
	UPROPERTY()
	FString Map;

	/* Path to nDisplay config that Switchboard should create the nDisplay cluster from */
	UPROPERTY()
	FString DisplayClusterConfigPath;

	/* Number of Editor devices that it should create */
	UPROPERTY()
	int32 NumEditorDevices = 1;

	/* Indicates that Switchboard should override IP addresses with localhost */
	UPROPERTY()
	bool bUseLocalhost = true;

	/* Indicates that Switchboard should auto-connect to the listeners of each created device */
	UPROPERTY()
	bool bAutoConnect = true;

	/* Indicates that Switchboard should delete the script arguments file */
	UPROPERTY()
	bool bAutoDeleteScriptArgsFile = true;
};

/**
 * Structure used to hold a reference of an nDisplay actor for detail customization purposes
 * Note: The reason for not simply using <ADisplayClusterRootActor> is to avoid depending on the nDisplay plugin
 */
USTRUCT(BlueprintType)
struct FDisplayClusterRootActorReference
{
	GENERATED_BODY()

	/** Points to the nDisplay actor */
	UPROPERTY()
	TSoftObjectPtr<AActor> DCRA;
};

/**
 * Structure used to collect New Config options from the user
 */
USTRUCT(BlueprintType)
struct FSwitchboardNewConfigUserOptions
{
	GENERATED_BODY()

	/* Optional name of the new config. If left empty, a name will be autogenerated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General")
	FString ConfigName;

	/* Indicates which nDisplay actor to use */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Devices", meta = (DisplayName = "nDisplay Actor"))
	FDisplayClusterRootActorReference DCRA;

	/* Number of Editor devices that it should create */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Devices", meta = (DisplayName = "Editors"))
	int32 NumEditorDevices = 1;

	/* Indicates that Switchboard should override IP addresses with localhost */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Network")
	bool bUseLocalhost = true;

	/* Indicates that Switchboard should auto-connect to the listeners of each created device */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Network")
	bool bAutoConnect = true;
};
