// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Misc/Paths.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "SwitchboardEditorSettings.generated.h"


UCLASS(Config=Engine, BlueprintType)
class SWITCHBOARDEDITOR_API USwitchboardEditorSettings : public UObject
{
	GENERATED_BODY()

	USwitchboardEditorSettings();

public:
	/** Path to Switchboard's Python virtual environment, where third-party dependencies are installed. */
	UPROPERTY(Config, EditAnywhere, Category="Switchboard")
	FDirectoryPath VirtualEnvironmentPath;

	/** Arguments that should be passed to the Switchboard Listener. */
	UPROPERTY(Config, EditAnywhere, Category="Switchboard Listener")
	FString ListenerCommandlineArguments;

	/** Get Editor Settings object for Switchboard */
	UFUNCTION(BlueprintPure, Category="Switchboard")
	static USwitchboardEditorSettings* GetSwitchboardEditorSettings();

public:
	FString GetListenerPlatformPath() const;
	FString GetListenerInvocation() const;
};
