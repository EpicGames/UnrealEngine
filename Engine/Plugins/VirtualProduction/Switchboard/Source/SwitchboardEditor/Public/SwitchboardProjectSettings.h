// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Misc/Paths.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "SwitchboardProjectSettings.generated.h"


UCLASS(Config=Game, BlueprintType)
class SWITCHBOARDEDITOR_API USwitchboardProjectSettings : public UObject
{
	GENERATED_BODY()

	USwitchboardProjectSettings();

public:
	/** OSC Listener for Switchboard. An OSC server can be started on launch via VPUtilitiesEditor */
	UPROPERTY(Config, EditAnywhere, Category="OSC", meta=(
	          DisplayName="Default Switchboard OSC Listener",
	          Tooltip="The OSC listener for Switchboard. An OSC server can be started on launch via the Virtual Production Editor section of the Project Settings. Switchboard uses port 8000 by default, but this can be configured in your Switchboard config settings."))
	FSoftObjectPath SwitchboardOSCListener = FSoftObjectPath(TEXT("/Switchboard/OSCSwitchboard.OSCSwitchboard"));

	/** Get Project Settings object for Switchboard */
	UFUNCTION(BlueprintPure, Category="Switchboard")
	static USwitchboardProjectSettings* GetSwitchboardProjectSettings();
};
