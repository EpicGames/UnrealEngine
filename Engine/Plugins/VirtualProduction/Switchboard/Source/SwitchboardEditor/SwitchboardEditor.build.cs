// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SwitchboardEditor : ModuleRules
{
	public SwitchboardEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"Blutility",
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"Engine",
				
				"InputCore",
				"Json",
				"JsonUtilities",
				"MessageLog",
				"Projects",
				"PropertyEditor",
				"Settings",
				"Slate",
				"SlateCore",
				"SwitchboardCommon",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
			});

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDefinitions.Add("SWITCHBOARD_SHORTCUTS=1");
		}
		else
		{
			PrivateDefinitions.Add("SWITCHBOARD_SHORTCUTS=0");
		}
	}
}
