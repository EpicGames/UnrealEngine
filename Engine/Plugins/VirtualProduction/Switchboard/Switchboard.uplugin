{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Switchboard",
	"Description": "Launcher/Installer for the Switchboard application.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "SwitchboardCommon",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "SwitchboardEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "OSC",
			"Enabled": true
		},
		{
			"Name": "Takes",
			"Enabled": true
		},
		{
			"Name": "VirtualProductionUtilities",
			"Enabled": true
		}
	]
}